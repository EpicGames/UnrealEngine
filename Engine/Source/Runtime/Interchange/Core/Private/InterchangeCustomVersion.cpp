// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeCustomVersion.h"
#include "Serialization/CustomVersion.h"

const FGuid FInterchangeCustomVersion::GUID("92738C43-2988-4D9C-9A3D-9BBE6EFF9FC0");
FCustomVersionRegistration GRegisterInterchangeCustomVersion(FInterchangeCustomVersion::GUID, FInterchangeCustomVersion::LatestVersion, TEXT("InterchangeAssetImportDataVer"));