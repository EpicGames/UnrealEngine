// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeHelper.h"
#include "InterchangeLogPrivate.h"
#include "HAL/IConsoleManager.h"
#include "Misc/Paths.h"

namespace UE::Interchange
{
	namespace Private
	{
		//List of invalid names copied from Engine/Source/Runtime/Core/Private/Misc/FileHelper.cpp
		//"NONE" and "NUL" were positioned first because they are the most likely to happen by mistake
		static const FString InvalidFilenames[] = {
			TEXT("NONE"), TEXT("NUL"), TEXT("CON"), TEXT("PRN"), TEXT("AUX"), TEXT("CLOCK$"),
			TEXT("COM1"), TEXT("COM2"), TEXT("COM3"), TEXT("COM4"), TEXT("COM5"), TEXT("COM6"), TEXT("COM7"), TEXT("COM8"), TEXT("COM9"),
			TEXT("LPT1"), TEXT("LPT2"), TEXT("LPT3"), TEXT("LPT4"), TEXT("LPT5"), TEXT("LPT6"), TEXT("LPT7"), TEXT("LPT8"), TEXT("LPT9")
		};

		int32 GInterchangeMaxAssetPathLength = 160; // a base value that should suits most cases
		static FAutoConsoleVariableRef CVarInterchangeMaxAssetPathLength(
			TEXT("Interchange.MaxAssetPathLength"),
			GInterchangeMaxAssetPathLength,
			TEXT("Interchange will try to limit asset path length to this value. Default: 160")
		);

		int32 GetUsableMaxAssetPathLength()
		{
			const int32 MinWorkable = 60;
			static int32 ProjectConstraint; // Deduced from os limit and project path
			static bool RunOnce = [&]()
			{
				const FString ProjectContentDir = FPaths::ProjectContentDir();
				const FString FullPathProjectContentDir = FPaths::ConvertRelativePathToFull(ProjectContentDir);
				ProjectConstraint = FPlatformMisc::GetMaxPathLength() - FullPathProjectContentDir.Len();

				if (ProjectConstraint < MinWorkable)
				{
					UE_LOGF(LogInterchangeCore, Error,
							"Interchange can encounter import issues due to a Content path too long, and an OS limitation on path length.\n" "Content path: '%ls'\n" "System max path length: %d.\n"
							, *FullPathProjectContentDir, FPlatformMisc::GetMaxPathLength());
				}

				if (GInterchangeMaxAssetPathLength > ProjectConstraint)
				{
					UE_LOGF(LogInterchangeCore, Warning,
							"The Interchange.MaxAssetPathLength value (%d) is too high for the current setup.\n" "Content path: '%ls'\n" "System max path length: %d.\n"
							, GInterchangeMaxAssetPathLength, *FullPathProjectContentDir, FPlatformMisc::GetMaxPathLength());
				}

				return true;
			}();

			return FMath::Max(FMath::Min(GInterchangeMaxAssetPathLength, ProjectConstraint), MinWorkable);
		}
	}

	void SanitizeName(FString& InOutName, bool bIsJoint)
	{
		if (bIsJoint)
		{
			//For joint we want to replace any space by a dash
			//Partially copied from FSkeletalMeshImportData::FixupBoneName
			InOutName.ReplaceInline(TEXT(" "), TEXT("-"), ESearchCase::CaseSensitive);
			InOutName.ReplaceInline(TEXT("+"), TEXT("_"), ESearchCase::CaseSensitive);
		}

		const TCHAR* InvalidChar = INVALID_OBJECTNAME_CHARACTERS;
		while (*InvalidChar)
		{
			InOutName.ReplaceCharInline(*InvalidChar, TCHAR('_'), ESearchCase::CaseSensitive);
			++InvalidChar;
		}

		static const FString SafeString = FString("Null");

		if (InOutName.IsEmpty())
		{
			InOutName = SafeString;
		}
		else
		{
			//Check against the list of invalid names for files and folder (NONE, CON, PRN, NUL, ...)
			//While only NONE is invalid for FNames, we want to avoid using names that users cannot creates manually in the content browser.
			for (const FString& InvalidName : Private::InvalidFilenames)
			{
				if (InOutName.Equals(InvalidName, ESearchCase::IgnoreCase))
				{
					InOutName = SafeString;
					break;
				}
			}

			//UE-333525 FName edge case where the name is "_<number>" which will be interpreted as a number with a NAME_None string.
			//This can be avoided by calling FName(*InName, 0) instead of FName(*InName) but there are places where FString are required (like DisplayName) that later on will be used as FNames.
			if (InOutName[0] == '_')
			{
				int32 InOutLen = InOutName.Len();
				UE::Core::Private::ParseNumberFromName(*InOutName, InOutLen);

				if (InOutLen == 0)
				{
					InOutName = SafeString + InOutName;
				}
			}
		}
	}

	FString MakeSanitizedName(const FString& InName, bool bIsJoint)
	{
		FString TmpName = InName;
		SanitizeName(TmpName, bIsJoint);
		return TmpName;
	}

	FString MakeName(const FString& InName, bool bIsJoint)
	{
		return MakeSanitizedName(InName, bIsJoint);
	}

	int32 GetAssetNameMaxCharCount(const FString& ParentPackage)
	{
		// can be tweaked, the goal is to be more restrictive than the filesystem
		// so that a project can be shared / moved without breaking the constraint
		int32 MaxAssetPathLength = Private::GetUsableMaxAssetPathLength();

		// internal limit of FNames + room for prefix, separators and null char. (Asset names occur twice in paths)
		int32 InternalNameConstraint = (NAME_SIZE - 100);

		int32 PackageLength = 1 + (!ParentPackage.IsEmpty() ? ParentPackage.Len() : 20);
		int32 Budget = FMath::Min((InternalNameConstraint - PackageLength) / 2, MaxAssetPathLength - PackageLength);
		Budget = FMath::Min(Budget, 255 - 10); // a filename cannot be longer than 255, and we keep a small buffer for the extension
		return FMath::Max(0, Budget);
	}

	FString GetAssetNameWBudget(const FString& DesiredAssetName, int32 CharBudget, TCHAR CharReplacement)
	{
		if (DesiredAssetName.Len() <= CharBudget)
		{
			return DesiredAssetName;
		}

		// Arbitrary number to avoid possible name collisions and having a very low char budget
		if (CharBudget <= 5)
		{
			UE_LOGF(LogInterchangeCore,
				   Warning,
				   "Char budget is too small for generating a new asset name, please adapt the name of the asset manually or try modifying the value of Interchange.MaxAssetPathLength"
			);
			return DesiredAssetName;
		}

		// Space for beginning, replacement and ending
		int32 Remaining = CharBudget - 2;
		int32 LeftCount = Remaining / 2;
		int32 RightCount = Remaining - LeftCount;

		FString Left = DesiredAssetName.Mid(0, LeftCount + 1);
		FString Right = DesiredAssetName.Mid(DesiredAssetName.Len() - (RightCount + 1));

		return Left + CharReplacement + Right;
	}
};