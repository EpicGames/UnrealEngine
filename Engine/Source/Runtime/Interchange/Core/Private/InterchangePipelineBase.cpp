// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePipelineBase.h"

#include "HAL/IConsoleManager.h"
#include "InterchangeLogPrivate.h"
#include "Misc/ConfigCacheIni.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "AnalyticsEventAttribute.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePipelineBase)

namespace UE::Interchange::PipelinePrivate
{	
	FString CreateConfigSectionName(const FName PipelineStackName, UClass* PipelineClass)
	{
		const FString Section = TEXT("Interchange_StackName__") + PipelineStackName.ToString() + TEXT("__PipelineClassName_") + PipelineClass->GetName();
		return Section;
	}

	uint32 GetConfigSectionPipelinePathHash(UInterchangePipelineBase* PipelineAsset)
	{
		uint32 PipelineHash = 0;
		FSoftObjectPath PipelinePath;
#if WITH_EDITORONLY_DATA
		if (PipelineAsset->IsStandAlonePipeline())
		{
			PipelinePath = PipelineAsset->OriginalPipelinePath;
		}
		else if (UInterchangePipelineBase* OuterMostPipeline = PipelineAsset->GetMostPipelineOuter())
		{
			PipelinePath = OuterMostPipeline->OriginalPipelinePath;
		}
#endif

		if (PipelinePath.IsValid())
		{
			PipelineHash = GetTypeHash(PipelinePath.ToString());
		}
		
		return PipelineHash;
	}

	FString CreateConfigSectionName(const FName PipelineStackName, UInterchangePipelineBase* PipelineAsset)
	{
		FString PathString;
		uint32 PipelineHash = GetConfigSectionPipelinePathHash(PipelineAsset);
		if (PipelineHash != 0)
		{
			PathString = FString::Printf(TEXT("__PipelinePathHash_%u"), PipelineHash);
			if (!PipelineAsset->IsStandAlonePipeline())
			{
				PathString = FString::Printf(TEXT("__PipelinePathHash_%u__SubPipelineClassName__%s"), PipelineHash, *PipelineAsset->GetClass()->GetName());
			}
		}

		if (PathString.IsEmpty())
		{
			UE_LOGF(LogInterchangeCore, Warning, "Failed to resolve pipeline's original pipeline path. Using class name based config section name.");
			// Fallback to class based config section name
			PathString = TEXT("__PipelineClassName_") + PipelineAsset->GetClass()->GetName();
		}

		const FString Section = TEXT("Interchange_StackName__") + PipelineStackName.ToString() + PathString;
		return Section;
	}
}

FString UInterchangePipelineBase::GetDefaultConfigFileName()
{
	const static FString PipelineConfigFilename = GEditorPerProjectIni;
	return PipelineConfigFilename;
}

FArchive& operator<<(FArchive& Ar, FInterchangePipelineContextParams& Params)
{
	Ar << Params.ContextType;
	Ar << Params.ImportObjectType;
	Ar << Params.ReimportAsset;
	Ar << Params.BaseNodeContainer;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FInterchangePipelinePropertyStatePerContext& States)
{
	Ar << States.bVisible;
	return Ar;
}

FArchive& operator<<(FArchive& Ar, FInterchangePipelinePropertyStates& States)
{
	Ar << States.bLocked;
	Ar << States.bPreDialogReset;
	Ar << States.BasicLayoutStates;
	Ar << States.ImportStates;
	Ar << States.ReimportStates;
	return Ar;
}

void UInterchangePipelineBase::LoadSettings(const FName PipelineStackName, bool bResetPreDialog /*= false*/)
{
	bool bOutRequiresSaving = false;
	LoadSettingsInternal(PipelineStackName, GetDefaultConfigFileName(), PropertiesStates, bResetPreDialog, bOutRequiresSaving);
	if (bOutRequiresSaving)
	{
		SaveSettings(PipelineStackName);
	}
}

void UInterchangePipelineBase::SaveSettings(const FName PipelineStackName)
{
	SaveSettingsInternal(PipelineStackName, GetDefaultConfigFileName());
	//Flush the config only once, we do not want to flush when recursively save the sub object pipeline.
	constexpr bool bRemoveFromCache = false;
	GConfig->Flush(bRemoveFromCache, GetDefaultConfigFileName());
}

void UInterchangePipelineBase::AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams)
{
	CacheContextParam = ContextParams;
	CachePropertiesStates = PropertiesStates;

	bAllowPropertyStatesEdition = (ContextParams.ContextType == EInterchangePipelineContext::None);
	bIsReimportContext = false;
	switch (ContextParams.ContextType)
	{
	case EInterchangePipelineContext::AssetReimport:
	case EInterchangePipelineContext::AssetAlternateSkinningReimport:
	case EInterchangePipelineContext::AssetCustomLODReimport:
	case EInterchangePipelineContext::SceneReimport:
	{
		bIsReimportContext = true;
	}
	break;
	}
}

void UInterchangePipelineBase::AdjustSettingsFromCache()
{
	PropertiesStates = CachePropertiesStates;
	AdjustSettingsForContext(CacheContextParam);
}

void UInterchangePipelineBase::TransferAdjustSettings(const UInterchangePipelineBase* SourcePipeline)
{
	CacheContextParam = SourcePipeline->CacheContextParam;
	CachePropertiesStates = SourcePipeline->CachePropertiesStates;
	bAllowPropertyStatesEdition = SourcePipeline->bAllowPropertyStatesEdition;
	bIsReimportContext = SourcePipeline->bIsReimportContext;
	bIsShowEssentials = SourcePipeline->bIsShowEssentials;
}

const FInterchangePipelinePropertyStates* UInterchangePipelineBase::GetPropertyStates(const FName PropertyPath) const
{
	return PropertiesStates.Find(PropertyPath);
}

FInterchangePipelinePropertyStates* UInterchangePipelineBase::GetMutablePropertyStates(const FName PropertyPath)
{
	return PropertiesStates.Find(PropertyPath);
}

bool UInterchangePipelineBase::DoesPropertyStatesExist(const FName PropertyPath) const
{
	return PropertiesStates.Contains(PropertyPath);
}

FInterchangePipelinePropertyStates& UInterchangePipelineBase::FindOrAddPropertyStates(const FName PropertyPath)
{
	return PropertiesStates.FindOrAdd(PropertyPath);
}

FName UInterchangePipelineBase::GetPropertiesStatesPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UInterchangePipelineBase, PropertiesStates);
}

FName UInterchangePipelineBase::GetResultsPropertyName()
{
	return GET_MEMBER_NAME_CHECKED(UInterchangePipelineBase, Results);
}

bool UInterchangePipelineBase::CanAddPropertyToAnalytics(TPropertyValueIterator<FProperty>& PropertyValueIterator) const
{
	const FProperty* Property = PropertyValueIterator.Key();
	if (!ensure(Property))
	{
		return false;
	}
	const void* PropertyValue = PropertyValueIterator.Value();

	if (Property->HasAnyPropertyFlags(CPF_Transient))
	{
		return false;
	}

	if (Property->GetFName() == UInterchangePipelineBase::GetPropertiesStatesPropertyName())
	{
		return false;
	}

	if (Property->GetFName() == UInterchangePipelineBase::GetResultsPropertyName())
	{
		return false;
	}

#if WITH_EDITORONLY_DATA
	const FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
	UInterchangePipelineBase* SubPipeline = SubObject 
												? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue(PropertyValue)) 
												: nullptr;
	if (!SubPipeline && Property->GetMetaData("Category").IsEmpty())
	{
		//In Editor do not add property with no category
		return false;
	}
#endif

	return true;
}

void UInterchangePipelineBase::AddPropertyAnalytics(TPropertyValueIterator<FProperty>& PropertyValueIterator, TArray<FAnalyticsEventAttribute>& PipelineAttributes)
{
	const FProperty* Property = PropertyValueIterator.Key();
	const void* Value = PropertyValueIterator.Value();

	TArray<const FProperty*> PropertyChain;
	PropertyValueIterator.GetPropertyChain(PropertyChain);

	const FString PropertyName = [&PropertyChain]() -> FString {
		if (PropertyChain.Num() == 1)
		{
			return PropertyChain[0]->GetName();
		}

		TStringBuilder<256> StringBuilder;	
		for (int32 PropertyIndex = PropertyChain.Num() - 1; PropertyIndex >= 0; --PropertyIndex)
		{
			StringBuilder << PropertyChain[PropertyIndex]->GetName();
			if (PropertyIndex > 0)
			{
				StringBuilder << TEXT(".");
			}
		}

		return StringBuilder.ToString();
	}();
	
	bool bUseDefaultHandler = false;
	constexpr int32 PortFlags = 0;
	if (const FArrayProperty* Array = CastField<FArrayProperty>(Property))
	{
		FScriptArrayHelper ArrayHelper(Array, Value);
		for (int32 i = 0; i < ArrayHelper.Num(); i++)
		{
			FString	ValueString;
			Array->Inner->ExportTextItem_Direct(ValueString, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);

			PipelineAttributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Property%s.%s"), *GetCategoryNameForAnalytics(PropertyChain), *PropertyName), ValueString));
		}
	}
	else if(const FStructProperty* StructProperty = CastField<FStructProperty>(Property))
	{
		if (StructProperty->Struct == TBaseStructure<FVector>::Get()
			|| StructProperty->Struct == TBaseStructure<FRotator>::Get()
			|| StructProperty->Struct == TBaseStructure<FColor>::Get()
			|| StructProperty->Struct == TBaseStructure<FInt32Interval>::Get())
		{
			bUseDefaultHandler = true;
			PropertyValueIterator.SkipRecursiveProperty();
		}
	}
	else
	{
		bUseDefaultHandler = true;
	}

	if (bUseDefaultHandler)
	{
		FString	ValueString;
		Property->ExportText_Direct(ValueString, Value, Value, this, PortFlags);
		PipelineAttributes.Add(FAnalyticsEventAttribute(FString::Printf(TEXT("Property%s.%s"), *GetCategoryNameForAnalytics(PropertyChain), *PropertyName), ValueString));
	}
}

FString UInterchangePipelineBase::GetCategoryNameForAnalytics(const TArray<const FProperty*>& PropertyChain) const
{
	FString CategoryName = FString();
	if (PropertyChain.IsEmpty())
	{
		return CategoryName;
	}


#if WITH_EDITORONLY_DATA
	if (const FProperty* Property = PropertyChain.Last())
	{
		CategoryName = Property->GetMetaData("Category");
		CategoryName.ReplaceCharInline(TEXT('.'), TEXT('_'));
		CategoryName.RemoveSpacesInline();
		CategoryName = TEXT(".") + CategoryName;
	}
#endif

	return CategoryName;
}



void UInterchangePipelineBase::LoadSettingsInternal(const FName PipelineStackName, const FString& ConfigFilename, TMap<FName, FInterchangePipelinePropertyStates>& ParentPropertiesStates, bool bResetPreDialog, bool& bOutRequiresSaving)
{
	int32 PortFlags = 0;
	UClass* Class = this->GetClass();
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//Do not load a transient property
		//Properties that are not editable shouldn't be loaded either (As they shouldn't have been saved to begin with).
		if (Property->HasAnyPropertyFlags(CPF_Transient) || !Property->HasAnyPropertyFlags(CPF_Edit))
		{
			continue;
		}

		FString SectionName = UE::Interchange::PipelinePrivate::CreateConfigSectionName(PipelineStackName, this);
		{
			constexpr bool bForce = false;
			const FConfigSection* Section = GConfig->GetSection(*SectionName, bForce, ConfigFilename);
			if (!Section)
			{
				SectionName = UE::Interchange::PipelinePrivate::CreateConfigSectionName(PipelineStackName, Class);
				bOutRequiresSaving |= true;
			}
		}

		const FName PropertyName = Property->GetFName();
		const FName PropertyPath = FName(Property->GetPathName());
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UInterchangePipelineBase, PropertiesStates))
		{
			continue;
		}
#if WITH_EDITOR
		if (Property->GetBoolMetaData(FName("AlwaysResetToDefault")))
		{
			//Not loading the property value will reset it
			continue;
		}
		
		
#endif //WITH_EDITOR

		if (const FInterchangePipelinePropertyStates* PropertyStates = ParentPropertiesStates.Find(PropertyPath))
		{
			if (PropertyStates->IsPropertyLocked())
			{
				//Skip this locked property
				continue;
			}

			//Some property need to be reset only when loading the import dialog
			if (bResetPreDialog && PropertyStates->IsPropertyPreDialogReset())
			{
				continue;
			}
		}

		FString Key = Property->GetName();
		const bool bIsPropertyInherited = Property->GetOwnerClass() != Class;
		UObject* SuperClassDefaultObject = Class->GetSuperClass()->GetDefaultObject();

		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			constexpr bool bForce = false;
			const FConfigSection* Section = GConfig->GetSection(*SectionName, bForce, ConfigFilename);
			if (Section != nullptr)
			{
				TArray<FConfigValue> List;
				const FName KeyName(*Key, FNAME_Find);
				Section->MultiFind(KeyName, List);

				FScriptArrayHelper_InContainer ArrayHelper(Array, this);
				// Only override default properties if there is something to override them with.
				if (List.Num() > 0)
				{
					ArrayHelper.EmptyAndAddValues(List.Num());
					for (int32 i = List.Num() - 1, c = 0; i >= 0; i--, c++)
					{
						Array->Inner->ImportText_Direct(*List[i].GetValue(), ArrayHelper.GetRawPtr(c), this, PortFlags);
					}
				}
				else
				{
					int32 Index = 0;
					const FConfigValue* ElementValue = nullptr;
					do
					{
						// Add array index number to end of key
						FString IndexedKey = FString::Printf(TEXT("%s[%i]"), *Key, Index);

						// Try to find value of key
						const FName IndexedName(*IndexedKey, FNAME_Find);
						if (IndexedName == NAME_None)
						{
							break;
						}
						ElementValue = Section->Find(IndexedName);

						// If found, import the element
						if (ElementValue != nullptr)
						{
							// expand the array if necessary so that Index is a valid element
							ArrayHelper.ExpandForIndex(Index);
							Array->Inner->ImportText_Direct(*ElementValue->GetValue(), ArrayHelper.GetRawPtr(Index), this, PortFlags);
						}

						Index++;
					} while (ElementValue || Index < ArrayHelper.Num());
				}
			}
		}
		else if (UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue_InContainer(this)) : nullptr)
		{
			// Load the settings if the referenced pipeline is a subobject of ours
			if (SubPipeline->IsInOuter(this))
			{
				SubPipeline->LoadSettingsInternal(PipelineStackName, ConfigFilename, ParentPropertiesStates, bResetPreDialog, bOutRequiresSaving);
			}
		}
		else
		{
			for (int32 i = 0; i < Property->ArrayDim; i++)
			{
				if (Property->ArrayDim != 1)
				{
					Key = FString::Printf(TEXT("%s[%i]"), *Property->GetName(), i);
				}

				FString Value;
				bool bFoundValue = GConfig->GetString(*SectionName, *Key, Value, ConfigFilename);

				if (bFoundValue)
				{
					if (Property->ImportText_Direct(*Value, Property->ContainerPtrToValuePtr<uint8>(this, i), this, PortFlags) == NULL)
					{
						// this should be an error as the properties from the .ini / .int file are not correctly being read in and probably are affecting things in subtle ways
						UE_LOGF(LogInterchangeCore, Error, "UInterchangePipeline (class:%ls) failed to load settings. Property: %ls Value: %ls", *this->GetClass()->GetName(), *Property->GetName(), *Value);
					}
				}
			}
		}
	}
}

void UInterchangePipelineBase::SaveSettingsInternal(const FName PipelineStackName, const FString& ConfigFilename)
{
	int32 PortFlags = 0;
	UClass* Class = this->GetClass();
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//Do not save a transient property
		//Properties that are not editable shouldn't be saved either.
		if (Property->HasAnyPropertyFlags(CPF_Transient) || !Property->HasAnyPropertyFlags(CPF_Edit))
		{
			continue;
		}

		const FName PropertyName = Property->GetFName();
		if (PropertyName == GET_MEMBER_NAME_CHECKED(UInterchangePipelineBase, PropertiesStates))
		{
			continue;
		}

		const FString SectionName = UE::Interchange::PipelinePrivate::CreateConfigSectionName(PipelineStackName, this);
		FString Key = Property->GetName();

		const bool bIsPropertyInherited = Property->GetOwnerClass() != Class;
		UObject* SuperClassDefaultObject = Class->GetSuperClass()->GetDefaultObject();
		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			GConfig->RemoveKeyFromSection(*SectionName, *Key, ConfigFilename);

			FScriptArrayHelper_InContainer ArrayHelper(Array, this);
			for (int32 i = 0; i < ArrayHelper.Num(); i++)
			{
				FString	Buffer;
				Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);
				GConfig->AddToSection(*SectionName, *Key, Buffer, ConfigFilename);
			}
		}
		else if (UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue_InContainer(this)) : nullptr)
		{
			// Save the settings if the referenced pipeline is a subobject of ours
			if (SubPipeline->IsInOuter(this))
			{
				SubPipeline->SaveSettingsInternal(PipelineStackName, ConfigFilename);
			}
		}
		else
		{
			TCHAR TempKey[MAX_SPRINTF] = TEXT("");
			for (int32 Index = 0; Index < Property->ArrayDim; Index++)
			{
				if (Property->ArrayDim != 1)
				{
					FCString::Sprintf(TempKey, TEXT("%s[%i]"), *Property->GetName(), Index);
					Key = TempKey;
				}

				FString	Value;
				Property->ExportText_InContainer(Index, Value, this, this, this, PortFlags);
				GConfig->SetString(*SectionName, *Key, *Value, ConfigFilename);
			}
		}
	}
}

UInterchangePipelineBase* UInterchangePipelineBase::GetMostPipelineOuter() const
{
	UInterchangePipelineBase* Top = (UInterchangePipelineBase*)this;
	for (;;)
	{
		if (UInterchangePipelineBase* CurrentOuter = Cast<UInterchangePipelineBase>(Top->GetOuter()))
		{
			Top = CurrentOuter;
			continue;
		}
		break;
	}
	return Top;
}

FString UInterchangePipelineBase::GetPipelineDisplayName() const
{
	int32 PortFlags = 0;
	UClass* Class = this->GetClass();
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FStrProperty* StringProperty = CastField<FStrProperty>(Property);
		if (!StringProperty)
		{
			continue;
		}
		const FName PropertyName = Property->GetFName();
		if (PropertyName != FName("PipelineDisplayName"))
		{
			continue;
		}
		//We found the property
		FString Value = StringProperty->GetPropertyValue_InContainer(this, 0);
		if (!Value.IsEmpty())
		{
			return Value;
		}
		//Stop field iteration
		break;
	}
	//Did not found a valid DisplayName property, return the name of the object
	return GetName();
}

#if WITH_EDITOR

void UInterchangePipelineBase::InternalToggleVisibilityPropertiesOfMetaDataValue(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, bool bDoTransientSubPipeline, const FString& MetaDataKey, const FString& MetaDataValue, const bool VisibilityState)
{
	if (!OuterMostPipeline)
	{
		return;
	}

	UClass* PipelineClass = Pipeline->GetClass();
	for (FProperty* Property = PipelineClass->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		UInterchangePipelineBase* SubPipeline = SubObject ? Cast<UInterchangePipelineBase>(SubObject->GetObjectPropertyValue_InContainer(Pipeline)) : nullptr;
		const bool bSkipTransient = !bDoTransientSubPipeline || !SubPipeline;
		//Do not save a transient property
		if (bSkipTransient && Property->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		const FName PropertyName = Property->GetFName();
		const FName PropertyPath = FName(Property->GetPathName());

		if (SubPipeline)
		{
			InternalToggleVisibilityPropertiesOfMetaDataValue(OuterMostPipeline, SubPipeline, bDoTransientSubPipeline, MetaDataKey, MetaDataValue, VisibilityState);
		}
		else
		{
			FString CategoryName = Property->GetMetaData(*MetaDataKey);
			if (CategoryName.Equals(MetaDataValue))
			{
				OuterMostPipeline->FindOrAddPropertyStates(PropertyPath).ReimportStates.bVisible = VisibilityState;
				OuterMostPipeline->FindOrAddPropertyStates(PropertyPath).ImportStates.bVisible = VisibilityState;
			}
		}
	}
}

void UInterchangePipelineBase::HidePropertiesOfCategory(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FString& HideCategoryName, bool bDoTransientSubPipeline /*= false*/)
{
	constexpr bool bVisibilityState = false;
	InternalToggleVisibilityPropertiesOfMetaDataValue(OuterMostPipeline, Pipeline, bDoTransientSubPipeline, TEXT("Category"), HideCategoryName, bVisibilityState);
}

void UInterchangePipelineBase::HidePropertiesOfSubCategory(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FString& HideSubCategoryName, bool bDoTransientSubPipeline /*= false*/)
{
	constexpr bool bVisibilityState = false;
	InternalToggleVisibilityPropertiesOfMetaDataValue(OuterMostPipeline, Pipeline, bDoTransientSubPipeline, TEXT("SubCategory"), HideSubCategoryName, bVisibilityState);
}

void UInterchangePipelineBase::HideProperty(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FName& HidePropertyName)
{
	if (!OuterMostPipeline)
	{
		return;
	}

	constexpr bool bVisibilityState = false;
	UClass* PipelineClass = Pipeline->GetClass();
	for (FProperty* Property = PipelineClass->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		//Do not save a transient property
		if (SubObject || Property->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		const FName PropertyName = Property->GetFName();
		const FName PropertyPath = FName(Property->GetPathName());
		if (HidePropertyName != PropertyName)
		{
			continue;
		}
		OuterMostPipeline->FindOrAddPropertyStates(PropertyPath).ImportStates.bVisible = bVisibilityState;
		OuterMostPipeline->FindOrAddPropertyStates(PropertyPath).ReimportStates.bVisible = bVisibilityState;
	}
}

#endif //WITH_EDITOR

struct FWeakObjectPtrData
{
	FString PropertyName;
	FWeakObjectProperty* WeakObjectProperty;
	void* ValuePtr;
};

void GatherObjectAndWeakObjectPtrs(UClass* Class, UObject* Object/*Value*/, TMap<FString, UObject*>& ObjectPtrs, TArray<FWeakObjectPtrData>& WeakObjectPtrs)
{
	for (TFieldIterator<FProperty> It(Class); It; ++It)
	{
		FProperty* Property = *It;

		FString VariableName = Property->GetName();

		if (FWeakObjectProperty* WeakObjectProperty = CastField<FWeakObjectProperty>(Property))
		{
			FWeakObjectPtrData& Data = WeakObjectPtrs.AddDefaulted_GetRef();
			Data.PropertyName = Property->GetName();
			Data.WeakObjectProperty = WeakObjectProperty;
			Data.ValuePtr = Property->ContainerPtrToValuePtr<uint8>(Object);
		}
		else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
		{
			const void* SubValue = Property->ContainerPtrToValuePtr<uint8>(Object);
			UObject* SubObject = ObjectProperty->GetObjectPropertyValue(SubValue);
			if (SubObject)
			{
				FString PropertyName = Property->GetName();
				if (ObjectPtrs.Contains(PropertyName))
				{
					//Prevent circular fetch (ULevel and UWorld will create an infinite loop)
					continue;
				}
				ObjectPtrs.Add(PropertyName, SubObject);

				UClass* SubObjectPropertyClass = SubObject->GetClass();

				GatherObjectAndWeakObjectPtrs(SubObjectPropertyClass, SubObject, ObjectPtrs, WeakObjectPtrs);
			}
		}
	}
}

void UInterchangePipelineBase::UpdateWeakObjectPtrs()
{
	//Fix WeakObjectPtr connections:
	TMap<FString, UObject*> ObjectPtrs;
	TArray<FWeakObjectPtrData> WeakObjectPtrs;
	GatherObjectAndWeakObjectPtrs(GetClass(), this, ObjectPtrs, WeakObjectPtrs);

	for (FWeakObjectPtrData& WeakObjectPtrData : WeakObjectPtrs)
	{
		if (ObjectPtrs.Contains(WeakObjectPtrData.PropertyName))
		{
			WeakObjectPtrData.WeakObjectProperty->SetPropertyValue(WeakObjectPtrData.ValuePtr, ObjectPtrs[WeakObjectPtrData.PropertyName]);
		}
	}
}

void UInterchangePipelineBase::PostDuplicate(bool bDuplicateForPIE)
{
	Super::PostDuplicate(bDuplicateForPIE);

	UpdateWeakObjectPtrs();
}

void UInterchangePipelineBase::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	if (Ar.GetPortFlags() & PPF_Duplicate)
	{
		Ar << bAllowPropertyStatesEdition;
		Ar << bIsReimportContext;
		Ar << bIsShowEssentials;
		Ar << bFromReimportOrOverride;
		Ar << Results;
		Ar << PropertiesStates;
		Ar << CachePropertiesStates;
		Ar << CacheContextParam;
	}
}
