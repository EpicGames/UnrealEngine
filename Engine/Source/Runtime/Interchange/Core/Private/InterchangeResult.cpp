// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeResult.h"
#include "Dom/JsonValue.h"
#include "Dom/JsonObject.h"
#include "Misc/Paths.h"
#include "Serialization/JsonReader.h"
#include "Serialization/JsonWriter.h"
#include "Serialization/JsonSerializer.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeResult)

// @TODO: move these methods out of UInterchangeResult and into InterchangeDispatcherTask?
// Is it useful to expose this functionality at the engine level?

FString UInterchangeResult::ToJson()
{
	TSharedPtr<FJsonObject> MsgObject = MakeShared<FJsonObject>();
	TSharedPtr<FJsonObject> PropertiesObject = MakeShared<FJsonObject>();

	MsgObject->SetStringField(TEXT("Type"), GetClass()->GetPathName());

	TArray<TSharedPtr<FJsonValue>> Properties;
	for (TFieldIterator<FProperty> FieldIterator(GetClass()); FieldIterator; ++FieldIterator)
	{
		FProperty* Field = *FieldIterator;
		FString PropertyName = Field->GetNameCPP();
		FString PropertyValue;
		Field->ExportTextItem_InContainer(PropertyValue, this, nullptr, this, 0);

		TSharedPtr<FJsonObject> Property = MakeShared<FJsonObject>();
		Property->SetStringField(TEXT("Name"), PropertyName);
		Property->SetStringField(TEXT("Value"), PropertyValue);
		Properties.Add(MakeShared<FJsonValueObject>(Property));
	}

	MsgObject->SetArrayField(TEXT("Properties"), Properties);

	FString JsonString;
	TSharedRef<TJsonWriter<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>> JsonWriter = TJsonWriterFactory<TCHAR, TPrettyJsonPrintPolicy<TCHAR>>::Create(&JsonString);
	if (!FJsonSerializer::Serialize(MsgObject.ToSharedRef(), JsonWriter))
	{
		// Error creating the json string 
		return FString();
	}
	return JsonString;
}


UInterchangeResult* UInterchangeResult::FromJson(const FString& JsonString)
{
	TSharedRef<TJsonReader<TCHAR>> Reader = FJsonStringReader::Create(JsonString);

	TSharedPtr<FJsonObject> MsgObject;
	if (!FJsonSerializer::Deserialize(Reader, MsgObject) || !MsgObject.IsValid())
	{
		// Cannot read the json file
		return nullptr;
	}

	FString ClassType;
	if (!MsgObject->TryGetStringField(TEXT("Type"), ClassType))
	{
		// The Json type id key is missing
		return nullptr;
	}

	const TArray<TSharedPtr<FJsonValue>>* Properties = nullptr;
	if (!MsgObject->TryGetArrayField(TEXT("Properties"), Properties))
	{
		// The Json properties array is missing
		return nullptr;
	}

	UClass* ClassToImport = UClass::TryFindTypeSlow<UClass>(ClassType);
	UInterchangeResult* Result = NewObject<UInterchangeResult>(GetTransientPackage(), ClassToImport);

	for (const TSharedPtr<FJsonValue>& Property : *Properties)
	{
		const TSharedPtr<FJsonObject>& PropertyObject = Property->AsObject();

		FString PropertyName;
		FString PropertyValue;

		if (!PropertyObject->TryGetStringField(TEXT("Name"), PropertyName))
		{
			return nullptr;
		}

		if (!PropertyObject->TryGetStringField(TEXT("Value"), PropertyValue))
		{
			return nullptr;
		}

		FProperty* Field = ClassToImport->FindPropertyByName(FName(*PropertyName));
		Field->ImportText_InContainer(*PropertyValue, Result, Result, 0);
	}

	return Result;
}

FText UInterchangeResult::GetMessageLogText() const
{
	FString SourceText = SourceAssetName;

	// Make sure file path is not too long
	const int32 MaximumSourceNameLength = 512;
	if (SourceText.Len() > MaximumSourceNameLength)
	{
		const int32 CountingFromLeftRightChopIndex = SourceText.Len() - (MaximumSourceNameLength-3);
		SourceText = TEXT("...") + SourceText.RightChop(CountingFromLeftRightChopIndex);
	}

	const FText AssetTypeName = FText::FromString(AssetType ? AssetType->GetName() : TEXT("Unknown"));
	FString AssetName = AssetFriendlyName;
	
	if (AssetName.IsEmpty())
	{
		FSoftObjectPath AssetSoftPath(DestinationAssetName);
		AssetName = AssetSoftPath.GetAssetName();
		if (AssetSoftPath.IsSubobject())
		{
			AssetName = FPaths::GetExtension(AssetSoftPath.GetSubPathString());
		}
	}

	return FText::Format(NSLOCTEXT("InterchangeManager", "MessageLog", "[{0} : '{1}', {2}] {3}"), FText::FromString(SourceText), FText::FromString(AssetName), AssetTypeName, GetText());
}
