// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeResultsContainer.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeResultsContainer)


void UInterchangeResultsContainer::Empty()
{
	FScopeLock ScopeLock(&Lock);
	Results.Empty();
}


void UInterchangeResultsContainer::Append(UInterchangeResultsContainer* Other)
{
	FScopeLock ScopeLock(&Lock);
	Results.Append(Other->GetResults());
}


void UInterchangeResultsContainer::Finalize()
{
	check(IsInGameThread());
	for (UInterchangeResult* Result : Results)
	{
		Result->AtomicallyClearInternalFlags(EInternalObjectFlags::Async);
	}
}

