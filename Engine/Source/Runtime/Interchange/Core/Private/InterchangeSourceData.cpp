// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSourceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSourceData)

UInterchangeSourceData::UInterchangeSourceData()
{
	Filename = TEXT("");
	FileContentHashCache.Reset();
}

UInterchangeSourceData::UInterchangeSourceData(FString InFilename)
{
	SetFilename(InFilename);
}

UObject* UInterchangeSourceData::GetContextObjectByTag(const FString& Tag) const
{
	return ContextObjectsByTag.FindRef(Tag);
}

void UInterchangeSourceData::SetContextObjectByTag(const FString& Tag, UObject* Object) const
{
	ContextObjectsByTag.Add(Tag, Object);
}

TArray<FString> UInterchangeSourceData::GetAllContextObjectTags() const
{
	TArray<FString> Result;
	ContextObjectsByTag.GetKeys(Result);
	return Result;
}

void UInterchangeSourceData::RemoveAllContextObjects() const
{
	ContextObjectsByTag.Reset();
}

void UInterchangeSourceData::ComputeFileContentHashCache() const
{
	FileContentHashCache.Reset();
	if (!Filename.IsEmpty())
	{
		// @todo : this is slow and not multi-threaded.  Use FXxHash64::HashBufferChunked instead?

		FileContentHashCache = FMD5Hash::HashFile(*Filename);
	}
}
