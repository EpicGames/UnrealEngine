// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTaskSystem.h"

#include "Async/Async.h"
#include "Async/TaskGraphInterfaces.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "CoreGlobals.h"
#include "HAL/CriticalSection.h"
#include "HAL/Thread.h"
#include "HAL/ThreadManager.h"
#include "InterchangeLogPrivate.h"
#include "Misc/CoreDelegates.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"

bool UE::Interchange::FInterchangeTaskSystem::bIsCreatingSingleton = false;

bool UE::Interchange::FInterchangeTaskBase::SetTaskStatus(EInterchangeTaskStatus NewTaskStatus)
{
	FScopeLock TaskStatusLock(&TaskStatusCriticalSection);
	if (TaskStatus == NewTaskStatus)
	{
		return true;
	}
	//If status is done, we cannot change it
	if (!ensure(TaskStatus != EInterchangeTaskStatus::Done))
	{
		return false;
	}
	TaskStatus = NewTaskStatus;
	return true;
}

void UE::Interchange::FInterchangeTaskBase::SetPrerequisites(const TArray<uint64>& InPrerequisiteTasks)
{
	check(GetTaskStatus() == EInterchangeTaskStatus::Waiting);
	PrerequisiteTasks = InPrerequisiteTasks;
}

void UE::Interchange::FInterchangeTaskBase::Wait() const
{
	FInterchangeTaskSystem::Get().WaitUntilTasksComplete({ TaskId });
}

UE::Interchange::FInterchangeTaskSystem::FInterchangeTaskSystem()
{
	if (!bIsCreatingSingleton)
	{
		ensureMsgf(bIsCreatingSingleton, TEXT("Interchange task system is a singleton you must call UE::Interchange::FInterchangeTaskSystem::Get() to access it."));
	}
}

UE::Interchange::FInterchangeTaskSystem& UE::Interchange::FInterchangeTaskSystem::Get()
{
	static TSharedPtr<FInterchangeTaskSystem, ESPMode::ThreadSafe> TaskSystemPtr;
	static bool bEndOfSingletonLife = false;

	if (!TaskSystemPtr.IsValid())
	{
		//Cannot create the singleton outside of the game thread
		check(IsInGameThread());

		//Ensure before crashing in case this function is call after the engine exit
		if (bEndOfSingletonLife)
		{
			ensure(!bEndOfSingletonLife);
		}

		bIsCreatingSingleton = true;
		TaskSystemPtr = MakeShared<FInterchangeTaskSystem, ESPMode::ThreadSafe>().ToSharedPtr();
		bIsCreatingSingleton = false;



		TaskSystemPtr->TickTickerHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("InterchangeTaskSystemTickHandle"), 0.0f, [](float DeltaTime)
			{
				TaskSystemPtr->Tick();
				return true; //Return true will keep the tick active
			});
			
		FCoreDelegates::OnPreExit.AddLambda([]()
			{
				if (!ensure(TaskSystemPtr.IsValid()))
				{
					return;
				}

				//We cancel any running task when we pre exit the engine
				for (TPair<uint64, TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe>> TaskInfo : TaskSystemPtr->TaskPerIdMap)
				{
					//Cancel any task
					TaskSystemPtr->CancelTask(TaskInfo.Key, false);
				}

				if (TaskSystemPtr->TickTickerHandle.IsValid())
				{
					FTSTicker::RemoveTicker(TaskSystemPtr->TickTickerHandle);
					TaskSystemPtr->TickTickerHandle.Reset();
				}

				//Free the singleton
				bEndOfSingletonLife = true;
				TaskSystemPtr = nullptr;
			});

	}
	//Return the singleton
	check(TaskSystemPtr.IsValid());
	return *TaskSystemPtr.Get();
}

uint64 UE::Interchange::FInterchangeTaskSystem::AddTask(TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task, TArray<uint64>& TaskPrerequisites)
{
	FScopeLock TaskPerIdMapLock(&TaskPerIdMapCriticalSection);
	FScopeLock PriorityValueLock(&PriorityValueCriticalSection);

	Task->SetTaskStatus(EInterchangeTaskStatus::Waiting);
	Task->TaskId = PriorityValue++;
	Task->SetPrerequisites(TaskPrerequisites);
	TaskPerIdMap.Add(Task->TaskId, Task);

	return Task->TaskId;
}

uint64 UE::Interchange::FInterchangeTaskSystem::AddTask(TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task)
{
	TArray<uint64> EmptyArray;
	return AddTask(Task, EmptyArray);
}

UE::Interchange::EInterchangeTaskStatus UE::Interchange::FInterchangeTaskSystem::GetTaskStatus(const uint64 TaskId) const
{
	FScopeLock TaskPerIdMapLock(&TaskPerIdMapCriticalSection);
	if (TaskPerIdMap.Contains(TaskId))
	{
		TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskPerIdMap.FindChecked(TaskId);
		return Task->GetTaskStatus();
	}

	//If task doesn't exist we assume its done
	return EInterchangeTaskStatus::Done;
}

void UE::Interchange::FInterchangeTaskSystem::InternalAddCancelRequestNoLock(const uint64 TaskId, bool bCancelPrerequisites)
{
	if (!TaskPerIdMap.Contains(TaskId))
	{
		//Nothing to cancel
		return;
	}

	TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskPerIdMap.FindChecked(TaskId);
	uint64& CanceltaskIdRequest = CancelTaskRequests.AddDefaulted_GetRef();
	CanceltaskIdRequest = Task->GetTaskId();
	if (bCancelPrerequisites)
	{
		for (uint64 PrerequisiteTaskId : Task->GetPrerequisiteTasks())
		{
			InternalAddCancelRequestNoLock(PrerequisiteTaskId, bCancelPrerequisites);
		}
	}
}

void UE::Interchange::FInterchangeTaskSystem::CancelTask(const uint64 TaskId, bool bCancelPrerequisites)
{
	FScopeLock TaskPerIdMapLock(&TaskPerIdMapCriticalSection);
	FScopeLock CancelTaskRequestsLock(&CancelTaskRequestsCriticalSection);
	InternalAddCancelRequestNoLock(TaskId, bCancelPrerequisites);
}

void UE::Interchange::FInterchangeTaskSystem::WaitUntilTasksComplete(const TArray<uint64>& TasksToComplete)
{
	check(IsInGameThread());
	const FTimespan MicroSecondTimespan = FTimespan::FromMicroseconds(1);
	const float DeltaSecond = static_cast<float>(MicroSecondTimespan.GetTotalSeconds());
	//Loop
	FPlatformProcess::ConditionalSleep([this, &TasksToComplete, &MicroSecondTimespan]()
		{
			LLM_SCOPE_BYNAME(TEXT("Interchange"));
			bool bTasksCompleted = true;
			//Look if all tasks are completed
			{
				FScopeLock TaskPerIdMapLock(&TaskPerIdMapCriticalSection);
				for (uint64 TaskId : TasksToComplete)
				{
					if (TaskPerIdMap.Contains(TaskId))
					{
						TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskPerIdMap.FindChecked(TaskId);
						if (Task->GetTaskStatus() != EInterchangeTaskStatus::Done)
						{
							bTasksCompleted = false;
							break;
						}
					}
				}
			}
			if (!bTasksCompleted)
			{
				//Tick all systems our tasks can depends to avoid stalling the game thread
				
				//We cannot tick the "core ticker" since we are already inside a tick and it will assert.
				//We cannot tick "runnable thread" it create some issue with the garbage collector tasks

				//Tick Interchange task system
				Tick();

				OnTaskSystemTick.Broadcast();
			}
			return bTasksCompleted;
		}, DeltaSecond);
}

void UE::Interchange::FInterchangeTaskSystem::Tick()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FInterchangeTaskSystem::Tick)
	//Tick must be execute on the game thread
	check(IsInGameThread());
	
	//Tick is suppose to be a safe place where we can create and manipulate UObjects
	ensure(!IsGarbageCollecting());
#if WITH_EDITOR
	ensure(!UE::GetIsEditorLoadingPackage());
#endif

	FScopeLock TaskPerIdMapLock(&TaskPerIdMapCriticalSection);

	//Execute the cancel task
	{
		FScopeLock CancelTaskRequestsLock(&CancelTaskRequestsCriticalSection);
		for (uint64 CancelTaskId : CancelTaskRequests)
		{
			if (!TaskPerIdMap.Contains(CancelTaskId))
			{
				continue;
			}
			TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskPerIdMap.FindChecked(CancelTaskId);
			if (Task->GetTaskStatus() == EInterchangeTaskStatus::Executing)
			{
				//Wait until the task is done
				Task->Wait();
			}
			Task->SetTaskStatus(EInterchangeTaskStatus::Done);
		}
	}
	if (TaskPerIdMap.Num() == 0)
	{
		//We do not have any task waiting or running
		ReleaseAndDoneTasks.Reset();
		return;
	}

	TArray<uint64> WaitingGameThreadTick;
	TArray<uint64> WaitingTaskGraph;
	TSet<uint64> DoneTasks;
	for (TPair<uint64, TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe>>& TaskInfo : TaskPerIdMap)
	{
		TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskInfo.Value;
		if (Task->GetTaskStatus() == EInterchangeTaskStatus::Waiting)
		{
			if (Task->GetTaskThread() == EInterchangeTaskThread::GameThread)
			{
				WaitingGameThreadTick.Add(TaskInfo.Key);
			}
			else if (Task->GetTaskThread() == EInterchangeTaskThread::AsyncThread)
			{
				WaitingTaskGraph.Add(TaskInfo.Key);
			}
		}
		else if (Task->GetTaskStatus() == EInterchangeTaskStatus::Done)
		{
			DoneTasks.Add(TaskInfo.Key);
		}
	}

	//Sort all waiting task by priority order, prerequisite task are always added before.
	//The priority is the task id, lower mean must be execute before higher task id
	auto SortLambda = [](const uint64& A, const uint64& B)
		{
			return A < B;
		};

	WaitingGameThreadTick.Sort(SortLambda);
	WaitingTaskGraph.Sort(SortLambda);

	auto UpdateWaitingTask = [this, &DoneTasks](const TArray<uint64>& TaskIds, bool bTimeLimit)
		{
			double StartTime = bTimeLimit ? FPlatformTime::Seconds() : 0.0;
			for (uint64 TaskId : TaskIds)
			{
				bool bAllPrerequisitesCompleted = true;
				TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task = TaskPerIdMap.FindChecked(TaskId);
				const TArray<uint64>& TaskPrerequisites = Task->GetPrerequisiteTasks();
				for (uint64 PrerequisiteTaskId : TaskPrerequisites)
				{
					if (ReleaseAndDoneTasks.Contains(PrerequisiteTaskId)
						|| DoneTasks.Contains(PrerequisiteTaskId))
					{
						continue;
					}
					//if we have one waiting or executing task, prerequisites are not completed
					if (TaskPerIdMap.Contains(PrerequisiteTaskId))
					{
						TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> PrerequisiteTask = TaskPerIdMap.FindChecked(PrerequisiteTaskId);
						if (PrerequisiteTask->GetTaskStatus() != EInterchangeTaskStatus::Done)
						{
							bAllPrerequisitesCompleted = false;
							break;
						}
					}
				}
				//Skip this task if not all prerequisite are completed
				if (!bAllPrerequisitesCompleted)
				{
					continue;
				}

				auto TaskCompletionHandler = [Task]()
					{
						//Only set the task to done if the status is executing
						if (Task->GetTaskStatus() == EInterchangeTaskStatus::Executing)
						{
							Task->SetTaskStatus(EInterchangeTaskStatus::Done);
						}
					};

				if (Task->GetTaskThread() == EInterchangeTaskThread::GameThread)
				{
					//Prerequisites completed, we can execute the task
					Task->SetTaskStatus(EInterchangeTaskStatus::Executing);
					Task->Execute();
					TaskCompletionHandler();
				}
				else if (Task->GetTaskThread() == EInterchangeTaskThread::AsyncThread)
				{
					Task->SetTaskStatus(EInterchangeTaskStatus::Executing);
					Task->Future = Async(EAsyncExecution::TaskGraph
					, [Task]()->uint64 //Task callback
					{
						Task->Execute();
						return Task->GetTaskId();
					}
					, TaskCompletionHandler); //Completion callback
				}

				if (bTimeLimit && (FPlatformTime::Seconds() - StartTime > 0.03333))
				{
					break;
				}
			}
		};

	//Update GameThread tasks with some time limits to not stall the game thread
	constexpr bool bTimeLimitTrue = true;
	UpdateWaitingTask(WaitingGameThreadTick, bTimeLimitTrue);

	//Update TaskGraph tasks without time limits since they are asynchronous.
	constexpr bool bTimeLimitFalse = false;
	UpdateWaitingTask(WaitingTaskGraph, bTimeLimitFalse);

	//Release Done task
	ReleaseAndDoneTasks.Append(DoneTasks);
	for (uint64 TaskId : DoneTasks)
	{
		TaskPerIdMap.Remove(TaskId);
	}
}
