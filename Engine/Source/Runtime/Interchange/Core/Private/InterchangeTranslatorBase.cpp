// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeTranslatorBase.h"

#include "Algo/Find.h"
#include "InterchangeLogPrivate.h"
#include "Misc/ConfigCacheIni.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeTranslatorBase)

namespace UE::Interchange::TranslatorPrivate
{
	FString CreateConfigSectionName(UClass* TranslatorClass)
	{
		FString Section = TEXT("Interchange_Translator_ClassName_") + TranslatorClass->GetName();
		return Section;
	}
}

void UInterchangeTranslatorSettings::LoadSettings()
{
	const FString& ConfigFilename = GEditorPerProjectIni;
	int32 PortFlags = 0;
	UClass* Class = this->GetClass();
	FString SectionName = UE::Interchange::TranslatorPrivate::CreateConfigSectionName(Class);
	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//Do not load a transient property
		if (Property->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}
		FString Key = Property->GetName();
		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			const FConfigSection* Section = GConfig->GetSection(*SectionName, false/*bForce*/, ConfigFilename);
			if (Section != nullptr)
			{
				TArray<FConfigValue> List;
				const FName KeyName(*Key, FNAME_Find);
				Section->MultiFind(KeyName, List);

				FScriptArrayHelper_InContainer ArrayHelper(Array, this);
				// Only override default properties if there is something to override them with.
				if (List.Num() > 0)
				{
					ArrayHelper.EmptyAndAddValues(List.Num());
					for (int32 i = List.Num() - 1, c = 0; i >= 0; i--, c++)
					{
						Array->Inner->ImportText_Direct(*List[i].GetValue(), ArrayHelper.GetRawPtr(c), this, PortFlags);
					}
				}
				else
				{
					int32 Index = 0;
					const FConfigValue* ElementValue = nullptr;
					do
					{
						// Add array index number to end of key
						FString IndexedKey = FString::Printf(TEXT("%s[%i]"), *Key, Index);

						// Try to find value of key
						const FName IndexedName(*IndexedKey, FNAME_Find);
						if (IndexedName == NAME_None)
						{
							break;
						}
						ElementValue = Section->Find(IndexedName);

						// If found, import the element
						if (ElementValue != nullptr)
						{
							// expand the array if necessary so that Index is a valid element
							ArrayHelper.ExpandForIndex(Index);
							Array->Inner->ImportText_Direct(*ElementValue->GetValue(), ArrayHelper.GetRawPtr(Index), this, PortFlags);
						}

						Index++;
					} while (ElementValue || Index < ArrayHelper.Num());
				}
			}
		}
		else
		{
			for (int32 i = 0; i < Property->ArrayDim; i++)
			{
				if (Property->ArrayDim != 1)
				{
					Key = FString::Printf(TEXT("%s[%i]"), *Property->GetName(), i);
				}

				FString Value;
				bool bFoundValue = GConfig->GetString(*SectionName, *Key, Value, ConfigFilename);

				if (bFoundValue)
				{
					if (Property->ImportText_Direct(*Value, Property->ContainerPtrToValuePtr<uint8>(this, i), this, PortFlags) == NULL)
					{
						// this should be an error as the properties from the .ini / .int file are not correctly being read in and probably are affecting things in subtle ways
						UE_LOGF(LogInterchangeCore, Error, "UInterchangeTranslatorSettings (class:%ls) failed to load settings. Property: %ls Value: %ls", *this->GetClass()->GetName(), *Property->GetName(), *Value);
					}
				}
			}
		}
	}
}

void UInterchangeTranslatorSettings::SaveSettings()
{
	const FString& ConfigFilename = GEditorPerProjectIni;
	int32 PortFlags = 0;
	UClass* Class = this->GetClass();
	FString SectionName = UE::Interchange::TranslatorPrivate::CreateConfigSectionName(Class);

	for (FProperty* Property = Class->PropertyLink; Property; Property = Property->PropertyLinkNext)
	{
		//Do not save a transient property
		if (Property->HasAnyPropertyFlags(CPF_Transient))
		{
			continue;
		}

		FString Key = Property->GetName();
		FObjectProperty* SubObject = CastField<FObjectProperty>(Property);
		FArrayProperty* Array = CastField<FArrayProperty>(Property);
		if (Array)
		{
			GConfig->RemoveKeyFromSection(*SectionName, *Key, ConfigFilename);

			FScriptArrayHelper_InContainer ArrayHelper(Array, this);
			for (int32 i = 0; i < ArrayHelper.Num(); i++)
			{
				FString	Buffer;
				Array->Inner->ExportTextItem_Direct(Buffer, ArrayHelper.GetRawPtr(i), ArrayHelper.GetRawPtr(i), this, PortFlags);
				GConfig->AddToSection(*SectionName, *Key, Buffer, ConfigFilename);
			}
		}
		else
		{
			TCHAR TempKey[MAX_SPRINTF] = TEXT("");
			for (int32 Index = 0; Index < Property->ArrayDim; Index++)
			{
				if (Property->ArrayDim != 1)
				{
					FCString::Sprintf(TempKey, TEXT("%s[%i]"), *Property->GetName(), Index);
					Key = TempKey;
				}

				FString	Value;
				Property->ExportText_InContainer(Index, Value, this, this, this, PortFlags);
				GConfig->SetString(*SectionName, *Key, *Value, ConfigFilename);
			}
		}
	}
	GConfig->Flush(0);
}

bool UInterchangeTranslatorBase::CanImportSourceData(const UInterchangeSourceData* InSourceData) const
{
	const bool bIncludeDot = false;
	const FString Extension = FPaths::GetExtension(InSourceData->GetFilename(), bIncludeDot);

	const bool bExtensionMatches =
		Algo::FindByPredicate( GetSupportedFormats(),
		[ &Extension ]( const FString& Format )
		{
			return Format.StartsWith( Extension );
		}) != nullptr;

	return bExtensionMatches;
}

