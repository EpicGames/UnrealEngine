// Copyright Epic Games, Inc. All Rights Reserved.
#include "Nodes/InterchangeBaseNode.h"

#include "CoreMinimal.h"
#include "HAL/IConsoleManager.h"
#include "Misc/SecureHash.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeHelper.h"
#include "InterchangeLogPrivate.h"

static bool GUseNodeUidAsName = false;
static FAutoConsoleVariableRef CvarGUseInterchangeUidAsName(
	TEXT("Interchange.FeatureFlags.UseNodeUidAsName"),
	GUseNodeUidAsName,
	TEXT("Whether to use the InterchangeBaseNode <DisplayName | UniqueUid> as the UObject Name. Useful for debugging."),
	ECVF_Default);

#define INTERCHANGE_USE_NODE_UID_AS_NAME (WITH_EDITOR && !UE_BUILD_SHIPPING)

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeBaseNode)

namespace UE
{
	namespace Interchange
	{

		const FAttributeKey& FBaseNodeStaticData::UniqueIDKey()
		{
			static FAttributeKey AttributeKey(TEXT("__UNQ_ID_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::DisplayLabelKey()
		{
			static FAttributeKey AttributeKey(TEXT("__DSPL_LBL_"));
			return AttributeKey;
		}
		
		const FAttributeKey& FBaseNodeStaticData::NamespaceKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Namespace_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::ParentIDKey()
		{
			static FAttributeKey AttributeKey(TEXT("__PARENT_UID_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::DesiredChildIndexKey()
		{
			static FAttributeKey AttributeKey(TEXT("__DESIRED_CHILD_INDEX_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::IsEnabledKey()
		{
			static FAttributeKey AttributeKey(TEXT("__IS_NBLD_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::TargetAssetIDsKey()
		{
			static FAttributeKey AttributeKey(TEXT("__TARGET_ASSET_IDS_"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::ClassTypeAttributeKey()
		{
			static FAttributeKey AttributeKey(TEXT("__ClassTypeAttribute__"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::AssetNameKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Asset_Name_Key__"));
			return AttributeKey;
		}

		const FAttributeKey& FBaseNodeStaticData::NodeContainerTypeKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Node_Container_Type_Key__"));
			return AttributeKey;
		}

	} //ns Interchange
} //ns UE

UInterchangeBaseNode::UInterchangeBaseNode()
{
	Attributes = MakeShared<UE::Interchange::FAttributeStorage, ESPMode::ThreadSafe>();
	TargetNodes.Initialize(Attributes, UE::Interchange::FBaseNodeStaticData::TargetAssetIDsKey().ToString());
	RegisterAttribute<bool>(UE::Interchange::FBaseNodeStaticData::IsEnabledKey(), true);
	RegisterAttribute<uint8>(UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey(), static_cast<uint8>(EInterchangeNodeContainerType::None));
}

#if INTERCHANGE_USE_NODE_UID_AS_NAME
static void UpdateNameForDebugging(UInterchangeBaseNode* Node)
{
	if (!GUseNodeUidAsName)
	{
		return;
	}

	//For debugging purpose
	const FString Name = Node->GetDisplayLabel() + TEXT(" | ") + Node->GetUniqueID();

	if (Name.Len() < NAME_SIZE && Node->Rename(*Name, nullptr, REN_Test))
	{
		Node->Rename(*Name, nullptr, REN_DontCreateRedirectors | REN_NonTransactional);
	}
}
#endif

void UInterchangeBaseNode::InitializeNode(const FString& UniqueID, const FString& DisplayLabel, const EInterchangeNodeContainerType NodeContainerType)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::UniqueIDKey(), UniqueID, UE::Interchange::EAttributeProperty::NoHash);
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("RegisterAttribute"), UE::Interchange::FBaseNodeStaticData::UniqueIDKey());
	}

	if (!SetDisplayLabel(DisplayLabel))
	{
		LogAttributeStorageErrors(Result, TEXT("RegisterAttribute"), UE::Interchange::FBaseNodeStaticData::DisplayLabelKey());
	}

	Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey(), static_cast<uint8>(NodeContainerType));
	if (!IsAttributeStorageResultSuccess(Result))
	{
		LogAttributeStorageErrors(Result, TEXT("RegisterAttribute"), UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey());
	}

	bIsInitialized = true;

#if INTERCHANGE_USE_NODE_UID_AS_NAME
	UpdateNameForDebugging(this);
#endif
}

FString UInterchangeBaseNode::GetTypeName() const
{
	const FString TypeName = "BaseNode";
	return TypeName;
}

FName UInterchangeBaseNode::GetIconName() const
{
	return NAME_None;
}

#if WITH_EDITOR

FString UInterchangeBaseNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	FString NodeAttributeKeyString = NodeAttributeKey.ToString();
	if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::ParentIDKey())
	{
		KeyDisplayName = TEXT("Parent Unique ID");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::DisplayLabelKey())
	{
		KeyDisplayName = TEXT("Name");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::IsEnabledKey())
	{
		KeyDisplayName = TEXT("Enabled");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::UniqueIDKey())
	{
		KeyDisplayName = TEXT("Unique ID");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::ClassTypeAttributeKey())
	{
		KeyDisplayName = TEXT("Node Class Type");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::AssetNameKey())
	{
		KeyDisplayName = TEXT("Imported Asset Name");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey())
	{
		KeyDisplayName = TEXT("Node Container Type");
	}
	else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::TargetAssetIDsKey())
	{
		KeyDisplayName = TEXT("Target Asset Count");
	}
	else if (NodeAttributeKeyString.StartsWith(UE::Interchange::FBaseNodeStaticData::TargetAssetIDsKey().ToString()))
	{
		KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Target Asset Index "), NodeAttributeKeyString);
	}
	
	return KeyDisplayName;
}

bool UInterchangeBaseNode::ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	//If context is preview we hide internal data
	if (UserInterfaceContext == EInterchangeNodeUserInterfaceContext::Preview)
	{
		if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::ParentIDKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::IsEnabledKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::UniqueIDKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FBaseNodeStaticData::TargetAssetIDsKey())
		{
			return true;
		}
		else if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FBaseNodeStaticData::TargetAssetIDsKey().ToString()))
		{
			return true;
		}
	}

	//Show anything else
	return false;
}

FString UInterchangeBaseNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString CategoryName = "Attributes";
	const FString NodeAttributeString = NodeAttributeKey.ToString();
	if (NodeAttributeString.StartsWith(TEXT("UserDefined_")) && NodeAttributeString.EndsWith(TEXT("_Value")))
	{
		CategoryName = TEXT("Custom Attributes");
	}
	return CategoryName;
}

#endif //WITH_EDITOR

bool UInterchangeBaseNode::HasAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	return Attributes->ContainAttribute(NodeAttributeKey);
}

UE::Interchange::EAttributeTypes UInterchangeBaseNode::GetAttributeType(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	return Attributes->GetAttributeType(NodeAttributeKey);
}

void UInterchangeBaseNode::GetAttributeKeys(TArray<UE::Interchange::FAttributeKey>& AttributeKeys) const
{
	Attributes->GetAttributeKeys(AttributeKeys);
}

bool UInterchangeBaseNode::RemoveAttribute(const FString& NodeAttributeKey)
{
	Attributes->UnregisterAttribute(UE::Interchange::FAttributeKey(NodeAttributeKey));
	return !HasAttribute(UE::Interchange::FAttributeKey(NodeAttributeKey));
}

bool UInterchangeBaseNode::AddBooleanAttribute(const FString& NodeAttributeKey, const bool& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(bool);
}

bool UInterchangeBaseNode::GetBooleanAttribute(const FString& NodeAttributeKey, bool& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(bool);
}

bool UInterchangeBaseNode::AddInt32Attribute(const FString& NodeAttributeKey, const int32& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(int32);
}

bool UInterchangeBaseNode::GetInt32Attribute(const FString& NodeAttributeKey, int32& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(int32);
}

bool UInterchangeBaseNode::AddFloatAttribute(const FString& NodeAttributeKey, const float& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(float);
}

bool UInterchangeBaseNode::GetFloatAttribute(const FString& NodeAttributeKey, float& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(float);
}

bool UInterchangeBaseNode::AddDoubleAttribute(const FString& NodeAttributeKey, const double& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(double);
}

bool UInterchangeBaseNode::GetDoubleAttribute(const FString& NodeAttributeKey, double& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(double);
}

bool UInterchangeBaseNode::AddStringAttribute(const FString& NodeAttributeKey, const FString& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(FString);
}

bool UInterchangeBaseNode::GetStringAttribute(const FString& NodeAttributeKey, FString& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(FString);
}

bool UInterchangeBaseNode::AddGuidAttribute(const FString& NodeAttributeKey, const FGuid& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(FGuid);
}

bool UInterchangeBaseNode::GetGuidAttribute(const FString& NodeAttributeKey, FGuid& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(FGuid);
}

bool UInterchangeBaseNode::AddLinearColorAttribute(const FString& NodeAttributeKey, const FLinearColor& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(FLinearColor);
}

bool UInterchangeBaseNode::GetLinearColorAttribute(const FString& NodeAttributeKey, FLinearColor& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(FLinearColor);
}

bool UInterchangeBaseNode::AddVector2Attribute(const FString& NodeAttributeKey, const FVector2f& Value)
{
	INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(FVector2f);
}

bool UInterchangeBaseNode::GetVector2Attribute(const FString& NodeAttributeKey, FVector2f& OutValue) const
{
	INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(FVector2f);
}

FString UInterchangeBaseNode::GetUniqueID() const
{
	ensure(bIsInitialized);
	FString UniqueID;
	Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::UniqueIDKey()).Get(UniqueID);
	FString Namespace;
	if (GetNamespace(Namespace))
	{
		UniqueID = Namespace + UniqueID;
	}
	return UniqueID;
}

FString UInterchangeBaseNode::GetDisplayLabel() const
{
	ensure(bIsInitialized);
	checkSlow(Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::DisplayLabelKey()));
	FString DisplayLabel;
	Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::DisplayLabelKey()).Get(DisplayLabel);
	return DisplayLabel;
}

bool UInterchangeBaseNode::SetDisplayLabel(const FString& InDisplayLabel)
{
	//Having a non-FName-friendly display label (like "None" or "_123") will most certainly cause issues in the editor when creating any kind of asset.
	//It's preferable to be very defensive and avoid that value completely.
	//This can still be bypassed by using the Attributes system directly.
	FString Sanitized = UE::Interchange::MakeSanitizedName(InDisplayLabel);

	bool bSuccess = false;

	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::DisplayLabelKey(), Sanitized);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::DisplayLabelKey());
		bSuccess = Handle.IsValid();
	}

#if INTERCHANGE_USE_NODE_UID_AS_NAME
	if (bIsInitialized && bSuccess)
	{
		UpdateNameForDebugging(this);
	}
#endif

	return bSuccess;
}

bool UInterchangeBaseNode::GetNamespace(FString& OutNamespace) const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::NamespaceKey()))
	{
		return false;
	}
	UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::NamespaceKey());
	if (Handle.IsValid())
	{
		UE::Interchange::EAttributeStorageResult Result = Handle.Get(OutNamespace);
		return IsAttributeStorageResultSuccess(Result);
	}
	return false;
}

bool UInterchangeBaseNode::SetNamespace(const FString& InNamespace)
{
	if(InNamespace.IsEmpty())
	{
		if(Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::NamespaceKey()))
		{
			Attributes->UnregisterAttribute(UE::Interchange::FBaseNodeStaticData::NamespaceKey());
		}
		return !Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::NamespaceKey());
	}
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::NamespaceKey(), InNamespace);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::NamespaceKey());
		return Handle.IsValid();
	}
	return false;
}

FString UInterchangeBaseNode::GetParentUid() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::ParentIDKey()))
	{
		return InvalidNodeUid();
	}

	FString ParentUniqueID;
	UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::ParentIDKey());
	if(Handle.IsValid())
	{
		Handle.Get(ParentUniqueID);
		return ParentUniqueID;
	}
	return InvalidNodeUid();
}

bool UInterchangeBaseNode::SetParentUid(const FString& ParentUid)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::ParentIDKey(), ParentUid);
	if(IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::ParentIDKey());
		return Handle.IsValid();
	}
	return false;
}

int32 UInterchangeBaseNode::GetDesiredChildIndex() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::DesiredChildIndexKey()))
	{
		return INDEX_NONE;
	}

	int32 ChildIndex;
	UE::Interchange::FAttributeStorage::TAttributeHandle<int32> Handle = Attributes->GetAttributeHandle<int32>(UE::Interchange::FBaseNodeStaticData::DesiredChildIndexKey());
	if (Handle.IsValid())
	{
		Handle.Get(ChildIndex);
		return ChildIndex;
	}
	return INDEX_NONE;
}

bool UInterchangeBaseNode::SetDesiredChildIndex(const int32& ChildIndex)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::DesiredChildIndexKey(), ChildIndex);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<int32> Handle = Attributes->GetAttributeHandle<int32>(UE::Interchange::FBaseNodeStaticData::DesiredChildIndexKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeBaseNode::IsEnabled() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::IsEnabledKey()))
	{
		return false;
	}

	UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FBaseNodeStaticData::IsEnabledKey());
	if (Handle.IsValid())
	{
		bool bIsEnabled = false;
		Handle.Get(bIsEnabled);
		return bIsEnabled;
	}
	return false;
}

bool UInterchangeBaseNode::SetEnabled(const bool bIsEnabled)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::IsEnabledKey(), bIsEnabled);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FBaseNodeStaticData::IsEnabledKey());
		return Handle.IsValid();
	}
	return false;
}

EInterchangeNodeContainerType UInterchangeBaseNode::GetNodeContainerType() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey()))
	{
		return EInterchangeNodeContainerType::None;
	}
	UE::Interchange::FAttributeStorage::TAttributeHandle<uint8> Handle = Attributes->GetAttributeHandle<uint8>(UE::Interchange::FBaseNodeStaticData::NodeContainerTypeKey());
	if (Handle.IsValid())
	{
		uint8 Value = static_cast<uint8>(EInterchangeNodeContainerType::None);
		Handle.Get(Value);
		return static_cast<EInterchangeNodeContainerType>(Value);
	}
	return EInterchangeNodeContainerType::None;
}

FGuid UInterchangeBaseNode::GetHash() const
{
	return Attributes->GetStorageHash();
}

FString UInterchangeBaseNode::GetAssetName() const
{
	FString OutName = GetDisplayLabel();
	if (Attributes->ContainAttribute(UE::Interchange::FBaseNodeStaticData::AssetNameKey()))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::AssetNameKey());
		if (Handle.IsValid())
		{
			FString Value;
			Handle.Get(Value);
			OutName = Value;
		}
	}
	if (OutName.Len() > 256)
	{
		// Compute a 128-bit hash based on the string and use that as a GUID :
		FTCHARToUTF8 Converted(*OutName);
		FMD5 MD5Gen;
		MD5Gen.Update((const uint8*)Converted.Get(), Converted.Length());
		uint32 Digest[4];
		MD5Gen.Final((uint8*)Digest);
		FString GuidStr = FGuid(Digest[0], Digest[1], Digest[2], Digest[3]).ToString(EGuidFormats::Base36Encoded);

		//Put the guid between the first and last 115 charcaters
		OutName = OutName.Left(115) + GuidStr + OutName.Right(115);
	}
	return OutName;
}

bool UInterchangeBaseNode::SetAssetName(const FString& AssetName)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FBaseNodeStaticData::AssetNameKey(), AssetName);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<FString> Handle = Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::AssetNameKey());
		return Handle.IsValid();
	}
	return false;
}

int32 UInterchangeBaseNode::GetTargetNodeCount() const
{
	return TargetNodes.GetCount();
}

void UInterchangeBaseNode::GetTargetNodeUids(TArray<FString>& OutTargetAssets) const
{
	TargetNodes.GetItems(OutTargetAssets);
}

bool UInterchangeBaseNode::AddTargetNodeUid(const FString& AssetUid) const
{
	return TargetNodes.AddItem(AssetUid);
}

bool UInterchangeBaseNode::RemoveTargetNodeUid(const FString& AssetUid) const
{
	return TargetNodes.RemoveItem(AssetUid);
}

FString UInterchangeBaseNode::InvalidNodeUid()
{
	return FString();
}

void UInterchangeBaseNode::Serialize(FArchive& Ar)
{
	UE::Interchange::FAttributeStorage& RefAttributes = *(Attributes.Get());
	Ar << RefAttributes;
	if (Ar.IsLoading())
	{
		//The node is consider Initialize if the UniqueID and the Display label are set properly
		bIsInitialized = (Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::UniqueIDKey()).IsValid() &&
						  Attributes->GetAttributeHandle<FString>(UE::Interchange::FBaseNodeStaticData::DisplayLabelKey()).IsValid());

	}
}

void UInterchangeBaseNode::CompareNodeStorage(const UInterchangeBaseNode* NodeA, const UInterchangeBaseNode* NodeB, TArray<UE::Interchange::FAttributeKey>& RemovedAttributes, TArray<UE::Interchange::FAttributeKey>& AddedAttributes, TArray<UE::Interchange::FAttributeKey>& ModifiedAttributes)
{
	UE::Interchange::FAttributeStorage::CompareStorage(*(NodeA->Attributes), *(NodeB->Attributes), RemovedAttributes, AddedAttributes, ModifiedAttributes);
}

void UInterchangeBaseNode::CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UInterchangeBaseNode* DestinationNode, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys)
{
	UE::Interchange::FAttributeStorage::CopyStorageAttributes(*(SourceNode->Attributes), *(DestinationNode->Attributes), AttributeKeys);
}

void UInterchangeBaseNode::CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys)
{
	UE::Interchange::FAttributeStorage::CopyStorageAttributes(*(SourceNode->Attributes), DestinationStorage, AttributeKeys);
}

void UInterchangeBaseNode::CopyStorageAttributes(const UE::Interchange::FAttributeStorage& SourceStorage, UInterchangeBaseNode* DestinationNode, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys)
{
	UE::Interchange::FAttributeStorage::CopyStorageAttributes(SourceStorage, *(DestinationNode->Attributes), AttributeKeys);
}

void UInterchangeBaseNode::CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage, const TArray<TPair<UE::Interchange::FAttributeKey, UE::Interchange::FAttributeKey>>& SrcDestAttributeKeys)
{
	UE::Interchange::FAttributeStorage::CopyStorageAttributes(*(SourceNode->Attributes), DestinationStorage, SrcDestAttributeKeys);
}

void UInterchangeBaseNode::CopyStorage(const UInterchangeBaseNode* SourceNode, UInterchangeBaseNode* DestinationNode)
{
	*(DestinationNode->Attributes) = *(SourceNode->Attributes);
}

FProperty* InterchangePrivateNodeBase::FindPropertyByPathChecked(TVariant<UObject*, uint8*>& Container, UStruct* Outer, FStringView PropertyPath)
{
	int32 SeparatorIndex;
	FStringView PropertyName;
	FStringView RestOfPropertyPath(PropertyPath);
	FProperty* Property = nullptr;

	do
	{
		if (!RestOfPropertyPath.FindChar(TEXT('.'), SeparatorIndex))
		{
			SeparatorIndex = RestOfPropertyPath.Len();
		}

		PropertyName = FStringView(RestOfPropertyPath.GetData(), SeparatorIndex);

		if (SeparatorIndex < RestOfPropertyPath.Len())
		{
			RestOfPropertyPath = FStringView(RestOfPropertyPath.GetData() + SeparatorIndex + 1, RestOfPropertyPath.Len() - SeparatorIndex - 1);
		}
		else
		{
			RestOfPropertyPath = FStringView();
		}

		Property = FindFieldChecked<FProperty>(Outer, FName(PropertyName.Len(), PropertyName.GetData()));

		if (RestOfPropertyPath.Len() > 0)
		{
			if (FStructProperty* StructProperty = CastField<FStructProperty>(Property))
			{
				Outer = StructProperty->Struct;

				if (Container.IsType<UObject*>())
				{
					Container.Set<uint8*>(Property->ContainerPtrToValuePtr<uint8>(Container.Get<UObject*>()));
				}
				else
				{
					Container.Set<uint8*>(Property->ContainerPtrToValuePtr<uint8>(Container.Get<uint8*>()));
				}
			}
			else if (FObjectProperty* ObjectProperty = CastField<FObjectProperty>(Property))
			{
				UObject* DestObject;
				if (Container.IsType<UObject*>())
				{
					DestObject = *Property->ContainerPtrToValuePtr<UObject*>(Container.Get<UObject*>());
				}
				else
				{
					DestObject = *Property->ContainerPtrToValuePtr<UObject*>(Container.Get<uint8*>());
				}
				Outer = DestObject->GetClass();
				Container.Set<UObject*>(DestObject);
			}
		}
	} while (RestOfPropertyPath.Len() > 0);

	check(Property);
	return Property;
}


