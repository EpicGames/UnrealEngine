// Copyright Epic Games, Inc. All Rights Reserved.
#include "Nodes/InterchangeBaseNodeContainer.h"

#include "CoreMinimal.h"
#include "Misc/FileHelper.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Serialization/LargeMemoryWriter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectIterator.h"
#include "UObject/CoreRedirects.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeBaseNodeContainer)

UInterchangeBaseNodeContainer::UInterchangeBaseNodeContainer()
{

}

FString UInterchangeBaseNodeContainer::AddNode(UInterchangeBaseNode* Node)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::AddNode)
	if (!Node)
	{
		return UInterchangeBaseNode::InvalidNodeUid();
	}
	FString NodeUniqueID = Node->GetUniqueID();
	if (NodeUniqueID == UInterchangeBaseNode::InvalidNodeUid())
	{
		return UInterchangeBaseNode::InvalidNodeUid();
	}

	//Cannot add an node with the same IDs
	if (Nodes.Contains(NodeUniqueID))
	{
		return NodeUniqueID;
	}

	//Copy the node
	Nodes.Add(NodeUniqueID, Node);
	return NodeUniqueID;
}

bool UInterchangeBaseNodeContainer::RemoveFactoryNode(const FString& NodeUniqueID)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::RemoveFactoryNode)
	if (UInterchangeFactoryBaseNode* OldNode = GetFactoryNode(NodeUniqueID))
	{
		Nodes.Remove(NodeUniqueID);
		return true;
	}

	return false;
}

bool UInterchangeBaseNodeContainer::ReplaceFactoryNode(const FString& NodeUniqueID, UInterchangeFactoryBaseNode* NewNode)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::ReplaceNode)
	if (RemoveFactoryNode(NodeUniqueID)) //Check existence and confirm it is FactoryNode
	{
		AddNode(NewNode);
		return true;
	}

	return false;
}

bool UInterchangeBaseNodeContainer::IsNodeUidValid(const FString& NodeUniqueID) const
{
	if (NodeUniqueID == UInterchangeBaseNode::InvalidNodeUid())
	{
		return false;
	}
	return Nodes.Contains(NodeUniqueID);
}

void UInterchangeBaseNodeContainer::SetNamespace(const FString& Namespace, UClass* TargetClass)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::SetNamespace)
	TMap<FString, FString> UniqueIdSwap;
	UniqueIdSwap.Reserve(Nodes.Num());
	//Change the asset unique ID
	for (auto& NodeKeyValue : Nodes)
	{
		UInterchangeBaseNode* Node = NodeKeyValue.Value;
		if(TargetClass && !Node->GetClass()->IsChildOf(TargetClass))
		{
			continue;
		}
		const FString NodeUniqueId = Node->GetUniqueID();
		Node->SetNamespace(Namespace);
		UniqueIdSwap.Add(NodeUniqueId, Node->GetUniqueID());
	}

	//Update all Attributes
	for (auto& NodeKeyValue : Nodes)
	{
		UInterchangeBaseNode* Node = NodeKeyValue.Value;
		const FString NodeUniqueId = Node->GetUniqueID();
		TArray<UE::Interchange::FAttributeKey> AttributeKeys;
		Node->GetAttributeKeys(AttributeKeys);
		for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
		{
			if (Node->GetAttributeType(AttributeKey) != UE::Interchange::EAttributeTypes::String)
			{
				continue;
			}
			if (AttributeKey == UE::Interchange::FBaseNodeStaticData::UniqueIDKey())
			{
				continue;
			}
			FString AttributeValue;
			if (!Node->GetStringAttribute(AttributeKey.Key, AttributeValue))
			{
				continue;
			}
			for(const TPair<FString, FString>& OldAndNewUid : UniqueIdSwap)
			{
				//Replace any reference to the new unique ids.
				if (AttributeValue.Equals(OldAndNewUid.Key))
				{
					Node->AddStringAttribute(AttributeKey.Key, OldAndNewUid.Value);
					break;
				}
			}
		}
	}

	//Update the container keys. Remove and re-add each node we remap.
	for (const TPair<FString, FString>& OldUidToNewUid : UniqueIdSwap)
	{
		if (OldUidToNewUid.Key == UInterchangeBaseNode::InvalidNodeUid())
		{
			continue;
		}
		UInterchangeBaseNode* ToReplaceNode = Nodes.FindRef(OldUidToNewUid.Key);
		if (!ToReplaceNode)
		{
			continue;
		}
		
		Nodes.Remove(OldUidToNewUid.Key);
		AddNode(ToReplaceNode);
	}
}

void UInterchangeBaseNodeContainer::IterateNodes(TFunctionRef<void(const FString&, UInterchangeBaseNode*)> IterationLambda) const
{
	for (auto& NodeKeyValue : Nodes)
	{
		IterationLambda(NodeKeyValue.Key, NodeKeyValue.Value);
	}
}

void UInterchangeBaseNodeContainer::BreakableIterateNodes(TFunctionRef<bool(const FString&, UInterchangeBaseNode*)> IterationLambda) const
{
	for (auto& NodeKeyValue : Nodes)
	{
		if (IterationLambda(NodeKeyValue.Key, NodeKeyValue.Value))
		{
			break;
		}
	}
}

void UInterchangeBaseNodeContainer::GetRoots(TArray<FString>& RootNodes) const
{
	for (auto& NodeKeyValue : Nodes)
	{
		if (NodeKeyValue.Value->GetParentUid() == UInterchangeBaseNode::InvalidNodeUid())
		{
			RootNodes.Add(NodeKeyValue.Key);
		}
	}
}

void UInterchangeBaseNodeContainer::GetNodes(const UClass* ClassNode, TArray<FString>& OutNodes) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::GetNodes)
	OutNodes.Empty();
	IterateNodes([&ClassNode, &OutNodes](const FString& NodeUid, UInterchangeBaseNode* Node)
	{
		if(Node->GetClass()->IsChildOf(ClassNode))
		{
			OutNodes.Add(Node->GetUniqueID());
		}
	});
}

const UInterchangeBaseNode* UInterchangeBaseNodeContainer::GetNode(const FString& NodeUniqueID) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::GetNode)
	if (NodeUniqueID == UInterchangeBaseNode::InvalidNodeUid())
	{
		return nullptr;
	}
	UInterchangeBaseNode* Node = Nodes.FindRef(NodeUniqueID);
	return Node;
}

UInterchangeFactoryBaseNode* UInterchangeBaseNodeContainer::GetFactoryNode(const FString& NodeUniqueID) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::GetFactoryNode)
	if (NodeUniqueID == UInterchangeBaseNode::InvalidNodeUid())
	{
		return nullptr;
	}
	UInterchangeFactoryBaseNode* FactoryNode = Cast< UInterchangeFactoryBaseNode>(Nodes.FindRef(NodeUniqueID));
	return FactoryNode;
}

bool UInterchangeBaseNodeContainer::SetNodeParentUid(const FString& NodeUniqueID, const FString& NewParentNodeUid)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::SetNodeParentUid)
	const bool bClearParent = NewParentNodeUid == UInterchangeBaseNode::InvalidNodeUid();

	UInterchangeBaseNode* Node = Nodes.FindRef(NodeUniqueID);
	if (!Node)
	{
		return false;
	}

	if (!bClearParent && !Nodes.Contains(NewParentNodeUid))
	{
		return false;
	}

	// Remove from previous parent
	const FString PreviousParent = Node->GetParentUid();
	if (PreviousParent != UInterchangeBaseNode::InvalidNodeUid())
	{
		if (TArray<FString>* FoundPreviousChildren = ChildrenCache.Find(PreviousParent))
		{
			FoundPreviousChildren->Remove(NodeUniqueID);
		}
	}

	// Set new parent
	if (bClearParent)
	{
		Node->RemoveAttribute(UE::Interchange::FBaseNodeStaticData::ParentIDKey().Key);
	}
	else
	{
		Node->SetParentUid(NewParentNodeUid);

		//Update the children cache
		TArray<FString>& Children = ChildrenCache.FindOrAdd(NewParentNodeUid);
		Children.Add(NodeUniqueID);
	}

	return true;
}

bool UInterchangeBaseNodeContainer::ClearNodeParentUid(const FString& NodeUniqueID)
{
	return SetNodeParentUid(NodeUniqueID, UInterchangeBaseNode::InvalidNodeUid());
}

bool UInterchangeBaseNodeContainer::SetNodeDesiredChildIndex(const FString& NodeUniqueID, const int32& NewNodeChildIndex)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::SetNodeDesiredChildIndex)
	UInterchangeBaseNode* Node = Nodes.FindRef(NodeUniqueID);
	if (!Node)
	{
		return false;
	}
	int32 OldNodeChildIndex = Node->GetDesiredChildIndex();
	if (OldNodeChildIndex != NewNodeChildIndex)
	{
		Node->SetDesiredChildIndex(NewNodeChildIndex);
		FString ParentNodeUid = Node->GetParentUid();
		if (ParentNodeUid != UInterchangeBaseNode::InvalidNodeUid())
		{
			if (TArray<FString>* ParentChildrenCache = ChildrenCache.Find(ParentNodeUid))
			{
				InternalReorderChildren(*ParentChildrenCache);
			}
		}
	}
	return true;
}

int32 UInterchangeBaseNodeContainer::GetNodeChildrenCount(const FString& NodeUniqueID) const
{
	if (TArray<FString>* CacheChildrenPtr = GetCachedNodeChildrenUids(NodeUniqueID))
	{
		return CacheChildrenPtr->Num();
	}

	TArray<FString> ChildrenUids = GetNodeChildrenUids(NodeUniqueID);
	return ChildrenUids.Num();
}

TArray<FString> UInterchangeBaseNodeContainer::GetNodeChildrenUids(const FString& NodeUniqueID) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::GetNodeChildrenUids)
	if (TArray<FString>* CacheChildrenPtr = GetCachedNodeChildrenUids(NodeUniqueID))
	{
		return *CacheChildrenPtr;
	}

	return {};
}

TArray<FString>* UInterchangeBaseNodeContainer::GetCachedNodeChildrenUids(const FString& NodeUniqueID) const
{
	if (TArray<FString>* CacheChildrenPtr = ChildrenCache.Find(NodeUniqueID))
	{
		return CacheChildrenPtr;
	}

	return nullptr;
}

UInterchangeBaseNode* UInterchangeBaseNodeContainer::GetNodeChildren(const FString& NodeUniqueID, int32 ChildIndex)
{
	return GetNodeChildrenInternal(NodeUniqueID, ChildIndex);
}

const UInterchangeBaseNode* UInterchangeBaseNodeContainer::GetNodeChildren(const FString& NodeUniqueID, int32 ChildIndex) const
{
	return const_cast<UInterchangeBaseNodeContainer*>(this)->GetNodeChildrenInternal(NodeUniqueID, ChildIndex);
}

void UInterchangeBaseNodeContainer::SerializeNodeContainerData(FArchive& Ar)
{
	if (Ar.IsLoading())
	{
		Nodes.Reset();
	}
	int32 NodeCount = Nodes.Num();
	Ar << NodeCount;

	if(Ar.IsSaving())
	{
		//The node name is not serialize since its an attribute inside the node that will be serialize by the node itself
		auto SerializeNodePair = [&Ar](UInterchangeBaseNode* BaseNode)
		{
			FString ClassFullName = BaseNode->GetClass()->GetFullName();
			Ar << ClassFullName;
			BaseNode->Serialize(Ar);
		};

		for (auto NodePair : Nodes)
		{
			SerializeNodePair(NodePair.Value);
		}
	}
	else if(Ar.IsLoading())
	{
		//Find all the potential node class
		TMap<FString, UClass*> ClassPerName;

		TArray<UClass*> Classes;
		GetDerivedClasses(UInterchangeBaseNode::StaticClass(), Classes);

		for (UClass* Class : Classes)
		{
			ClassPerName.Add(Class->GetFullName(), Class);
		}

		// The base class is never returned by GetDerivedClasses but was present before, add it manually.
		ClassPerName.Add(UInterchangeBaseNode::StaticClass()->GetFullName(), UInterchangeBaseNode::StaticClass());

		for (int32 NodeIndex = 0; NodeIndex < NodeCount; ++NodeIndex)
		{
			FString ClassFullName;
			Ar << ClassFullName;

			FCoreRedirectObjectName RedirectedObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(ClassFullName));
			if (RedirectedObjectName.IsValid())
			{
				ClassFullName = RedirectedObjectName.ToString();
			}

			//This cannot fail to make sure we have a healty serialization
			if (!ensure(ClassPerName.Contains(ClassFullName)))
			{
				//We did not successfully serialize the content of the file into the node container
				return;
			}
			UClass* ToCreateClass = ClassPerName.FindChecked(ClassFullName);
			//Create a UInterchangeBaseNode with the proper class
			UInterchangeBaseNode* BaseNode = NewObject<UInterchangeBaseNode>(this, ToCreateClass);
			BaseNode->Serialize(Ar);
			AddNode(BaseNode);
		}
		ComputeChildrenCache();
	}
}

void UInterchangeBaseNodeContainer::SaveToFile(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::SaveToFile)
	FLargeMemoryWriter Ar;
	SerializeNodeContainerData(Ar);
	uint8* ArchiveData = Ar.GetData();
	int64 ArchiveSize = Ar.TotalSize();
	TArray64<uint8> Buffer(ArchiveData, ArchiveSize);
	FFileHelper::SaveArrayToFile(Buffer, *Filename);
}

void UInterchangeBaseNodeContainer::LoadFromFile(const FString& Filename)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::LoadFromFile)
	//All sub object should be gone with the reset
	Nodes.Reset();
	TArray64<uint8> Buffer;
	FFileHelper::LoadFileToArray(Buffer, *Filename);
	uint8* FileData = Buffer.GetData();
	int64 FileDataSize = Buffer.Num();
	if (FileDataSize < 1)
	{
		//Nothing to load from this file
		return;
	}
	//Buffer keep the ownership of the data, the large memory reader is use to serialize the TMap
	FLargeMemoryReader Ar(FileData, FileDataSize);
	SerializeNodeContainerData(Ar);
}

void UInterchangeBaseNodeContainer::ComputeChildrenCache()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::ComputeChildrenCache)
	ResetChildrenCache();
	for (const auto& NodeKeyValue : Nodes)
	{
		//Update the parent cache
		const FString ParentUid = NodeKeyValue.Value->GetParentUid();
		if (!ParentUid.IsEmpty())
		{
			TArray<FString>& Children = ChildrenCache.FindOrAdd(ParentUid);
			Children.Add(NodeKeyValue.Key);
		}
	}
	for (TPair<FString, TArray<FString>>& ParentChildrenCache : ChildrenCache)
	{
		InternalReorderChildren(ParentChildrenCache.Value);
	}
}

void UInterchangeBaseNodeContainer::SetChildrenCache(const TMap<FString, TArray<FString>>& InChildrenCache)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::SetChildrenCache)

	ChildrenCache.Reset();
	ChildrenCache.Append(InChildrenCache);
}

TMap<FString, TArray<FString>>& UInterchangeBaseNodeContainer::GetChildrenCache()
{
	return ChildrenCache;
}

void UInterchangeBaseNodeContainer::InternalReorderChildren(TArray<FString>& Children) const
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeBaseNodeContainer::InternalReorderChildren)
	bool bSortChildren = false;
	const int32 ChildrenCount = Children.Num();
	TMap<int32, FString> NodeChildIndexCache;
	NodeChildIndexCache.Reserve(ChildrenCount);
	for(int32 ChildIndex = 0; ChildIndex < ChildrenCount; ++ChildIndex)
	{
		const FString& ChildUid = Children[ChildIndex];
		const UInterchangeBaseNode* Node = GetNode(ChildUid);
		int32 ChildDesiredIndex = Node ? Node->GetDesiredChildIndex() : INDEX_NONE;
		if (ChildDesiredIndex != INDEX_NONE)
		{
			bSortChildren = true;
		}
		else
		{
			//Keep the order for the INDEX_NONE item
			ChildDesiredIndex = ChildrenCount + ChildIndex;
		}
		NodeChildIndexCache.FindOrAdd(ChildDesiredIndex) = ChildUid;
	}

	//If no indexes was specified, do not sort the children
	if (!bSortChildren)
	{
		return;
	}

	NodeChildIndexCache.KeySort([](const int32& ChildA, const int32& ChildB)
	{
		return ChildA < ChildB;
	});

	Children.Reset();

	for (const TPair<int32, FString>& ChildData : NodeChildIndexCache)
	{
		Children.Add(ChildData.Value);
	}
}

UInterchangeBaseNode* UInterchangeBaseNodeContainer::GetNodeChildrenInternal(const FString& NodeUniqueID, int32 ChildIndex)
{
	TArray<FString> ChildrenUids = GetNodeChildrenUids(NodeUniqueID);
	if (!ChildrenUids.IsValidIndex(ChildIndex))
	{
		return nullptr;
	}

	return Nodes.FindRef(ChildrenUids[ChildIndex]);
}

bool UInterchangeBaseNodeContainer::GetIsAncestor(const FString& NodeUniqueID, const FString& AncestorUID) const
{
	FString CurrentNodeUID = NodeUniqueID;

	while (CurrentNodeUID != UInterchangeBaseNode::InvalidNodeUid())
	{
		UInterchangeBaseNode* Node = Nodes.FindRef(CurrentNodeUID);
		if (!Node)
		{
			break;
		}
		const FString& ParentUID = Node->GetParentUid();

		if (AncestorUID == ParentUID)
		{
			return true;
		}

		CurrentNodeUID = ParentUID;
	}

	return false;
}

void UInterchangeBaseNodeContainer::SetupNode(UInterchangeBaseNode* Node, const FString& NodeUID,
	const FString& DisplayLabel, EInterchangeNodeContainerType ContainerType,
	const FString& ParentNodeUID)
{
	Node->InitializeNode(NodeUID, DisplayLabel, ContainerType);
	
	AddNode(Node);

	if (ParentNodeUID.Len())
	{
		SetNodeParentUid(NodeUID, ParentNodeUID);
	}
}

void UInterchangeBaseNodeContainer::SetupAndReplaceFactoryNode(UInterchangeFactoryBaseNode* Node, const FString& NodeUID,
	const FString& DisplayLabel, EInterchangeNodeContainerType ContainerType,
	const FString& OldNodeUID,
	const FString& ParentNodeUID)
{
	if (RemoveFactoryNode(OldNodeUID))
	{
		SetupNode(Node, NodeUID, DisplayLabel, ContainerType, ParentNodeUID);
	}
}
