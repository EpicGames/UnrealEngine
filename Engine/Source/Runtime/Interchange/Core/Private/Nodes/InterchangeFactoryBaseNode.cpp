// Copyright Epic Games, Inc. All Rights Reserved.
#include "Nodes/InterchangeFactoryBaseNode.h"

#include "CoreMinimal.h"
#include "InterchangeLogPrivate.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Package.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeFactoryBaseNode)


namespace UE
{
	namespace Interchange
	{
		const FString& FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()
		{
			static FString BaseNodeFactoryDependencies_BaseKey = "__BaseNodeFactoryDependencies__";
			return BaseNodeFactoryDependencies_BaseKey;
		}

		const FString& FFactoryBaseNodeStaticData::AttributesAppliedThroughDelegatesBaseKey()
		{
			static FString AttributesAppliedThroughDelegates_BaseKey = "__AttributeAppliedThroughDelegate__";
			return AttributesAppliedThroughDelegates_BaseKey;
		}

		const FAttributeKey& FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Reimport_Strategy_Flags_Key__"));
			return AttributeKey;
		}

		const FAttributeKey& FFactoryBaseNodeStaticData::SkipNodeImportKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Internal_Task_Skip_Node__"));
			return AttributeKey;
		}

		const FAttributeKey& FFactoryBaseNodeStaticData::ForceNodeReimportKey()
		{
			static FAttributeKey AttributeKey(TEXT("__Internal_Task_Force_Node_Reimport__"));
			return AttributeKey;
		}

	} //ns Interchange
} //ns UE


UInterchangeFactoryBaseNode::UInterchangeFactoryBaseNode()
{
	FactoryDependencies.Initialize(Attributes, UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey());
	AttributesAppliedThroughDelegates.Initialize(Attributes, UE::Interchange::FFactoryBaseNodeStaticData::AttributesAppliedThroughDelegatesBaseKey());

	RegisterAttribute<uint8>(UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey(), static_cast<uint8>(EReimportStrategyFlags::ApplyNoProperties));
}

#if WITH_EDITOR
FString UInterchangeFactoryBaseNode::GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	FString KeyDisplayName = NodeAttributeKey.ToString();
	const FString OriginalKeyName = KeyDisplayName;
	if (NodeAttributeKey == Macro_CustomSubPathKey)
	{
		KeyDisplayName = TEXT("Asset Sub-Path");
	}
	else if (OriginalKeyName.Equals(UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()))
	{
		KeyDisplayName = TEXT("Factory Dependencies Count");
	}
	else if (OriginalKeyName.StartsWith(UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()))
	{
		KeyDisplayName = UE::Interchange::BuildDisplayNameForArrayKey(TEXT("Factory Dependencies Index "), OriginalKeyName);
	}
	else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey())
	{
		KeyDisplayName = TEXT("Re-Import Strategy");
	}
	else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey())
	{
		KeyDisplayName = TEXT("Skip Node Import");
	}
	else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey())
	{
		KeyDisplayName = TEXT("Force Node Reimport");
	}
	else
	{
		KeyDisplayName = Super::GetKeyDisplayName(NodeAttributeKey);
	}

	return KeyDisplayName;
}


FString UInterchangeFactoryBaseNode::GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (NodeAttributeKey.ToString().StartsWith(UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()))
	{
		return TEXT("FactoryDependencies");
	}
	else
	{
		return Super::GetAttributeCategory(NodeAttributeKey);
	}
}

bool UInterchangeFactoryBaseNode::ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
{
	if (UserInterfaceContext == EInterchangeNodeUserInterfaceContext::Preview)
	{
		const FString KeyDisplayName = NodeAttributeKey.ToString();
		if (KeyDisplayName.Equals(UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()))
		{
			return true;
		}
		else if (KeyDisplayName.StartsWith(UE::Interchange::FFactoryBaseNodeStaticData::FactoryDependenciesBaseKey()))
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey())
		{
			return true;
		}
		else if (NodeAttributeKey == UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey())
		{
			return true;
		}
		else if (NodeAttributeKey == Macro_CustomReferenceObjectKey)
		{
			return true;
		}
	}

	return Super::ShouldHideAttribute(NodeAttributeKey);
}
#endif //WITH_EDITOR

UClass* UInterchangeFactoryBaseNode::GetObjectClass() const
{
	return nullptr;
}

EReimportStrategyFlags UInterchangeFactoryBaseNode::GetReimportStrategyFlags() const
{
	checkSlow(Attributes->ContainAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey()));
	uint8 ReimportStrategyFlags = static_cast<uint8>(EReimportStrategyFlags::ApplyNoProperties);
	Attributes->GetAttributeHandle<uint8>(UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey()).Get(ReimportStrategyFlags);
	return static_cast<EReimportStrategyFlags>(ReimportStrategyFlags);
}

bool UInterchangeFactoryBaseNode::SetReimportStrategyFlags(const EReimportStrategyFlags& ReimportStrategyFlags)
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey(), static_cast<uint8>(ReimportStrategyFlags));
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<uint8> Handle = Attributes->GetAttributeHandle<uint8>(UE::Interchange::FFactoryBaseNodeStaticData::ReimportStrategyFlagsKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeFactoryBaseNode::ShouldSkipNodeImport() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey()))
	{
		return false;
	}
	bool bShouldSkipNodeImport = false;
	Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey()).Get(bShouldSkipNodeImport);
	return bShouldSkipNodeImport;
}

bool UInterchangeFactoryBaseNode::SetSkipNodeImport()
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey(), true);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeFactoryBaseNode::UnsetSkipNodeImport()
{
	if (!Attributes->ContainAttribute(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey()))
	{
		return true;
	}
	UE::Interchange::EAttributeStorageResult Result = Attributes->UnregisterAttribute(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey());
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::SkipNodeImportKey());
		return !Handle.IsValid();
	}
	return false;
}

bool UInterchangeFactoryBaseNode::GetCustomSubPath(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SubPath, FString)
}

bool UInterchangeFactoryBaseNode::SetCustomSubPath(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SubPath, FString)
}

bool UInterchangeFactoryBaseNode::GetCustomLevelUid(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(LevelUid, FString);
}

bool UInterchangeFactoryBaseNode::SetCustomLevelUid(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(LevelUid, FString);
}


int32 UInterchangeFactoryBaseNode::GetFactoryDependenciesCount() const
{
	return FactoryDependencies.GetCount();
}

//Deprecated.
void UInterchangeFactoryBaseNode::GetFactoryDependency(const int32 Index, FString& OutDependency) const
{
	UE_LOGF(LogInterchangeCore, Error, "GetFactoryDependency is deprecated and will not return correct value. Please use GetFactoryDependencies/HasFactoryDependency instead.");
}

TSet<FString> UInterchangeFactoryBaseNode::GetFactoryDependencies() const
{
	return FactoryDependencies.ToSet();
}

bool UInterchangeFactoryBaseNode::HasFactoryDependency(const FString& DependencyUid) const
{
	return FactoryDependencies.HasItem(DependencyUid);
}

bool UInterchangeFactoryBaseNode::AddFactoryDependencyUid(const FString& DependencyUid)
{
	return FactoryDependencies.AddItem(DependencyUid);
}

bool UInterchangeFactoryBaseNode::RemoveFactoryDependencyUid(const FString& DependencyUid)
{
	return FactoryDependencies.RemoveItem(DependencyUid);
}

bool UInterchangeFactoryBaseNode::GetCustomReferenceObject(FSoftObjectPath& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ReferenceObject, FSoftObjectPath)
}

bool UInterchangeFactoryBaseNode::SetCustomReferenceObject(const FSoftObjectPath& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ReferenceObject, FSoftObjectPath)
}

FString UInterchangeFactoryBaseNode::BuildFactoryNodeUid(const FString& TranslatedNodeUid)
{
	const static FString FactoryPrefix = "Factory_";

	if (TranslatedNodeUid.StartsWith(FactoryPrefix))
	{
		return TranslatedNodeUid;
	}

	return FactoryPrefix + TranslatedNodeUid;
}

FString UInterchangeFactoryBaseNode::BuildTranslatedNodeUid(const FString& InFactoryNodeUid)
{
	const static FString FactoryPrefix = "Factory_";

	FStringView View{InFactoryNodeUid};
	if (View.StartsWith(FactoryPrefix))
	{
		View.RemovePrefix(FactoryPrefix.Len());
		return FString{View};
	}

	return InFactoryNodeUid;
}

void UInterchangeFactoryBaseNode::ApplyAllCustomAttributeToObject(UObject* Object) const
{
	UClass* ObjectClass = Object->GetClass();
	for (const TPair<UClass*, TArray<UE::Interchange::FApplyAttributeToAsset>>& ClassDelegatePair : ApplyCustomAttributeDelegates)
	{
		if (ObjectClass->IsChildOf(ClassDelegatePair.Key))
		{
			for (const UE::Interchange::FApplyAttributeToAsset& Delegate : ClassDelegatePair.Value)
			{
				if (Delegate.IsBound())
				{
					Delegate.Execute(Object);
				}
			}
		}
	}
}

void UInterchangeFactoryBaseNode::FillAllCustomAttributeFromObject(UObject* Object) const
{
	UClass* ObjectClass = Object->GetClass();
	for (const TPair<UClass*, TArray<UE::Interchange::FFillAttributeToAsset>>& ClassDelegatePair : FillCustomAttributeDelegates)
	{
		if (ObjectClass->IsChildOf(ClassDelegatePair.Key))
		{
			for (const UE::Interchange::FFillAttributeToAsset& Delegate : ClassDelegatePair.Value)
			{
				if (Delegate.IsBound())
				{
					Delegate.Execute(Object);
				}
			}
		}
	}
}

void UInterchangeFactoryBaseNode::RestoreAllCustomAttributeDelegates()
{
	TArray<FString> AttributeKeys;
	AttributesAppliedThroughDelegates.GetItems(AttributeKeys);
	for (const FString& Key : AttributeKeys)
	{
		AttributesAppliedThroughDelegatesKeySet.Add(Key);
	}

	// The same attributes would be set again when the actual reset happens.
	AttributesAppliedThroughDelegates.RemoveAllItems();
	OnRestoreAllCustomAttributeDelegates();
}

void UInterchangeFactoryBaseNode::RemoveCustomAttributesForClass(UClass* Class)
{
	ApplyCustomAttributeDelegates.Remove(Class);
	FillCustomAttributeDelegates.Remove(Class);
}

void UInterchangeFactoryBaseNode::RemoveAllCustomAttributeDelegates()
{
	ApplyCustomAttributeDelegates.Empty();
	FillCustomAttributeDelegates.Empty();
}

UInterchangeFactoryBaseNode* UInterchangeFactoryBaseNode::DuplicateWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
{
	UInterchangeFactoryBaseNode* CurrentNode = NewObject<UInterchangeFactoryBaseNode>(GetTransientPackage(), SourceNode->GetClass());

	CurrentNode->CopyWithObject(SourceNode, Object);

	CurrentNode->FillAllCustomAttributeFromObject(Object);

	return CurrentNode;
}

bool UInterchangeFactoryBaseNode::ShouldForceNodeReimport() const
{
	if (!Attributes->ContainAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey()))
	{
		return false;
	}
	bool bShouldForceNodeReimport = false;
	Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey()).Get(bShouldForceNodeReimport);
	return bShouldForceNodeReimport;
}

bool UInterchangeFactoryBaseNode::SetForceNodeReimport()
{
	UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey(), true);
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey());
		return Handle.IsValid();
	}
	return false;
}

bool UInterchangeFactoryBaseNode::UnsetForceNodeReimport()
{
	if (!Attributes->ContainAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey()))
	{
		return true;
	}
	UE::Interchange::EAttributeStorageResult Result = Attributes->UnregisterAttribute(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey());
	if (IsAttributeStorageResultSuccess(Result))
	{
		UE::Interchange::FAttributeStorage::TAttributeHandle<bool> Handle = Attributes->GetAttributeHandle<bool>(UE::Interchange::FFactoryBaseNodeStaticData::ForceNodeReimportKey());
		return !Handle.IsValid();
	}
	return false;
}

void UInterchangeFactoryBaseNode::OnRestoreAllCustomAttributeDelegates()
{
	// Should be filled with appropriate attributes in the child classes.
}
