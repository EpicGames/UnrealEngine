// Copyright Epic Games, Inc. All Rights Reserved.
#include "Nodes/InterchangeSourceNode.h"

#include "Nodes/InterchangeBaseNodeContainer.h"

#include "CoreMinimal.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSourceNode)


namespace UE::Interchange
{
	const FString& FSourceNodeExtraInfoStaticData::GetApplicationVendorExtraInfoKey()
	{
		static FString ApplicationVendorExtraInfoKey = "Application Vendor";
		return ApplicationVendorExtraInfoKey;
	}

	const FString& FSourceNodeExtraInfoStaticData::GetApplicationNameExtraInfoKey()
	{
		static FString ApplicationNameExtraInfoKey = "Application Name";
		return ApplicationNameExtraInfoKey;
	}

	const FString& FSourceNodeExtraInfoStaticData::GetApplicationVersionExtraInfoKey()
	{
		static FString ApplicationVersionExtraInfoKey = "Application Version";
		return ApplicationVersionExtraInfoKey;
	}

	namespace SourceNode
	{
		FString GetSourceNodeUniqueID()
		{
			static FString StaticUid = "__SourceNode__";
			return StaticUid;
		}

		const FString& GetExtraInformationKey()
		{
			static FString ExtraInformationKey("__ExtraInformation__Key");
			return ExtraInformationKey;
		}
	}
}

UInterchangeSourceNode::UInterchangeSourceNode()
{
	ExtraInformation.Initialize(Attributes.ToSharedRef(), UE::Interchange::SourceNode::GetExtraInformationKey());
}

void UInterchangeSourceNode::InitializeSourceNode(const FString& UniqueID, const FString& DisplayLabel, UInterchangeBaseNodeContainer* NodeContainer)
{
	NodeContainer->SetupNode(this, UniqueID, DisplayLabel, EInterchangeNodeContainerType::TranslatedAsset);
}

FString UInterchangeSourceNode::GetTypeName() const
{
	const FString TypeName = "SourceNode";
	return TypeName;
}


UInterchangeSourceNode* UInterchangeSourceNode::FindOrCreateUniqueInstance(UInterchangeBaseNodeContainer* NodeContainer)
{
	const FString StaticUid = UE::Interchange::SourceNode::GetSourceNodeUniqueID();
	UInterchangeSourceNode* SourceNode = Cast<UInterchangeSourceNode>(const_cast<UInterchangeBaseNode*>(NodeContainer->GetNode(StaticUid)));
	if (!SourceNode)
	{
		SourceNode = NewObject<UInterchangeSourceNode>(NodeContainer, NAME_None);
		NodeContainer->SetupNode(SourceNode, StaticUid, StaticUid, EInterchangeNodeContainerType::FactoryData);
	}

	return SourceNode;
}

const UInterchangeSourceNode* UInterchangeSourceNode::GetUniqueInstance(const UInterchangeBaseNodeContainer* NodeContainer)
{
	static FString StaticUid = UE::Interchange::SourceNode::GetSourceNodeUniqueID();
	return Cast<const UInterchangeSourceNode>(NodeContainer->GetNode(StaticUid));
}

bool UInterchangeSourceNode::GetCustomSourceFrameRateNumerator(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceFrameRateNumerator, int32);
}

bool UInterchangeSourceNode::SetCustomSourceFrameRateNumerator(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceFrameRateNumerator, int32);
}

bool UInterchangeSourceNode::GetCustomSourceFrameRateDenominator(int32& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceFrameRateDenominator, int32);
}

bool UInterchangeSourceNode::SetCustomSourceFrameRateDenominator(const int32& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceFrameRateDenominator, int32);
}

bool UInterchangeSourceNode::GetCustomSourceTimelineStart(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceTimelineStart, double);
}

bool UInterchangeSourceNode::SetCustomSourceTimelineStart(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceTimelineStart, double);
}

bool UInterchangeSourceNode::GetCustomSourceTimelineEnd(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SourceTimelineEnd, double);
}

bool UInterchangeSourceNode::SetCustomSourceTimelineEnd(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SourceTimelineEnd, double);
}

bool UInterchangeSourceNode::GetCustomAnimatedTimeStart(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimatedTimeStart, double);
}

bool UInterchangeSourceNode::SetCustomAnimatedTimeStart(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimatedTimeStart, double);
}

bool UInterchangeSourceNode::GetCustomAnimatedTimeEnd(double& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AnimatedTimeEnd, double);
}

bool UInterchangeSourceNode::SetCustomAnimatedTimeEnd(const double& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AnimatedTimeEnd, double);
}

bool UInterchangeSourceNode::GetCustomImportUnusedMaterial(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ImportUnusedMaterial, bool);
}

bool UInterchangeSourceNode::SetCustomImportUnusedMaterial(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ImportUnusedMaterial, bool);
}

bool UInterchangeSourceNode::SetExtraInformation(const FString& Name, const FString& Value)
{
	return ExtraInformation.SetKeyValue(Name, Value);
}

bool UInterchangeSourceNode::RemoveExtraInformation(const FString& Name)
{
	return ExtraInformation.RemoveKey(Name);
}

void UInterchangeSourceNode::GetExtraInformation(TMap<FString, FString>& OutExtraInformation) const
{
	OutExtraInformation = ExtraInformation.ToMap();
}

bool UInterchangeSourceNode::GetCustomAxisConversionInverseTransform(FTransform& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AxisConversionInverseTransform, FTransform);
}
bool UInterchangeSourceNode::SetCustomAxisConversionInverseTransform(const FTransform& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AxisConversionInverseTransform, FTransform);
}

bool UInterchangeSourceNode::GetCustomSubPathPrefix(FString& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SubPathPrefix, FString);
}
bool UInterchangeSourceNode::SetCustomSubPathPrefix(const FString& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SubPathPrefix, FString);
}

bool UInterchangeSourceNode::GetCustomUseAssetTypeSubPathSuffix(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(UseAssetTypeSubPathSuffix, bool);
}
bool UInterchangeSourceNode::SetCustomUseAssetTypeSubPathSuffix(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(UseAssetTypeSubPathSuffix, bool);
}

bool UInterchangeSourceNode::GetCustomReimportStrategyFlags(uint8& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(ReimportStrategyFlags, uint8);
}

bool UInterchangeSourceNode::SetCustomReimportStrategyFlags(uint8 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(ReimportStrategyFlags, uint8)
}

bool UInterchangeSourceNode::GetCustomSkeletalMeshFrontAxis(uint8& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(SkeletalMeshFrontAxis, uint8);
}

bool UInterchangeSourceNode::SetCustomSkeletalMeshFrontAxis(uint8 AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(SkeletalMeshFrontAxis, uint8);
}

bool UInterchangeSourceNode::GetCustomNaniteTriangleThreshold(int64& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(NaniteTriangleThreshold, int64);
}

bool UInterchangeSourceNode::SetCustomNaniteTriangleThreshold(const int64& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(NaniteTriangleThreshold, int64)
}

bool UInterchangeSourceNode::GetCustomAllowSceneRootAsJoint(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AllowSceneRootAsJoint, bool);
}

bool UInterchangeSourceNode::SetCustomAllowSceneRootAsJoint(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AllowSceneRootAsJoint, bool)
}

bool UInterchangeSourceNode::GetCustomAlignSkeletalMeshInScene(bool& AttributeValue) const
{
	IMPLEMENT_NODE_ATTRIBUTE_GETTER(AlignSkeletalMeshInScene, bool);
}

bool UInterchangeSourceNode::SetCustomAlignSkeletalMeshInScene(const bool& AttributeValue)
{
	IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AlignSkeletalMeshInScene, bool)
}
