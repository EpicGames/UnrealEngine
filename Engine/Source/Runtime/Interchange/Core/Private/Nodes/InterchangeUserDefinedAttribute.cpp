// Copyright Epic Games, Inc. All Rights Reserved.

#include "Nodes/InterchangeUserDefinedAttribute.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Nodes/InterchangeBaseNode.h"

#include "Misc/FrameRate.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeUserDefinedAttribute)

const FString UInterchangeUserDefinedAttributesAPI::UserDefinedAttributeBaseKey = TEXT("UserDefined_");
const FString UInterchangeUserDefinedAttributesAPI::UserDefinedAttributeDelegateKey = TEXT("AddDelegate_");
const FString UInterchangeUserDefinedAttributesAPI::UserDefinedAttributeValuePostKey = TEXT("_Value");
const FString UInterchangeUserDefinedAttributesAPI::UserDefinedAttributePayLoadPostKey = TEXT("_Payload");

bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_Boolean(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const bool& Value, const FString& PayloadKey, bool RequiresDelegate /*= false*/)
{
	TOptional<FString> OptionalPayload;
	if (!PayloadKey.IsEmpty())
	{
		OptionalPayload = PayloadKey;
	}
	return UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, Value, OptionalPayload, RequiresDelegate);
}

bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_Float(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const float& Value, const FString& PayloadKey, bool RequiresDelegate /*= false*/)
{
	TOptional<FString> OptionalPayload;
	if (!PayloadKey.IsEmpty())
	{
		OptionalPayload = PayloadKey;
	}
	return UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, Value, OptionalPayload, RequiresDelegate);
}

bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_Double(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const double& Value, const FString& PayloadKey, bool RequiresDelegate /*= false*/)
{
	TOptional<FString> OptionalPayload;
	if (!PayloadKey.IsEmpty())
	{
		OptionalPayload = PayloadKey;
	}
	return UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, Value, OptionalPayload, RequiresDelegate);
}

bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_Int32(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const int32& Value, const FString& PayloadKey, bool RequiresDelegate /*= false*/)
{
	TOptional<FString> OptionalPayload;
	if (!PayloadKey.IsEmpty())
	{
		OptionalPayload = PayloadKey;
	}
	return UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, Value, OptionalPayload, RequiresDelegate);
}

bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute_FString(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const FString& Value, const FString& PayloadKey, bool RequiresDelegate /*= false*/)
{
	TOptional<FString> OptionalPayload;
	if (!PayloadKey.IsEmpty())
	{
		OptionalPayload = PayloadKey;
	}
	return UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, Value, OptionalPayload, RequiresDelegate);
}

bool UInterchangeUserDefinedAttributesAPI::RemoveUserDefinedAttribute(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName)
{
	bool RequiresDelegates;
	const bool bGeneratePayloadKey = false;
	if (HasAttribute(InterchangeNode, UserDefinedAttributeName, bGeneratePayloadKey, RequiresDelegates))
	{
		UE::Interchange::FAttributeKey UserDefinedValueKey = MakeUserDefinedPropertyValueKey(UserDefinedAttributeName, RequiresDelegates);
		if (!InterchangeNode->RemoveAttribute(UserDefinedValueKey.Key))
		{
			return false;
		}

		UE::Interchange::FAttributeKey UserDefinedPayloadKey = MakeUserDefinedPropertyPayloadKey(UserDefinedAttributeName, RequiresDelegates);
		if (InterchangeNode->HasAttribute(UserDefinedPayloadKey))
		{
			if (!InterchangeNode->RemoveAttribute(UserDefinedPayloadKey.Key))
			{
				return false;
			}
		}
	}

	//Attribute was successfully removed/ never existed
	return true;
}

bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_Boolean(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, bool& OutValue, FString& OutPayloadKey)
{
	TOptional<FString> OptionalPayload;
	bool bResult = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, OutValue, OptionalPayload);
	if (OptionalPayload.IsSet())
	{
		OutPayloadKey = OptionalPayload.GetValue();
	}
	return bResult;
}

bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_Float(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, float& OutValue, FString& OutPayloadKey)
{
	TOptional<FString> OptionalPayload;
	bool bResult = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, OutValue, OptionalPayload);
	if (OptionalPayload.IsSet())
	{
		OutPayloadKey = OptionalPayload.GetValue();
	}
	return bResult;
}

bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_Double(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, double& OutValue, FString& OutPayloadKey)
{
	TOptional<FString> OptionalPayload;
	bool bResult = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, OutValue, OptionalPayload);
	if (OptionalPayload.IsSet())
	{
		OutPayloadKey = OptionalPayload.GetValue();
	}
	return bResult;
}

bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_Int32(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, int32& OutValue, FString& OutPayloadKey)
{
	TOptional<FString> OptionalPayload;
	bool bResult = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, OutValue, OptionalPayload);
	if (OptionalPayload.IsSet())
	{
		OutPayloadKey = OptionalPayload.GetValue();
	}
	return bResult;
}

bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute_FString(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, FString& OutValue, FString& OutPayloadKey)
{
	TOptional<FString> OptionalPayload;
	bool bResult = UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(InterchangeNode, UserDefinedAttributeName, OutValue, OptionalPayload);
	if (OptionalPayload.IsSet())
	{
		OutPayloadKey = OptionalPayload.GetValue();
	}
	return bResult;
}

TArray<FInterchangeUserDefinedAttributeInfo> UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(const UInterchangeBaseNode* InterchangeNode)
{
	check(InterchangeNode);
	TArray<UE::Interchange::FAttributeKey> AttributeKeys;
	InterchangeNode->GetAttributeKeys(AttributeKeys);
	TArray<FInterchangeUserDefinedAttributeInfo> UserDefinedAttributeInfos;
	int32 RightChopIndex = UserDefinedAttributeBaseKey.Len();
	int32 LeftChopIndex = UserDefinedAttributeValuePostKey.Len();
	int32 AddDelegateRightChopIndex = UserDefinedAttributeDelegateKey.Len();

	for (const UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		if (AttributeKey.Key.StartsWith(UserDefinedAttributeBaseKey) && AttributeKey.Key.EndsWith(UserDefinedAttributeValuePostKey))
		{
			bool RequiresDelegate = false;

			FStringView UserDefinedAttributeName = FStringView(AttributeKey.Key);
			UserDefinedAttributeName.RemovePrefix(RightChopIndex);
			UserDefinedAttributeName.RemoveSuffix(LeftChopIndex);

			if (UserDefinedAttributeName.StartsWith(UserDefinedAttributeDelegateKey))
			{
				UserDefinedAttributeName.RemovePrefix(AddDelegateRightChopIndex);
				RequiresDelegate = true;
			}

			FInterchangeUserDefinedAttributeInfo& UserDefinedAttributeInfo = UserDefinedAttributeInfos.AddDefaulted_GetRef();
			UserDefinedAttributeInfo.Type = InterchangeNode->GetAttributeType(AttributeKey);
			UserDefinedAttributeInfo.Name = UserDefinedAttributeName;
			UserDefinedAttributeInfo.RequiresDelegate = RequiresDelegate;

			// Get the optional payload key
			const UE::Interchange::FAttributeKey UserDefinedPayloadKey = MakeUserDefinedPropertyPayloadKey(UserDefinedAttributeName, RequiresDelegate);
			if (InterchangeNode->HasAttribute(UserDefinedPayloadKey))
			{
				FString PayloadKey;
				InterchangeNode->GetStringAttribute(UserDefinedPayloadKey.Key, PayloadKey);
				UserDefinedAttributeInfo.PayloadKey = PayloadKey;
			}
		}
	}

	return UserDefinedAttributeInfos;
}

void UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttributeInfos(const UInterchangeBaseNode* InterchangeNode, TArray<FInterchangeUserDefinedAttributeInfo>& UserDefinedAttributeInfos)
{
	check(InterchangeNode);
	UserDefinedAttributeInfos = GetUserDefinedAttributeInfos(InterchangeNode);
}

void UInterchangeUserDefinedAttributesAPI::DuplicateAllUserDefinedAttribute(const UInterchangeBaseNode* InterchangeSourceNode, UInterchangeBaseNode* InterchangeDestinationNode, bool bAddSourceNodeName)
{
	check(InterchangeSourceNode);
	check(InterchangeDestinationNode);
	TArray<UE::Interchange::FAttributeKey> AttributeKeys;
	InterchangeSourceNode->GetAttributeKeys(AttributeKeys);
	int32 RightChopIndex = UserDefinedAttributeBaseKey.Len();
	int32 LeftChopIndex = UserDefinedAttributeValuePostKey.Len();
	int32 AddDelegateRightChopIndex = UserDefinedAttributeDelegateKey.Len();

	for (UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		const FString AttributeKeyString = AttributeKey.ToString();
		if (AttributeKeyString.StartsWith(UserDefinedAttributeBaseKey) && AttributeKeyString.EndsWith(UserDefinedAttributeValuePostKey))
		{
			bool RequiresDelegate = false;
			FString UserDefinedAttributeName = AttributeKeyString.RightChop(RightChopIndex).LeftChop(LeftChopIndex);
			if (UserDefinedAttributeName.StartsWith(UserDefinedAttributeDelegateKey))
			{
				UserDefinedAttributeName = UserDefinedAttributeName.RightChop(AddDelegateRightChopIndex);
				RequiresDelegate = true;
			}
			
			UE::Interchange::EAttributeTypes Type = InterchangeSourceNode->GetAttributeType(AttributeKey);
			FString DuplicateName = UserDefinedAttributeName;
			if (bAddSourceNodeName)
			{
				DuplicateName = InterchangeSourceNode->GetDisplayLabel() + TEXT(".") + UserDefinedAttributeName;
			}

			//Get the optional payload key
			const UE::Interchange::FAttributeKey UserDefinedPayloadKey = MakeUserDefinedPropertyPayloadKey(UserDefinedAttributeName, RequiresDelegate);
			TOptional<FString> PayloadKey;
			if (InterchangeSourceNode->HasAttribute(UserDefinedPayloadKey))
			{
				FString PayloadKeyValue;
				InterchangeSourceNode->GetStringAttribute(UserDefinedPayloadKey.Key, PayloadKeyValue);
				PayloadKey = PayloadKeyValue;
			}
			switch (Type)
			{
				case UE::Interchange::EAttributeTypes::Bool:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<bool>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Color:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FColor>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::DateTime:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FDateTime>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Double:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<double>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Enum:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<uint8>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Float:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<float>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Guid:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FGuid>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int8:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<int8>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int16:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<int16>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int32:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<int32>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int64:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<int64>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntRect:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FIntRect>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::LinearColor:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FLinearColor>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Name:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FName>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::RandomStream:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FRandomStream>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::String:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FString>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Timespan:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FTimespan>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectors:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FTwoVectors>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt8:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<uint8>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt16:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<uint16>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt32:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<uint32>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt64:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<uint64>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector2d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntPoint:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FIntPoint>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntVector:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FIntVector>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalf:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector2DHalf>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Float16:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FFloat16>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBox:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FOrientedBox>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumber:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FFrameNumber>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameRate:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FFrameRate>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameTime:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FFrameTime>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPath:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FSoftObjectPath>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FMatrix44f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FMatrix44d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Plane4f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FPlane4f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Plane4d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FPlane4d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Quat4f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FQuat4f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Quat4d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FQuat4d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FRotator3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FRotator3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Transform3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FTransform3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Transform3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FTransform3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector2f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector4f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector4f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector4d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FVector4d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box2f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBox2f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box2D:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBox2D>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBox3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBox3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBoxSphereBounds3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FBoxSphereBounds3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3f:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FSphere3f>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3d:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<FSphere3d>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::BoolArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<bool>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::ColorArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FColor>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::DateTimeArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FDateTime>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::DoubleArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<double>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::EnumArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint8>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FloatArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<float>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::GuidArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FGuid>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int8Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint8>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int16Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<int16>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int32Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<int32>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Int64Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<int64>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntRectArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FIntRect>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::LinearColorArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FLinearColor>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::NameArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FName>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::RandomStreamArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FRandomStream>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::StringArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FString>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::TimespanArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FTimespan>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectorsArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FTwoVectors>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint8>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray64:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray64<uint8>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt16Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint16>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt32Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint32>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::UInt64Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<uint64>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector2d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntPointArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FIntPoint>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::IntVectorArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FIntVector>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalfArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector2DHalf>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Float16Array:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FFloat16>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBoxArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FOrientedBox>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumberArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FFrameNumber>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameRateArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FFrameRate>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::FrameTimeArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FFrameTime>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPathArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FSoftObjectPath>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FMatrix44f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FMatrix44d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Plane4fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FPlane4f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Plane4dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FPlane4d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Quat4fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FQuat4f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Quat4dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FQuat4d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FRotator3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FRotator3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Transform3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FTransform3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Transform3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FTransform3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector2fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector2f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector4fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector4f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Vector4dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FVector4d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box2fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBox2f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box2DArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBox2D>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBox3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Box3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBox3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBoxSphereBounds3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FBoxSphereBounds3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3fArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FSphere3f>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3dArray:
					CreateUserDefinedAttribute(InterchangeDestinationNode, DuplicateName, InterchangeSourceNode->GetAttributeChecked<TArray<FSphere3d>>(AttributeKey.Key), PayloadKey, RequiresDelegate);
					break;
				default:
					ensureMsgf(false, TEXT("Unsupported EAttributeTypes value!"));
					break;
			}
		}
	}
}

void UInterchangeUserDefinedAttributesAPI::AddApplyAndFillDelegatesToFactory(UInterchangeFactoryBaseNode* InterchangeFactoryNode, UClass* ParentClass)
{
	TArray<UE::Interchange::FAttributeKey> AttributeKeys;
	InterchangeFactoryNode->GetAttributeKeys(AttributeKeys);
	int32 RightChopIndex = UserDefinedAttributeBaseKey.Len();
	int32 LeftChopIndex = UserDefinedAttributeValuePostKey.Len();
	int32 AddDelegateRightChopIndex = UserDefinedAttributeDelegateKey.Len();

	for (UE::Interchange::FAttributeKey& AttributeKey : AttributeKeys)
	{
		const FString AttributeKeyString = AttributeKey.ToString();
		if (AttributeKeyString.StartsWith(UserDefinedAttributeBaseKey) && AttributeKeyString.EndsWith(UserDefinedAttributeValuePostKey))
		{
			bool RequiresDelegate = false;
			FString UserDefinedAttributeName = AttributeKeyString.RightChop(RightChopIndex).LeftChop(LeftChopIndex);
			if (!UserDefinedAttributeName.StartsWith(UserDefinedAttributeDelegateKey))
			{
				continue;				
			}

			UserDefinedAttributeName = UserDefinedAttributeName.RightChop(AddDelegateRightChopIndex);
			RequiresDelegate = true;

			UE::Interchange::EAttributeTypes Type = InterchangeFactoryNode->GetAttributeType(AttributeKey);
			switch (Type)
			{
				case UE::Interchange::EAttributeTypes::Bool:
					InterchangeFactoryNode->AddApplyAndFillDelegates<bool>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Color:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FColor>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::DateTime:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FDateTime>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Double:
					InterchangeFactoryNode->AddApplyAndFillDelegates<double>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Enum:
					InterchangeFactoryNode->AddApplyAndFillDelegates<bool>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Float:
					InterchangeFactoryNode->AddApplyAndFillDelegates<float>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Guid:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FGuid>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int8:
					InterchangeFactoryNode->AddApplyAndFillDelegates<int8>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int16:
					InterchangeFactoryNode->AddApplyAndFillDelegates<int16>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int32:
					InterchangeFactoryNode->AddApplyAndFillDelegates<int32>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int64:
					InterchangeFactoryNode->AddApplyAndFillDelegates<int64>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntRect:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FIntRect>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::LinearColor:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FLinearColor>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Name:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FName>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::RandomStream:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FRandomStream>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::String:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FString>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Timespan:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FTimespan>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectors:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FTwoVectors>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt8:
					InterchangeFactoryNode->AddApplyAndFillDelegates<uint8>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt16:
					InterchangeFactoryNode->AddApplyAndFillDelegates<uint16>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt32:
					InterchangeFactoryNode->AddApplyAndFillDelegates<uint32>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt64:
					InterchangeFactoryNode->AddApplyAndFillDelegates<uint64>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector2d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntPoint:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FIntPoint>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntVector:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FIntVector>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalf:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector2DHalf>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Float16:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FFloat16>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBox:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FOrientedBox>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumber:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FFrameNumber>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameRate:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FFrameRate>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameTime:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FFrameTime>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPath:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FSoftObjectPath>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FMatrix44f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FMatrix44d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Plane4f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FPlane4f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Plane4d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FPlane4d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Quat4f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FQuat4f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Quat4d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FQuat4d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FRotator3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FRotator3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Transform3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FTransform3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Transform3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FTransform3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector2f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector4f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector4f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector4d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FVector4d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box2f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBox2f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box2D:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBox2D>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBox3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBox3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBoxSphereBounds3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FBoxSphereBounds3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3f:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FSphere3f>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3d:
					InterchangeFactoryNode->AddApplyAndFillDelegates<FSphere3d>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::BoolArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<bool>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::ColorArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FColor>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::DateTimeArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FDateTime>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::DoubleArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<double>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::EnumArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<bool>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FloatArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<float>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::GuidArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FGuid>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int8Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<uint8>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int16Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<int16>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int32Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<int32>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Int64Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<int64>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntRectArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FIntRect>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::LinearColorArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FLinearColor>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::NameArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FName>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::RandomStreamArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FRandomStream>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::StringArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FString>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::TimespanArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FTimespan>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::TwoVectorsArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FTwoVectors>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<uint8>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::ByteArray64:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray64<uint8>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt16Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<uint16>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt32Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<uint32>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::UInt64Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<uint64>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector2d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntPointArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FIntPoint>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::IntVectorArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FIntVector>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2DHalfArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector2DHalf>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Float16Array:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FFloat16>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::OrientedBoxArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FOrientedBox>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameNumberArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FFrameNumber>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameRateArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FFrameRate>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::FrameTimeArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FFrameTime>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::SoftObjectPathArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FSoftObjectPath>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FMatrix44f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Matrix44dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FMatrix44d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Plane4fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FPlane4f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Plane4dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FPlane4d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Quat4fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FQuat4f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Quat4dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FQuat4d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FRotator3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Rotator3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FRotator3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Transform3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FTransform3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Transform3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FTransform3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector2fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector2f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector4fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector4f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Vector4dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FVector4d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box2fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBox2f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box2DArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBox2D>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBox3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Box3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBox3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBoxSphereBounds3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::BoxSphereBounds3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FBoxSphereBounds3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3fArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FSphere3f>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				case UE::Interchange::EAttributeTypes::Sphere3dArray:
					InterchangeFactoryNode->AddApplyAndFillDelegates<TArray<FSphere3d>>(AttributeKey.Key, ParentClass, *UserDefinedAttributeName);
					break;
				default:
					ensureMsgf(false, TEXT("Unsupported EAttributeTypes value!"));
					break;
			}
		}
	}
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyValueKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate)
{
	return MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate);
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyValueKey(const FString& UserDefinedAttributeName, bool RequiresDelegate)
{
	return MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate);
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyPayloadKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate)
{
	const bool bGeneratePayloadKey = true;
	return MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate, bGeneratePayloadKey);
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyPayloadKey(const FString& UserDefinedAttributeName, bool RequiresDelegate)
{
	const bool bGeneratePayloadKey = true;
	return MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate, bGeneratePayloadKey);
}

bool UInterchangeUserDefinedAttributesAPI::HasAttribute(const UInterchangeBaseNode* InterchangeSourceNode, const FString& InUserDefinedAttributeName, bool GeneratePayloadKey, bool& OutRequiresDelegate)
{
	UE::Interchange::FAttributeKey KeyWithDelegate = MakeUserDefinedPropertyKey(InUserDefinedAttributeName,/*RequiresDelegate =*/ true, GeneratePayloadKey);
	UE::Interchange::FAttributeKey KeyWithoutDelegate = MakeUserDefinedPropertyKey(InUserDefinedAttributeName,/*RequiresDelegate =*/ false, GeneratePayloadKey);
	OutRequiresDelegate = true;
	if (InterchangeSourceNode->HasAttribute(KeyWithDelegate))
	{
		return true;
	}
	else if (InterchangeSourceNode->HasAttribute(KeyWithoutDelegate))
	{
		OutRequiresDelegate = false;
		return true;
	}

	return false;
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate, bool GeneratePayloadKey /*= false*/)
{
	// Create a unique Key for this user defined attribute
	TStringBuilder<256> StringBuilder;
	
	StringBuilder.Append(UserDefinedAttributeBaseKey);

	if (RequiresDelegate)
	{
		StringBuilder.Append(UserDefinedAttributeDelegateKey);
	}

	StringBuilder.Append(UserDefinedAttributeName);

	if (GeneratePayloadKey)
	{
		StringBuilder.Append(UserDefinedAttributePayLoadPostKey);
	}
	else
	{
		StringBuilder.Append(UserDefinedAttributeValuePostKey);
	}

	return UE::Interchange::FAttributeKey(StringBuilder.ToView());
}

UE::Interchange::FAttributeKey UInterchangeUserDefinedAttributesAPI::MakeUserDefinedPropertyKey(const FString& UserDefinedAttributeName, bool RequiresDelegate, bool GeneratePayloadKey /*= false*/)
{
	return MakeUserDefinedPropertyKey(FStringView(UserDefinedAttributeName), RequiresDelegate, GeneratePayloadKey);
}