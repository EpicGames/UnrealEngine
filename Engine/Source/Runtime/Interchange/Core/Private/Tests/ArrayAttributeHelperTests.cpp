// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParallelFor.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/Atomic.h"
#include "Types/AttributeStorage.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FArrayAttributeHelperTests,
	"System.Runtime.Interchange.ArrayAttributeHelperTests",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)

namespace UE::ArrayAttributeHelperTests::Private
{
	template<typename T>
	void RunTestInternal(FArrayAttributeHelperTests& Tester)
	{
		using namespace UE::Interchange;

		const FString BaseKeyName = "TestKey";
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();

		UE::Interchange::TArrayAttributeHelper<T> Helper;
		Helper.Initialize(TestStorage, BaseKeyName);

		const T SomeItem = T{};
		Helper.AddItem(SomeItem);
		Helper.AddItem(SomeItem);
		Helper.AddItem(SomeItem);
		Tester.TestEqual(TEXT("Count after adding 3"), Helper.GetCount(), 3);

		T GotItem;
		Helper.GetItem(2, GotItem);
		Tester.TestEqual(TEXT("Retrieved item after adding 3"), GotItem, SomeItem);

		Helper.RemoveItem(SomeItem);
		Tester.TestEqual(TEXT("Count after removal with duplicates"), Helper.GetCount(), 2);

		TArray<T> Results;
		Helper.GetItems(Results);
		Tester.TestEqual(TEXT("Actual number of items in the result"), Results.Num(), 2);

		for (const T& Result : Results)
		{
			Tester.TestEqual(TEXT("Item equals the expected value"), Result, SomeItem);
		}

		Helper.RemoveAllItems();
		Tester.TestEqual(TEXT("Count after remove all"), Helper.GetCount(), 0);

		Helper.GetItems(Results);
		Tester.TestEqual(TEXT("Actual number of items in the result"), Results.Num(), 0);
	}
}	 // namespace UE::ArrayAttributeHelperTests::Private

bool FArrayAttributeHelperTests::RunTest(const FString& Parameters)
{
	using namespace UE::ArrayAttributeHelperTests::Private;

	RunTestInternal<FString>(*this);
	RunTestInternal<int32>(*this);
	RunTestInternal<double>(*this);
	RunTestInternal<float>(*this);

	return true;
}

#endif	  // WITH_DEV_AUTOMATION_TESTS
