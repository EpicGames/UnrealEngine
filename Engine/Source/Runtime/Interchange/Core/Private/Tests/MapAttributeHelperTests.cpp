// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParallelFor.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/Atomic.h"
#include "Types/AttributeStorage.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FMapAttributeHelperTests,
	"System.Runtime.Interchange.MapAttributeHelperTests",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)

namespace UE::MapAttributeHelperTests::Private
{
	template<typename T, typename U>
	void RunTestInternal(FMapAttributeHelperTests& Tester)
	{
		using namespace UE::Interchange;

		const FString BaseKeyName = "TestKey";
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();

		UE::Interchange::TMapAttributeHelper<T, U> Helper;
		Helper.Initialize(TestStorage.ToSharedRef(), BaseKeyName);

		T SomeKey = T{};
		T AnotherKey = T{};
		if constexpr (std::is_same_v<T, FString>)
		{
			SomeKey = TEXT("SomeKey");
			AnotherKey = TEXT("AnotherKey");
		}
		else if constexpr (std::is_arithmetic_v<T>)
		{
			SomeKey = 1;
			AnotherKey = 20;
		}

		U SomeValue = U{};
		U AnotherValue = U{};
		if constexpr (std::is_same_v<U, FString>)
		{
			SomeValue = TEXT("SomeValue");
			AnotherValue = TEXT("AnotherValue");
		}
		else if constexpr (std::is_arithmetic_v<U>)
		{
			SomeValue = 1;
			AnotherValue = 20;
		}

		Helper.SetKeyValue(SomeKey, AnotherValue);
		Helper.SetKeyValue(SomeKey, SomeValue);
		Helper.SetKeyValue(AnotherKey, SomeValue);
		Helper.SetKeyValue(AnotherKey, AnotherValue);

		TMap<T, U> Map = Helper.ToMap();
		Tester.TestEqual(TEXT("Compare maps after ToMap"), Map.Num(), 2);

		U SomeGotValue = U{};
		U AnotherGotValue = U{};
		bool bSuccess = Helper.GetValue(SomeKey, SomeGotValue);
		bSuccess = bSuccess && Helper.GetValue(AnotherKey, AnotherGotValue);
		Tester.TestTrue(TEXT("GetValue succeeded"), bSuccess);
		Tester.TestEqual(TEXT("Compare result after add 1"), SomeGotValue, SomeValue);
		Tester.TestEqual(TEXT("Compare result after add 2"), AnotherGotValue, AnotherValue);

		Tester.TestTrue(TEXT("Test removing a key"), Helper.RemoveKey(AnotherKey));

		U RemovedValue;
		bSuccess = Helper.RemoveKeyAndGetValue(SomeKey, RemovedValue);
		Tester.TestTrue(TEXT("RemoveKeyAndGetValue succeeded"), bSuccess);
		Tester.TestEqual(TEXT("Compare removed value"), RemovedValue, SomeValue);

		Tester.TestFalse(TEXT("Test removing missing key"), Helper.RemoveKey(AnotherKey));
	}
}	 // namespace UE::MapAttributeHelperTests::Private

bool FMapAttributeHelperTests::RunTest(const FString& Parameters)
{
	using namespace UE::MapAttributeHelperTests::Private;

	RunTestInternal<FString, FString>(*this);
	RunTestInternal<FString, int32>(*this);
	RunTestInternal<FString, float>(*this);
	RunTestInternal<int32, FString>(*this);
	RunTestInternal<int32, int32>(*this);
	RunTestInternal<int32, float>(*this);

	return true;
}

#endif	  // WITH_DEV_AUTOMATION_TESTS
