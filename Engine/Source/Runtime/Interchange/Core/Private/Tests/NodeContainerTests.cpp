// Copyright Epic Games, Inc. All Rights Reserved.

#include "CoreMinimal.h"
#include "Async/ParallelFor.h"
#include "Misc/AutomationTest.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Templates/Atomic.h"
#include "Types/AttributeStorage.h"
#include "UObject/UObjectGlobals.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FNodeContainerTest, "System.Runtime.Interchange.NodeContainer", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)

bool FNodeContainerTest::RunTest(const FString& Parameters)
{
	using namespace UE::Interchange;

	//Create a node container
	UInterchangeBaseNodeContainer* NodeContainer = NewObject<UInterchangeBaseNodeContainer>();
	if (!NodeContainer)
	{
		AddError(TEXT("Cannot create a UInterchangeNodeContainer object."));
		return false;
	}
	//Add a couple of translated asset Nodes
	const FString TranslatedAssetNodePrefix = "TranslatedAssetNode_";
	const int32 TranslatedAssetNodeCount = 10;
	for (int32 NodeIndex = 0; NodeIndex < TranslatedAssetNodeCount; ++NodeIndex)
	{
		UInterchangeBaseNode* Node = NewObject<UInterchangeBaseNode>();
		FString NodeUniqueID = TranslatedAssetNodePrefix + FString::FromInt(NodeIndex);
		NodeContainer->SetupNode(Node, NodeUniqueID, NodeUniqueID, EInterchangeNodeContainerType::TranslatedAsset);
	}

	//Add a couple of translated scene Nodes
	const FString TranslatedSceneNodePrefix = "TranslatedSceneNode_";
	const FString TranslatedRootNodeUid = TranslatedSceneNodePrefix + FString::FromInt(0);
	const int32 TranslatedSceneNodeCount = 100;
	const int32 ChildCount = 9;
	{
		int32 CurrentParentIndex = 0;
		FString CurrentParentUid = TranslatedSceneNodePrefix + FString::FromInt(0);
		for(int32 NodeIndex = 0; NodeIndex < TranslatedSceneNodeCount; ++NodeIndex)
		{
			UInterchangeBaseNode* Node = NewObject<UInterchangeBaseNode>();
			FString NodeUniqueID = TranslatedSceneNodePrefix + FString::FromInt(NodeIndex);
			NodeContainer->SetupNode(Node, NodeUniqueID, NodeUniqueID, EInterchangeNodeContainerType::TranslatedScene, CurrentParentIndex != NodeIndex ? CurrentParentUid : TEXT(""));
			int32 ComputeParentIndex = (NodeIndex % ChildCount == 0) ? NodeIndex : CurrentParentIndex;
			if (ComputeParentIndex != CurrentParentIndex)
			{
				//We have a new parent
				CurrentParentIndex = ComputeParentIndex;
				CurrentParentUid = NodeUniqueID;
			}
		}
	}

	//Add some factory node
	const FString FactoryNodePrefix = "FactoryNode_";
	const int32 FactoryNodeCount = 3;
	{
		for (int32 NodeIndex = 0; NodeIndex < FactoryNodeCount; ++NodeIndex)
		{
			UInterchangeBaseNode* Node = NewObject<UInterchangeBaseNode>();
			FString NodeUniqueID = FactoryNodePrefix + FString::FromInt(NodeIndex);
			NodeContainer->SetupNode(Node, NodeUniqueID, NodeUniqueID, EInterchangeNodeContainerType::FactoryData);
			Node->AddTargetNodeUid(TranslatedRootNodeUid);
		}
	}

	TArray<FString> TranslatedAssetNodes;
	TArray<FString> TranslatedSceneNodes;
	TArray<FString> FactoryNodes;
	TArray<UInterchangeBaseNode*> FactoryNodesPtr;
	{
		bool bUnknownNodeType = false;
		NodeContainer->IterateNodes([&TranslatedAssetNodes, &TranslatedSceneNodes, &FactoryNodes, &FactoryNodesPtr, &bUnknownNodeType](const FString& NodeUid, UInterchangeBaseNode* Node)
		{
			switch(Node->GetNodeContainerType())
			{
				case EInterchangeNodeContainerType::TranslatedAsset:
				{
					TranslatedAssetNodes.Add(NodeUid);
				}
				break;
			
				case EInterchangeNodeContainerType::TranslatedScene:
				{
					TranslatedSceneNodes.Add(NodeUid);
				}
				break;

				case EInterchangeNodeContainerType::FactoryData:
				{
					FactoryNodes.Add(NodeUid);
					FactoryNodesPtr.Add(Node);
				}
				break;

				default:
				{
					bUnknownNodeType = true;
				}
			}
		});

		TestEqual(TEXT("Node container translated asset node count"), TranslatedAssetNodes.Num(), TranslatedAssetNodeCount);
		TestEqual(TEXT("Node container translated scene node count"), TranslatedSceneNodes.Num(), TranslatedSceneNodeCount);
		TestEqual(TEXT("Node container translated asset node count"), FactoryNodes.Num(), FactoryNodeCount);
		TestFalse(TEXT("Node container contains unknown node"), bUnknownNodeType);
	}
	
	//Test root node
	{
		TArray<FString> RootNodes;
		NodeContainer->GetRoots(RootNodes);
		TestEqual(TEXT("Node container root count"), RootNodes.Num(), 14);
	}

	//Children caches tests
	{
		TArray<FString>* RootChildrenUidsPtr = NodeContainer->GetCachedNodeChildrenUids(TranslatedRootNodeUid);
		if(RootChildrenUidsPtr)
		{
			//Copy the array
			TArray<FString> RootChildrenUids = *RootChildrenUidsPtr;
			//Test children cache feature
			{
				TestEqual(TEXT("Node container root node children count"), RootChildrenUids.Num(), ChildCount);
				for (int32 ChildIndex = 0; ChildIndex < RootChildrenUids.Num(); ++ChildIndex)
				{
					FString ExpectedChildName = TranslatedSceneNodePrefix + FString::FromInt(ChildIndex + 1);
					if (!TestEqual(TEXT("Node container child unique id doesn't match"), RootChildrenUids[ChildIndex], ExpectedChildName))
					{
						//Log only one error
						break;
					}
				}
			}

			//Test children index feature
			{
				TArray<int32> RemapChildren;
				const int32 RootChildCount = RootChildrenUids.Num();
				RemapChildren.AddZeroed(RootChildCount);
				for (int32 ChildIndex = 0; ChildIndex < RootChildCount; ++ChildIndex)
				{
					RemapChildren[ChildIndex] = RootChildCount - ChildIndex - 1;
					NodeContainer->SetNodeDesiredChildIndex(RootChildrenUids[ChildIndex], RemapChildren[ChildIndex]);
				}
				auto VerifyCacheChildrenReorder = [this, &NodeContainer, &RootChildrenUids, &RemapChildren, &TranslatedRootNodeUid]()
				{
					TArray<FString>* ReorderedChildrenPtr = NodeContainer->GetCachedNodeChildrenUids(TranslatedRootNodeUid);
					if(ReorderedChildrenPtr)
					{
						TArray<FString> ReorderedChildren = *ReorderedChildrenPtr;
						if(RootChildrenUids.Num() == ReorderedChildren.Num())
						{
							for (int32 ChildIndex = 0; ChildIndex < ReorderedChildren.Num(); ++ChildIndex)
							{
								if (!TestEqual(TEXT("Node container child index reorder fail"), ReorderedChildren[ChildIndex], RootChildrenUids[RemapChildren[ChildIndex]]))
								{
									//Log only one error
									break;
								}
							}
						}
						else
						{
							AddError(TEXT("Node container child index feature fail, the number of child change when indexes are specified."));
						}
					}
					else
					{
						AddError(TEXT("Node container child index feature fail, Cannot get the cache node childrenUids."));
					}
				};

				//Verify before re-computing the cache
				VerifyCacheChildrenReorder();

				NodeContainer->ComputeChildrenCache();

				//Verify after the cache was reset and recompute
				VerifyCacheChildrenReorder();
			}
		}
		else
		{
			AddError(TEXT("Node container child index feature fail, Cannot get the cache node childrenUids."));
		}
	}

	//Test namespace feature
	{
		const FString Namespace = "Foo";
		const FString NamespaceAndUniqueId = Namespace + TranslatedRootNodeUid;
		const UInterchangeBaseNode* TranslatedNode = nullptr;

		auto TestFactoryNodeTargetUid = [this, &FactoryNodesPtr, &NamespaceAndUniqueId, &TranslatedRootNodeUid](bool bWithNamespace)
		{
			for (UInterchangeBaseNode* Node : FactoryNodesPtr)
			{
				if (Node)
				{
					TArray<FString> TargetNodeUids;
					Node->GetTargetNodeUids(TargetNodeUids);
					if (TargetNodeUids.IsValidIndex(0))
					{
						const FString TestUid = bWithNamespace ? NamespaceAndUniqueId : TranslatedRootNodeUid;
						if(!TestEqual(TEXT("Namespace should update node reference into all node attributes"), TestUid, TargetNodeUids[0]))
						{
							break;
						}
					}
				}
			}
		};

		//Test without the namespace
		TranslatedNode = NodeContainer->GetNode(TranslatedRootNodeUid);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return the node if the unique id is pass without the namesapce, if the container dont have any namespace"), TranslatedNode != nullptr);
		TranslatedNode = nullptr;
		TranslatedNode = NodeContainer->GetNode(NamespaceAndUniqueId);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return null node if the namespace and the unique id are combine, if the container dont have any namespace"), TranslatedNode == nullptr);
		TestFactoryNodeTargetUid(false);

		//Test with the namespace
		NodeContainer->SetNamespace(Namespace, nullptr);
		TranslatedNode = nullptr;
		TranslatedNode = NodeContainer->GetNode(TranslatedRootNodeUid);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return null node if the unique id is pass without the namesapce, if the container have a namespace"), TranslatedNode == nullptr);
		TranslatedNode = nullptr;
		TranslatedNode = NodeContainer->GetNode(NamespaceAndUniqueId);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return the node if the namespace and the unique id are combine, if the container have a namespace"), TranslatedNode != nullptr);
		TestFactoryNodeTargetUid(true);
		
		//Test with the namespace removed
		NodeContainer->SetNamespace(FString(), nullptr);
		TranslatedNode = nullptr;
		TranslatedNode = NodeContainer->GetNode(TranslatedRootNodeUid);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return the node if the unique id is pass without the namesapce, if the container dont have any namespace"), TranslatedNode != nullptr);
		TranslatedNode = nullptr;
		TranslatedNode = NodeContainer->GetNode(NamespaceAndUniqueId);
		TestTrue(TEXT("UInterchangeNodeContainer::GetNode() should return null node if the namespace and the unique id are combine, if the container dont have any namespace"), TranslatedNode == nullptr);
		TestFactoryNodeTargetUid(false);
	}

	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
