// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParallelFor.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/Atomic.h"
#include "Types/AttributeStorage.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(
	FSetAttributeHelperTests,
	"System.Runtime.Interchange.SetAttributeHelperTests",
	EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter
)

namespace UE::SetAttributeHelperTests::Private
{
	template<typename T>
	void RunTestInternal(FSetAttributeHelperTests& Tester, const T Item1, const T Item2, const T Item3, const T Item4)
	{
		using namespace UE::Interchange;

		const FString BaseKeyName = "TestKey";

		// Initialize from empty
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();

			UE::Interchange::TSetAttributeHelper<T> Helper;
			Helper.Initialize(TestStorage, BaseKeyName);

			Tester.TestEqual(TEXT("Count after empty Initialize"), Helper.GetCount(), 0);
			Tester.TestEqual(TEXT("Count of Set after empty Initialize"), Helper.ToSet().Num(), 0);
		}

		// Initialize from previous TSetAttributeHelper
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();
			{
				UE::Interchange::TSetAttributeHelper<T> Helper;
				Helper.Initialize(TestStorage, BaseKeyName);

				Helper.AddItem(Item1);
				Helper.AddItem(Item2);
				Helper.AddItem(Item3);

				Tester.TestEqual(TEXT("Count after pre-filling a TSetAttributeHelper"), Helper.GetCount(), 3);
				Tester.TestTrue(TEXT(""), Helper.HasItem(Item1));
				Tester.TestTrue(TEXT(""), Helper.HasItem(Item2));
				Tester.TestTrue(TEXT(""), Helper.HasItem(Item3));
			}

			UE::Interchange::TSetAttributeHelper<T> AnotherHelper;
			AnotherHelper.Initialize(TestStorage, BaseKeyName);

			Tester.TestEqual(TEXT("Count after Initialize from a previous TSetAttributeHelper"), AnotherHelper.GetCount(), 3);
			Tester.TestTrue(TEXT(""), AnotherHelper.HasItem(Item1));
			Tester.TestTrue(TEXT(""), AnotherHelper.HasItem(Item2));
			Tester.TestTrue(TEXT(""), AnotherHelper.HasItem(Item3));

			TSet<T> Set = AnotherHelper.ToSet();
			Tester.TestEqual(TEXT("Count of Set after Initialize from a previous TSetAttributeHelper"), Set.Num(), 3);
			Tester.TestTrue(TEXT(""), Set.Contains(Item1));
			Tester.TestTrue(TEXT(""), Set.Contains(Item2));
			Tester.TestTrue(TEXT(""), Set.Contains(Item3));
		}

		// Initialize from previous TArrayAttributeHelper
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();
			{
				UE::Interchange::TArrayAttributeHelper<T> ArrayHelper;
				ArrayHelper.Initialize(TestStorage, BaseKeyName);

				ArrayHelper.AddItem(Item1);
				ArrayHelper.AddItem(Item2);
				ArrayHelper.AddItem(Item3);
			}

			{
				UE::Interchange::TSetAttributeHelper<T> SetHelper;
				SetHelper.Initialize(TestStorage, BaseKeyName);

				Tester.TestEqual(TEXT("Count after Initialize from a previous TArrayAttributeHelper"), SetHelper.GetCount(), 3);
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item1));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item2));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item3));

				TSet<T> Set = SetHelper.ToSet();
				Tester.TestEqual(TEXT("Count of Set after Initialize from a previous TArrayAttributeHelper"), Set.Num(), 3);
				Tester.TestTrue(TEXT(""), Set.Contains(Item1));
				Tester.TestTrue(TEXT(""), Set.Contains(Item2));
				Tester.TestTrue(TEXT(""), Set.Contains(Item3));

				SetHelper.AddItem(Item4);
				SetHelper.RemoveItem(Item1);
				SetHelper.RemoveItem(Item3);

				Tester.TestFalse(TEXT(""), SetHelper.HasItem(Item1));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item2));
				Tester.TestFalse(TEXT(""), SetHelper.HasItem(Item3));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item4));
			}

			{
				UE::Interchange::TArrayAttributeHelper<T> ArrayHelper;
				ArrayHelper.Initialize(TestStorage, BaseKeyName);

				Tester.TestEqual(TEXT("Count of TArrayAttributeHelper after Initialize from a previous TSetAttributeHelper"), ArrayHelper.GetCount(), 2);

				TArray<T> OutItems;
				ArrayHelper.GetItems(OutItems);

				Tester.TestEqual(TEXT("Count of TArray returned by TArrayAttributeHelper"), OutItems.Num(), 2);
				Tester.TestTrue(TEXT("Verify Item2 is in the TArrayAttributeHelper"), OutItems.Contains(Item2));
				Tester.TestTrue(TEXT("Verify Item4 is in the TArrayAttributeHelper"), OutItems.Contains(Item4));
			}
		}

		// AddItem, HasItem, RemoveItem
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();
			UE::Interchange::TSetAttributeHelper<T> Helper;
			Helper.Initialize(TestStorage, BaseKeyName);

			bool bRet;

			// Add
			bRet = Helper.AddItem(Item1);

			Tester.TestTrue(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after adding 1 item"), Helper.GetCount(), 1);
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item1));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item2));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item3));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item4));

			// Duplicate Add
			bRet = Helper.AddItem(Item1);

			Tester.TestFalse(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after adding an existing item"), Helper.GetCount(), 1);
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item1));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item2));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item3));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item4));

			// Add
			Helper.AddItem(Item2);
			Helper.AddItem(Item3);

			Tester.TestEqual(TEXT("Count after adding 2 items"), Helper.GetCount(), 3);
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item1));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item2));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item3));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item4));

			// Remove non existing
			bRet = Helper.RemoveItem(Item4);

			Tester.TestFalse(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after removing an non-existing item"), Helper.GetCount(), 3);
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item1));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item2));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item3));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item4));

			// Remove at first index
			bRet = Helper.RemoveItem(Item1);

			Tester.TestTrue(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after removing item 1"), Helper.GetCount(), 2);
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item1));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item2));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item3));

			// Remove at last index
			bRet = Helper.RemoveItem(Item3);

			Tester.TestTrue(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after removing item 3"), Helper.GetCount(), 1);
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item1));
			Tester.TestTrue(TEXT(""), Helper.HasItem(Item2));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item3));

			// Remove last item in the Set
			bRet = Helper.RemoveItem(Item2);

			Tester.TestTrue(TEXT(""), bRet);
			Tester.TestEqual(TEXT("Count after removing last item"), Helper.GetCount(), 0);
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item1));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item2));
			Tester.TestFalse(TEXT(""), Helper.HasItem(Item3));
		}

		// Empty
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();
			UE::Interchange::TSetAttributeHelper<T> Helper;
			Helper.Initialize(TestStorage, BaseKeyName);

			Helper.Empty();
			Tester.TestEqual(TEXT("Count after emptying a new TSetAttributeHelper"), Helper.GetCount(), 0);

			Helper.AddItem(Item1);
			Helper.AddItem(Item2);
			Helper.AddItem(Item3);
			Helper.AddItem(Item4);

			Tester.TestEqual(TEXT("Count after adding 4 items"), Helper.GetCount(), 4);

			Helper.Empty();
			Tester.TestEqual(TEXT("Count after emptying a full TSetAttributeHelper"), Helper.GetCount(), 0);
		}

		// Gaps and duplicate
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> TestStorage = MakeShared<FAttributeStorage>();
			{
				UE::Interchange::TArrayAttributeHelper<T> ArrayHelper;
				ArrayHelper.Initialize(TestStorage, BaseKeyName);

				ArrayHelper.AddItem(Item1);
				ArrayHelper.AddItem(Item1); // Duplicate

				ArrayHelper.AddItem(Item2);

				ArrayHelper.AddItem(Item3);
				ArrayHelper.AddItem(Item3); // Duplicate

				ArrayHelper.AddItem(Item4);

				Tester.TestEqual(TEXT("Count after pre-filling the TArrayAttributeHelper"), ArrayHelper.GetCount(), 6);
			}

			{
				UE::Interchange::TSetAttributeHelper<T> SetHelper;
				SetHelper.Initialize(TestStorage, BaseKeyName);

				Tester.TestEqual(TEXT("Count after Initializing with duplicated items"), SetHelper.GetCount(), 4);
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item1));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item2));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item3));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item4));
			}

			// Manually remove an entry in the middle to "corrupt" the storage and Initialize a TSetAttributeHelper
			{
				FAttributeKey KeyToRemove = UE::Interchange::BuildArrayKeyFromIndex(BaseKeyName, 2);
				Tester.TestEqual(TEXT("Verify the key is properly removed from storage"), TestStorage->UnregisterAttribute(KeyToRemove), EAttributeStorageResult::Operation_Success);
				
				UE::Interchange::TSetAttributeHelper<T> SetHelper;
				SetHelper.Initialize(TestStorage, BaseKeyName);

				Tester.TestEqual(TEXT("Count after Initializing with corrupted items"), SetHelper.GetCount(), 3);
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item1));
				Tester.TestFalse(TEXT(""), SetHelper.HasItem(Item2));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item3));
				Tester.TestTrue(TEXT(""), SetHelper.HasItem(Item4));
			}
		}
	}
}	 // namespace UE::SetAttributeHelperTests::Private

bool FSetAttributeHelperTests::RunTest(const FString& Parameters)
{
	using namespace UE::SetAttributeHelperTests::Private;

	RunTestInternal<FString>(*this, "Foo", "Bar", "Baz", "Qux");
	RunTestInternal<int32>(*this, 111, 2222, 33333, 444444);
	RunTestInternal<double>(*this, 111.111, 2222.222, 33333.333, 444444.44);
	RunTestInternal<float>(*this, 111.111f, 2222.222f, 33333.333f, 444444.44f);

	return true;
}

#endif	  // WITH_DEV_AUTOMATION_TESTS
