// Copyright Epic Games, Inc. All Rights Reserved.

#include "Async/ParallelFor.h"
#include "CoreMinimal.h"
#include "Misc/AutomationTest.h"
#include "Serialization/MemoryReader.h"
#include "Serialization/MemoryWriter.h"
#include "Templates/Atomic.h"
#include "Types/AttributeStorage.h"
#include "UObject/SoftObjectPath.h"

#if WITH_DEV_AUTOMATION_TESTS

IMPLEMENT_SIMPLE_AUTOMATION_TEST(FAttributeStorageTest, "System.Runtime.Interchange.AttributeStorage", EAutomationTestFlags::EditorContext | EAutomationTestFlags::EngineFilter)


#define ADD_ATTRIBUTE_STORAGE(ATTRIBUTE_TYPE, ATTRIBUTE_VALUE)																							\
{																																						\
	const ATTRIBUTE_TYPE RefValue = ATTRIBUTE_VALUE;																									\
	UE::Interchange::FAttributeKey Key = CreateUniqueKey();																												\
	if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, RefValue)))																	\
	{																																					\
		AddError(FString(TEXT("`AttributeStorage` must handle adding " #ATTRIBUTE_TYPE " attribute")));													\
	}																																					\
	ATTRIBUTE_TYPE StoredValue;																															\
	TestStorage.GetAttributeHandle<ATTRIBUTE_TYPE>(Key).Get(StoredValue);																				\
	TestEqual<ATTRIBUTE_TYPE>(TEXT("`AttributeStorage` must handle retrieving registered " #ATTRIBUTE_TYPE " attribute"), StoredValue, RefValue);		\
	ATTRIBUTE_TYPE EmptyValue{};																														\
	TestStorage.GetAttributeHandle<ATTRIBUTE_TYPE>(Key).Set(EmptyValue);																				\
	ATTRIBUTE_TYPE StoredEmptyValue;																													\
	TestStorage.GetAttributeHandle<ATTRIBUTE_TYPE>(Key).Get(StoredEmptyValue);																			\
	TestEqual<ATTRIBUTE_TYPE>(TEXT("`AttributeStorage` must handle clearing " #ATTRIBUTE_TYPE " attribute value"), StoredEmptyValue, EmptyValue);		\
	TestStorage.GetAttributeHandle<ATTRIBUTE_TYPE>(Key).Set(RefValue);																					\
	ATTRIBUTE_TYPE StoredValueAgain;																													\
	TestStorage.GetAttributeHandle<ATTRIBUTE_TYPE>(Key).Get(StoredValueAgain);																			\
	TestEqual<ATTRIBUTE_TYPE>(TEXT("`AttributeStorage` must handle overwriting " #ATTRIBUTE_TYPE " attribute"), StoredValueAgain, RefValue);			\
}

#define ADD_ATTRIBUTE_STORAGE_NOCOMPARE(ATTRIBUTE_TYPE, ATTRIBUTE_VALUE)																				\
	const ATTRIBUTE_TYPE RefValue = ATTRIBUTE_VALUE;																									\
	UE::Interchange::FAttributeKey Key = CreateUniqueKey();																												\
	if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, RefValue)))																	\
	{																																					\
		AddError(FString(TEXT("`AttributeStorage` must handle adding " #ATTRIBUTE_TYPE " attribute")));													\
	}


namespace UE::InterchangeStorageTest::Private
{
	namespace ETestEnum
	{
		enum Type : int
		{
			Perspective,
			Orthographic
		};
	};
}

bool FAttributeStorageTest::RunTest(const FString& Parameters)
{
	using namespace UE::Interchange;

	FString KeyPrefix = "TestKey";
	int32 KeySuffixCounter = 1;
	auto CreateUniqueKey = [&KeyPrefix, &KeySuffixCounter]()->UE::Interchange::FAttributeKey
	{
		check(IsInGameThread());
		FString KeyFinal = KeyPrefix + TEXT("_") + FString::FromInt(KeySuffixCounter++);
		const UE::Interchange::FAttributeKey UniqueKeyRet = UE::Interchange::FAttributeKey(KeyFinal);
		return UniqueKeyRet;
	};

	FAttributeStorage TestStorage = FAttributeStorage();
	//This seed make the tests deterministic
	FRandomStream RandomStream(564389);
	const UE::Interchange::FAttributeKey TestInt32KeyName = CreateUniqueKey();
	const UE::Interchange::FAttributeKey BigArrayKey = CreateUniqueKey();
	const int32 NegativeValueRef = -1;
	const int32 OverrideValueRef = 3327;
	const UE::Interchange::FAttributeKey RandomStreamKey = CreateUniqueKey();
	int32 TestStoredSeed = 0;
	const int32 RandHelperUInt8 = 255;

	ADD_ATTRIBUTE_STORAGE(bool, false);
	ADD_ATTRIBUTE_STORAGE(bool, true);
	ADD_ATTRIBUTE_STORAGE(FColor, FColor(0xdf,0xcf, 0x00));
	ADD_ATTRIBUTE_STORAGE(FDateTime, FDateTime(2022,01,25,10, 24));
	ADD_ATTRIBUTE_STORAGE(double, 0.1234567890123456789);
	ADD_ATTRIBUTE_STORAGE(float, 0.1234567f);
	ADD_ATTRIBUTE_STORAGE(FGuid, FGuid(0xA4F0E4CD, 0xF97C4375, 0x9C25C211, 0xFD17B8BF));
	ADD_ATTRIBUTE_STORAGE(int8, -0x7f);
	ADD_ATTRIBUTE_STORAGE(int8, 0x7f);
	ADD_ATTRIBUTE_STORAGE(int16, -0x75cc);
	ADD_ATTRIBUTE_STORAGE(int16, 0x7fcc);
	ADD_ATTRIBUTE_STORAGE(int32, -0x75ccabcd);
	ADD_ATTRIBUTE_STORAGE(int32, 0x7fccabcd);
	ADD_ATTRIBUTE_STORAGE(int64, -0x75ccabcd12345678);
	ADD_ATTRIBUTE_STORAGE(int64, 0x7fccabcd12345678);
	ADD_ATTRIBUTE_STORAGE(FIntRect, FIntRect(-1,-2, 1, 2));
	ADD_ATTRIBUTE_STORAGE(FLinearColor, FLinearColor(0.59f, 0.49f, 0.0f));
	ADD_ATTRIBUTE_STORAGE(FName, FName("Testing FName storage"));

	{
		ADD_ATTRIBUTE_STORAGE_NOCOMPARE(FRandomStream, FRandomStream(34))
		FRandomStream StoredValue;
		TestStorage.GetAttributeHandle<FRandomStream>(Key).Get(StoredValue);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve FRandomStream attribute"), StoredValue.ToString(), RefValue.ToString());
	}
	
	ADD_ATTRIBUTE_STORAGE(FString, FString("Testing FString storage"));
	ADD_ATTRIBUTE_STORAGE(FString, FString());
	ADD_ATTRIBUTE_STORAGE(FString, FString("A"));
	ADD_ATTRIBUTE_STORAGE(FTimespan, FTimespan(10, 44, 38));
	ADD_ATTRIBUTE_STORAGE(FTwoVectors, FTwoVectors(FVector(150.0, -203.0, 4500.7), FVector(1.0, 1.0, 1.0)));
	ADD_ATTRIBUTE_STORAGE(uint8, 0x85);
	ADD_ATTRIBUTE_STORAGE(uint8, 0x7f);
	ADD_ATTRIBUTE_STORAGE(uint16, 0x85cc);
	ADD_ATTRIBUTE_STORAGE(uint16, 0x7fcc);
	ADD_ATTRIBUTE_STORAGE(uint32, 0x85ccabcd);
	ADD_ATTRIBUTE_STORAGE(uint32, 0x7fccabcd);
	ADD_ATTRIBUTE_STORAGE(uint64, 0x85ccabcd12345678);
	ADD_ATTRIBUTE_STORAGE(uint64, 0x7fccabcd12345678);
	ADD_ATTRIBUTE_STORAGE(FIntPoint, FIntPoint(1, 4));
	ADD_ATTRIBUTE_STORAGE(FIntVector, FIntVector(1, 4, 3));

	{
		ADD_ATTRIBUTE_STORAGE_NOCOMPARE(FVector2DHalf, FVector2DHalf(150.0, -203.0))
		FVector2DHalf StoredValue;
		TestStorage.GetAttributeHandle<FVector2DHalf>(Key).Get(StoredValue);
		if (TestEqual(TEXT("`AttributeStorage` must handle add and retrieve FVector2DHalf attribute"), StoredValue.X, RefValue.X))
		{
			TestEqual(TEXT("`AttributeStorage` must handle add and retrieve FVector2DHalf attribute"), StoredValue.Y, RefValue.Y);
		}
	}

	ADD_ATTRIBUTE_STORAGE(FFloat16, FFloat16(0.1234567f));

	{
		ADD_ATTRIBUTE_STORAGE_NOCOMPARE(FOrientedBox, FOrientedBox())
		FOrientedBox StoredValue;
		TestStorage.GetAttributeHandle<FOrientedBox>(Key).Get(StoredValue);
		if (!StoredValue.AxisX.Equals(RefValue.AxisX) ||
			!StoredValue.AxisY.Equals(RefValue.AxisY) ||
			!StoredValue.AxisZ.Equals(RefValue.AxisZ) ||
			!FMath::IsNearlyEqual(StoredValue.ExtentX, RefValue.ExtentX) ||
			!FMath::IsNearlyEqual(StoredValue.ExtentY, RefValue.ExtentY) ||
			!FMath::IsNearlyEqual(StoredValue.ExtentZ, RefValue.ExtentZ) ||
			!StoredValue.Center.Equals(RefValue.Center))
		{
			AddError(FString("`AttributeStorage` must handle adding FOrientedBox attribute"));
		}
	}

	ADD_ATTRIBUTE_STORAGE(FFrameNumber, FFrameNumber(30));
	ADD_ATTRIBUTE_STORAGE(FFrameRate, FFrameRate(1,60));
	ADD_ATTRIBUTE_STORAGE(FFrameTime, FFrameTime(29,0.5f));
	ADD_ATTRIBUTE_STORAGE(FSoftObjectPath, FSoftObjectPath(UClass::StaticClass()));
	ADD_ATTRIBUTE_STORAGE(FMatrix44f, FMatrix44f(FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)).ToMatrixWithScale()));
	ADD_ATTRIBUTE_STORAGE(FMatrix44d, FMatrix44d(FTransform3d(FRotator3d(25.0, 2.0, 3.14159), FVector3d(150.0, -203.0, 4500.7), FVector3d(1.0, 1.0, 1.0)).ToMatrixWithScale()));
	ADD_ATTRIBUTE_STORAGE(FPlane4f, FPlane4f(FVector3f(1.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f)));
	ADD_ATTRIBUTE_STORAGE(FPlane4d, FPlane4d(FVector3d(1.0, 1.0, 0), FVector3d(0, 0, 1.0)));
	ADD_ATTRIBUTE_STORAGE(FQuat4f, FQuat4f(1.0f, 1.0f, 0.0f, 1.0f));
	ADD_ATTRIBUTE_STORAGE(FQuat4d, FQuat4d(1.0, 1.0, 0, 1.0));
	ADD_ATTRIBUTE_STORAGE(FRotator3f, FRotator3f(25.0f, 2.0f, 3.14159f));
	ADD_ATTRIBUTE_STORAGE(FRotator3d, FRotator3d(25.0, 2.0, 3.14159));
	{
		ADD_ATTRIBUTE_STORAGE_NOCOMPARE(FTransform3f, FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)))
		FTransform3f StoredValue;
		TestStorage.GetAttributeHandle<FTransform3f>(Key).Get(StoredValue);
		if (!StoredValue.Equals(RefValue))
		{
			AddError(FString("`AttributeStorage` must handle adding FTransform3f attribute"));
		}
	}
	{
		ADD_ATTRIBUTE_STORAGE_NOCOMPARE(FTransform3d, FTransform3d(FRotator3d(25.0, 2.0, 3.14159), FVector3d(150.0, -203.0, 4500.7), FVector3d(1.0, 1.0, 1.0)))
		FTransform3d StoredValue;
		TestStorage.GetAttributeHandle<FTransform>(Key).Get(StoredValue);
		if (!StoredValue.Equals(RefValue))
		{
			AddError(FString("`AttributeStorage` must handle adding FTransform3d attribute"));
		}
	}
	ADD_ATTRIBUTE_STORAGE(FVector3f, FVector3f(150.0f, -203.0f, 4500.7f));
	ADD_ATTRIBUTE_STORAGE(FVector3d, FVector3d(150.0, -203.0, 4500.7));
	ADD_ATTRIBUTE_STORAGE(FVector2f, FVector2f(150.0f, -203.0f));
	ADD_ATTRIBUTE_STORAGE(FVector4f, FVector4f(150.0f, -203.0f, 4500.7f, 2.0f));
	ADD_ATTRIBUTE_STORAGE(FVector4d, FVector4d(150.0, -203.0, 4500.7, 2.0));
	ADD_ATTRIBUTE_STORAGE(FBox2f, FBox2f(FVector2f(-1.0f), FVector2f(1.0f)));
	ADD_ATTRIBUTE_STORAGE(FBox2D, FBox2D(FVector2D(-1.0), FVector2D(1.0)));
	ADD_ATTRIBUTE_STORAGE(FBox3f, FBox3f(FVector3f(-1.0f), FVector3f(1.0f)));
	ADD_ATTRIBUTE_STORAGE(FBox3d, FBox3d(FVector3d(-1.0), FVector3d(1.0)));
	ADD_ATTRIBUTE_STORAGE(FBoxSphereBounds3f, FBoxSphereBounds3f(FVector3f(0.0f), FVector3f(2.0f), 2.2f));
	ADD_ATTRIBUTE_STORAGE(FBoxSphereBounds3d, FBoxSphereBounds3d(FVector3d(0.0), FVector3d(2.0), 2.2));
	{
		FSphere3f RefValue;
		RefValue.Center = FVector3f(2.0f);
		RefValue.W = 2.2f;
		UE::Interchange::FAttributeKey Key = CreateUniqueKey();
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, RefValue)))
		{
			AddError(FString("`AttributeStorage` must handle adding FSphere3f attribute"));
		}
		FSphere3f StoredValue;
		TestStorage.GetAttributeHandle<FSphere3f>(Key).Get(StoredValue);
		if (!StoredValue.Equals(RefValue))
		{
			AddError(FString("`AttributeStorage` must handle adding FSphere3f attribute"));
		}
	}
	{
		FSphere3d RefValue;
		RefValue.Center = FVector3d(2.0);
		RefValue.W = 2.2;
		UE::Interchange::FAttributeKey Key = CreateUniqueKey();
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, RefValue)))
		{
			AddError(FString("`AttributeStorage` must handle adding FSphere3d attribute"));
		}
		FSphere3d StoredValue;
		TestStorage.GetAttributeHandle<FSphere3d>(Key).Get(StoredValue);
		if (!StoredValue.Equals(RefValue))
		{
			AddError(FString("`AttributeStorage` must handle adding FSphere attribute"));
		}
	}

	//Test Adding/Reading uint8 attribute with different value
	{
		ADD_ATTRIBUTE_STORAGE(uint8, 0);
		ADD_ATTRIBUTE_STORAGE(uint8, 43);
	}

	//Test Adding/Reading int32 attribute with a negative value
	{
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(TestInt32KeyName, NegativeValueRef)))
		{
			AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
		}
		int32 StoredValue = 0;
		TestStorage.GetAttributeHandle<int32>(TestInt32KeyName).Get(StoredValue);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve int32 attribute"), StoredValue, NegativeValueRef);
	}

	//Adding many FVector
	for (int32 AddedIndex = 0; AddedIndex < 2; ++AddedIndex)
	{
		const FVector ValueRef = RandomStream.VRand();
		ADD_ATTRIBUTE_STORAGE(FVector, ValueRef);
	}

	//Adding FName
	{
		const FName ValueRef = "The magic carpet ride!";
		ADD_ATTRIBUTE_STORAGE(FName, ValueRef);
	}

	//Adding FSoftObjectPath
	{

		const FSoftObjectPath ValueRef(UClass::StaticClass());
		ADD_ATTRIBUTE_STORAGE(FSoftObjectPath, ValueRef);
	}

	//Adding one big TArray64<uint8> outside of the Hash
	{
		TArray64<uint8> ValueRef;
		uint64 ArrayNum = 500;
		ValueRef.Reserve(ArrayNum);
		for (uint64 ArrayIndex = 0; ArrayIndex < ArrayNum; ++ArrayIndex)
		{
			ValueRef.Add(static_cast<uint8>(RandomStream.RandHelper(RandHelperUInt8)));
		}

		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(BigArrayKey, ValueRef, EAttributeProperty::NoHash)))
		{
			AddError(FString("`AttributeStorage` must handle adding TArray<uint8> attribute"));
		}
		TArray64<uint8> StoredValue;
		TestStorage.GetAttributeHandle<TArray64<uint8> >(BigArrayKey).Get(StoredValue);
		int32 TestArrayIndex = RandomStream.RandHelper(ArrayNum - 1);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray64<uint8> attribute"), StoredValue[TestArrayIndex], ValueRef[TestArrayIndex]);
	}

	//Adding many TArray<uint8>
	for (int32 AddedIndex = 0; AddedIndex < 5; ++AddedIndex)
	{
		TArray<uint8> ValueRef;
		int32 ArrayNum = 2 * (AddedIndex + 1);
		for (int32 ArrayIndex = 0; ArrayIndex < ArrayNum; ++ArrayIndex)
		{
			ValueRef.Add(static_cast<uint8>(RandomStream.RandHelper(RandHelperUInt8)));
		}
		UE::Interchange::FAttributeKey Key = CreateUniqueKey();
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, ValueRef)))
		{
			AddError(FString("`AttributeStorage` must handle adding TArray<uint8> attribute"));
			break;
		}
		TArray<uint8> StoredValue;
		TestStorage.GetAttributeHandle<TArray<uint8> >(Key).Get(StoredValue);
		int32 TestArrayIndex = RandomStream.RandHelper(ArrayNum-1);
		if (!TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<uint8> attribute"), StoredValue[TestArrayIndex], ValueRef[TestArrayIndex]))
		{
			break;
		}
	}

	//Store a FString
	{
		const FString ValueRef = "The quick brown fox jumped over the lazy dogs";
		ADD_ATTRIBUTE_STORAGE(FString, ValueRef);
	}

	//Store the random stream
	{
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(RandomStreamKey, RandomStream)))
		{
			AddError(FString("`AttributeStorage` must handle adding FRandomStream attribute"));
		}
		FRandomStream StoredRandomStream;
		TestStorage.GetAttributeHandle<FRandomStream>(RandomStreamKey).Get(StoredRandomStream);
		//Equal random stream should give the same result when asking fraction
		TestStoredSeed = RandomStream.RandHelper(RandHelperUInt8);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve FRandomStream attribute"), StoredRandomStream.RandHelper(RandHelperUInt8), TestStoredSeed);
	}

	//Store an enum class
	{
		enum class EAttributeStorageEnumTest : uint8
		{
			EnumValue0 = 0,
			EnumValue23 = 23,
			EnumValue35 = 35,
			EnumValue255 = 255,
		};

		ADD_ATTRIBUTE_STORAGE(EAttributeStorageEnumTest, EAttributeStorageEnumTest::EnumValue0);
		ADD_ATTRIBUTE_STORAGE(EAttributeStorageEnumTest, EAttributeStorageEnumTest::EnumValue23);
		ADD_ATTRIBUTE_STORAGE(EAttributeStorageEnumTest, EAttributeStorageEnumTest::EnumValue35);
		ADD_ATTRIBUTE_STORAGE(EAttributeStorageEnumTest, EAttributeStorageEnumTest::EnumValue255);
	}

	//Store a TEnumAsByte class
	{
		enum EBlueprintEnum : int
		{
			Opaque,
			Masked,
			Translucent,
			Additive
		};
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<EBlueprintEnum>, EBlueprintEnum::Opaque);
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<EBlueprintEnum>, EBlueprintEnum::Masked);
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<EBlueprintEnum>, EBlueprintEnum::Translucent);
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<EBlueprintEnum>, EBlueprintEnum::Additive);

		using namespace UE::InterchangeStorageTest::Private;
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<ETestEnum::Type>, ETestEnum::Perspective);
		ADD_ATTRIBUTE_STORAGE(TEnumAsByte<ETestEnum::Type>, ETestEnum::Orthographic);
	}

	// Store array types
	{
		enum class EAttributeStorageEnumTest : uint8
		{
			EnumValue0 = 0,
			EnumValue23 = 23,
			EnumValue35 = 35,
			EnumValue255 = 255,
		};

		enum EBlueprintEnum : int
		{
			Opaque,
			Masked,
			Translucent,
			Additive
		};

		using namespace UE::InterchangeStorageTest::Private;

		ADD_ATTRIBUTE_STORAGE(TArray<bool>, TArray<bool>({true, false, false}));
		ADD_ATTRIBUTE_STORAGE(TArray<FColor>, TArray<FColor>({FColor::Black, FColor::Green, FColor::Yellow}));
		ADD_ATTRIBUTE_STORAGE(TArray<FDateTime>, TArray<FDateTime>({FDateTime(), FDateTime(2024, 02, 03), FDateTime(4765417645)}));
		ADD_ATTRIBUTE_STORAGE(TArray<double>, TArray<double>({0.1234567890123456789, 0.123, -3.5e10}));

		ADD_ATTRIBUTE_STORAGE(TArray<EAttributeStorageEnumTest>, TArray<EAttributeStorageEnumTest>({EAttributeStorageEnumTest::EnumValue0, EAttributeStorageEnumTest::EnumValue255, EAttributeStorageEnumTest::EnumValue35}));
		ADD_ATTRIBUTE_STORAGE(TArray<EAttributeStorageEnumTest>, TArray<EAttributeStorageEnumTest>({EAttributeStorageEnumTest::EnumValue0}));
		ADD_ATTRIBUTE_STORAGE(TArray<EAttributeStorageEnumTest>, TArray<EAttributeStorageEnumTest>({}));
		ADD_ATTRIBUTE_STORAGE(TArray<EBlueprintEnum>, TArray<EBlueprintEnum>({EBlueprintEnum::Opaque, EBlueprintEnum::Masked, EBlueprintEnum::Translucent}));
		ADD_ATTRIBUTE_STORAGE(TArray<EBlueprintEnum>, TArray<EBlueprintEnum>({EBlueprintEnum::Opaque, EBlueprintEnum::Masked}));
		ADD_ATTRIBUTE_STORAGE(TArray<EBlueprintEnum>, TArray<EBlueprintEnum>({}));
		ADD_ATTRIBUTE_STORAGE(TArray<TEnumAsByte<ETestEnum::Type>>, TArray<TEnumAsByte<ETestEnum::Type>>({ETestEnum::Perspective, ETestEnum::Orthographic}));
		ADD_ATTRIBUTE_STORAGE(TArray<TEnumAsByte<ETestEnum::Type>>, TArray<TEnumAsByte<ETestEnum::Type>>({ETestEnum::Perspective}));
		ADD_ATTRIBUTE_STORAGE(TArray<TEnumAsByte<ETestEnum::Type>>, TArray<TEnumAsByte<ETestEnum::Type>>({}));

		ADD_ATTRIBUTE_STORAGE(TArray<float>, TArray<float>({0.1234567f, 0.1234567f, 0.0f}));
		ADD_ATTRIBUTE_STORAGE(TArray<FGuid>, TArray<FGuid>({FGuid(0xA4F0E4CD, 0xF97C4375, 0x9C25C211, 0xFD17B8BF), FGuid()}));
		ADD_ATTRIBUTE_STORAGE(TArray<int8>, TArray<int8>({-0x7f, 65}));
		ADD_ATTRIBUTE_STORAGE(TArray<int16>, TArray<int16>({-0x75cc, -2}));
		ADD_ATTRIBUTE_STORAGE(TArray<int32>, TArray<int32>({0x7fccabcd, 655}));
		ADD_ATTRIBUTE_STORAGE(TArray<int64>, TArray<int64>({0x7fccabcd12345678, 456154}));
		ADD_ATTRIBUTE_STORAGE(TArray<FIntRect>, TArray<FIntRect>({FIntRect(-1,-2, 1, 2), FIntRect(-1,-2, 1, 2), FIntRect()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FLinearColor>, TArray<FLinearColor>({FLinearColor(0.59f, 0.49f, 0.0f), FLinearColor::Red}));
		ADD_ATTRIBUTE_STORAGE(TArray<FName>, TArray<FName>({FName("AAAAAAAAAAAAAAAAAAAAAAAAAAAAAAA"), FName("BBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBBB"), NAME_None}));
		ADD_ATTRIBUTE_STORAGE(TArray<FName>, TArray<FName>({FName("CCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCCC"), NAME_None}));
		ADD_ATTRIBUTE_STORAGE(TArray<FName>, TArray<FName>({NAME_None}));
		ADD_ATTRIBUTE_STORAGE(TArray<FName>, TArray<FName>({}));
		{
			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FRandomStream>, TArray<FRandomStream>({34, 45, 2}));
		}
		ADD_ATTRIBUTE_STORAGE(TArray<FString>, TArray<FString>({TEXT("DDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDDD"), TEXT("EEEEEEEEEEEEEEEEEEEEEEEEEEEE"), FString{}}));
		ADD_ATTRIBUTE_STORAGE(TArray<FString>, TArray<FString>({TEXT("FFFFFFFFFFFFF")}));
		ADD_ATTRIBUTE_STORAGE(TArray<FString>, TArray<FString>({TEXT("")}));
		ADD_ATTRIBUTE_STORAGE(TArray<FString>, TArray<FString>({}));
		ADD_ATTRIBUTE_STORAGE(TArray<FSoftObjectPath>, TArray<FSoftObjectPath>({(UClass::StaticClass(), UObject::StaticClass())}));
		ADD_ATTRIBUTE_STORAGE(TArray<FSoftObjectPath>, TArray<FSoftObjectPath>({(UClass::StaticClass(), FSoftObjectPath())}));
		ADD_ATTRIBUTE_STORAGE(TArray<FSoftObjectPath>, TArray<FSoftObjectPath>({(FSoftObjectPath())}));
		ADD_ATTRIBUTE_STORAGE(TArray<FSoftObjectPath>, TArray<FSoftObjectPath>({}));
		ADD_ATTRIBUTE_STORAGE(TArray<FTimespan>, TArray<FTimespan>({FTimespan{10, 44, 38}, FTimespan{47652}}));
		ADD_ATTRIBUTE_STORAGE(TArray<FTwoVectors>, TArray<FTwoVectors>({ FTwoVectors(FVector(150.0, -203.0, 4500.7), FVector(1.0, 1.0, 1.0)), FTwoVectors(FVector(0.0, 0.0, 4500.7), FVector(1.0, 0.0, 1.0)) }));
		ADD_ATTRIBUTE_STORAGE(TArray<uint8>, TArray<uint8>({0x85, 0}));
		ADD_ATTRIBUTE_STORAGE(TArray<uint16>, TArray<uint16>({0x85cc}));
		ADD_ATTRIBUTE_STORAGE(TArray<uint32>, TArray<uint32>({0x85ccabcd}));
		ADD_ATTRIBUTE_STORAGE(TArray<uint64>, TArray<uint64>({0x85ccabcd12345678}));
		ADD_ATTRIBUTE_STORAGE(TArray<FIntPoint>, TArray<FIntPoint>({FIntPoint{1, 4}, FIntPoint{0, 0}}));
		ADD_ATTRIBUTE_STORAGE(TArray<FIntVector>, TArray<FIntVector>({FIntVector(1, 4, 3), FIntVector()}));

		{
			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FVector2DHalf>, TArray<FVector2DHalf>({FVector2DHalf(150.0, -203.0)}))
			TArray<FVector2DHalf> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FVector2DHalf>>(Key).Get(StoredValue);
			for (int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				if (TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FVector2DHalf> attribute"), StoredValue[Index].X, RefValue[Index].X))
				{
					TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FVector2DHalf> attribute"), StoredValue[Index].Y, RefValue[Index].Y);
				}
			}
		}
		ADD_ATTRIBUTE_STORAGE(TArray<FFloat16>, TArray<FFloat16>({0.12f, 0.0f, -0.2f}));

		{
			TArray<FOrientedBox> SourceBoxes{FOrientedBox(), FOrientedBox(), FOrientedBox()};
			SourceBoxes[0].Center = RandomStream.VRand();
			SourceBoxes[0].AxisX = RandomStream.VRand();
			SourceBoxes[0].AxisY = RandomStream.VRand();
			SourceBoxes[0].AxisZ = RandomStream.VRand();
			SourceBoxes[0].ExtentX = RandomStream.FRand();
			SourceBoxes[0].ExtentY = RandomStream.FRand();
			SourceBoxes[0].ExtentZ = RandomStream.FRand();
			SourceBoxes[1].Center = RandomStream.VRand();
			SourceBoxes[1].AxisX = RandomStream.VRand();
			SourceBoxes[1].AxisY = RandomStream.VRand();
			SourceBoxes[1].AxisZ = RandomStream.VRand();
			SourceBoxes[1].ExtentX = RandomStream.FRand();
			SourceBoxes[1].ExtentY = RandomStream.FRand();
			SourceBoxes[1].ExtentZ = RandomStream.FRand();

			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FOrientedBox>, SourceBoxes)
			TArray<FOrientedBox> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FOrientedBox>>(Key).Get(StoredValue);

			for(int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].Center, RefValue[Index].Center);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].AxisX, RefValue[Index].AxisX);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].AxisY, RefValue[Index].AxisY);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].AxisZ, RefValue[Index].AxisZ);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].ExtentX, RefValue[Index].ExtentX);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].ExtentY, RefValue[Index].ExtentY);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FOrientedBox> attribute"), StoredValue[Index].ExtentZ, RefValue[Index].ExtentZ);
			}
		}

		ADD_ATTRIBUTE_STORAGE(TArray<FFrameNumber>, TArray<FFrameNumber>({FFrameNumber{30}, FFrameNumber{1}, FFrameNumber{}}));
		ADD_ATTRIBUTE_STORAGE(TArray<FFrameRate>, TArray<FFrameRate>({FFrameRate(1,60), FFrameRate()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FFrameTime>, TArray<FFrameTime>({FFrameTime(29,0.5f), FFrameTime()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FMatrix44f>, TArray<FMatrix44f>({
			(FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)).ToMatrixWithScale()),
			(FTransform3f(FRotator3f(0.0f, -2.0f, 3.1f), FVector3f(150.0f, 23.0f, 450.7f), FVector3f(2.0f, 2.0f, 2.0f)).ToMatrixWithScale()),
			FMatrix44f::Identity
		}));
		ADD_ATTRIBUTE_STORAGE(TArray<FMatrix44d>, TArray<FMatrix44d>({
			(FTransform3d(FRotator3d(25.0f, 2.0f, 3.14159f), FVector3d(150.0f, -203.0f, 4500.7f), FVector3d(1.0f, 1.0f, 1.0f)).ToMatrixWithScale()),
			(FTransform3d(FRotator3d(0.0f, -2.0f, 5), FVector3d(15.0f, -2.0f, 40.7f), FVector3d(4.0f, 4.0f, 4.0f)).ToMatrixWithScale()),
			FMatrix44d::Identity
		}));
		ADD_ATTRIBUTE_STORAGE(TArray<FPlane4f>, TArray<FPlane4f>({
			FPlane4f(FVector3f(1.0f, 1.0f, 0.0f), FVector3f(0.0f, 0.0f, 1.0f)),
			FPlane4f(FVector3f(-1.0f, -1.0f, 0.0f), FVector3f(1.0f, 0.0f, 1.0f)),
			FPlane4f(),
		}));
		ADD_ATTRIBUTE_STORAGE(TArray<FPlane4d>, TArray<FPlane4d>({
			FPlane4d(FVector3d(1.0, 1.0, 0), FVector3d(0, 0, 1.0)),
			FPlane4d(FVector3d(1.0, -1.0, 0), FVector3d(2.0, 0, 1.0)),
			FPlane4d(),
		}));
		ADD_ATTRIBUTE_STORAGE(TArray<FQuat4f>, TArray<FQuat4f>({FQuat4f(1.0f, 1.0f, 0.0f, 1.0f), FQuat4f(-1.0f, -1.0f, 0.0f, 1.0f), FQuat4f()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FQuat4d>, TArray<FQuat4d>({FQuat4d(1.0, 1.0, 0, 1.0), FQuat4d(1.0f, 1.0f, 0.0f, 2.0f), FQuat4d::Identity}));
		ADD_ATTRIBUTE_STORAGE(TArray<FRotator3f>, TArray<FRotator3f>({FRotator3f(25.0f, 2.0f, 3.14159f), FRotator3f(2.0f, 3.0f, -3.14159f), FRotator3f()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FRotator3d>, TArray<FRotator3d>({FRotator3d(25.0, 2.0, 3.14159), FRotator3d(55.0, -2.0, 6.14159), FRotator3d()}));

		{
			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FTransform3f>, TArray<FTransform3f>({
				FTransform3f(FRotator3f(25.0f, 2.0f, 3.14159f), FVector3f(150.0f, -203.0f, 4500.7f), FVector3f(1.0f, 1.0f, 1.0f)),
				FTransform3f(FRotator3f(0.0f, -2.0f, 3.59f), FVector3f(10.0f, 3.0f, 450.7f), FVector3f(2.0f, 2.0f, 2.0f)),
				FTransform3f::Identity,
			}));

			TArray<FTransform3f> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FTransform3f>>(Key).Get(StoredValue);
			TestEqual(TEXT("`AttributeStorage` must handle adding TArray<FTransform3f> attribute"), StoredValue.Num(), RefValue.Num());
			for (int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				if (!StoredValue[Index].Equals(RefValue[Index]))
				{
					AddError(FString("`AttributeStorage` must handle adding TArray<FTransform3f> attribute"));
				}
			}
		}
		{
			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FTransform3d>, TArray<FTransform3d>({
				FTransform3d(FRotator3d(25.0f, 2.0f, 3.14159f), FVector3d(150.0f, -203.0f, 4500.7f), FVector3d(1.0f, 1.0f, 1.0f)),
				FTransform3d(FRotator3d(0.0f, -2.0f, 3.59f), FVector3d(10.0f, 3.0f, 450.7f), FVector3d(2.0f, 2.0f, 2.0f)),
				FTransform3d::Identity,
			}));

			TArray<FTransform3d> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FTransform3d>>(Key).Get(StoredValue);
			TestEqual(TEXT("`AttributeStorage` must handle adding TArray<FTransform3d> attribute"), StoredValue.Num(), RefValue.Num());
			for (int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				if (!StoredValue[Index].Equals(RefValue[Index]))
				{
					AddError(FString("`AttributeStorage` must handle adding TArray<FTransform3d> attribute"));
				}
			}
		}

		ADD_ATTRIBUTE_STORAGE(TArray<FVector2f>, TArray<FVector2f>({FVector2f(150.0f, -203.0f), FVector2f(15065.0f, -03.0f), FVector2f()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FVector2d>, TArray<FVector2d>({FVector2d(150.0, -203.0), FVector2d(16550.0, 203.0), FVector2d()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FVector3f>, TArray<FVector3f>({FVector3f(150.0f, -203.0f, 4500.7f), FVector3f()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FVector3d>, TArray<FVector3d>({FVector3d(150.0, -203.0, 4500.7), FVector3d()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FVector4f>, TArray<FVector4f>({FVector4f(150.0f, -203.0f, 4500.7f, 2.0f), FVector4f()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FVector4d>, TArray<FVector4d>({FVector4d(150.0, -203.0, 4500.7, 2.0), FVector4d()}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBox2f>, TArray<FBox2f>({FBox2f(FVector2f(-1.0f), FVector2f(1.0f)), FBox2f(FVector2f(1.0f), FVector2f(-1.0f))}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBox2D>, TArray<FBox2D>({FBox2D(FVector2D(-1.0), FVector2D(1.0)), FBox2D(FVector2D(1.0), FVector2D(-1.0))}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBox3f>, TArray<FBox3f>({FBox3f(FVector3f(-1.0f), FVector3f(1.0f)), FBox3f(FVector3f(1.0f), FVector3f(-1.0f))}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBox3d>, TArray<FBox3d>({FBox3d(FVector3d(-1.0), FVector3d(1.0)), FBox3d(FVector3d(1.0), FVector3d(-1.0))}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBoxSphereBounds3f>, TArray<FBoxSphereBounds3f>({FBoxSphereBounds3f(FVector3f(0.0f), FVector3f(2.0f), 2.2f), FBoxSphereBounds3f(FVector3f(-1.0f), FVector3f(-2.0f), 5.2f)}));
		ADD_ATTRIBUTE_STORAGE(TArray<FBoxSphereBounds3d>, TArray<FBoxSphereBounds3d>({FBoxSphereBounds3d(FVector3d(0.0), FVector3d(2.0), 2.2), FBoxSphereBounds3d(FVector3d(6.0), FVector3d(4.0), 6.2)}));

		{
			TArray<FSphere3f> Spheres{FSphere3f(), FSphere3f(), FSphere3f()};
			Spheres[0].Center.X = RandomStream.FRand();
			Spheres[0].Center.Y = RandomStream.FRand();
			Spheres[0].Center.Z = RandomStream.FRand();
			Spheres[0].W = RandomStream.FRand();
			Spheres[1].Center.X = RandomStream.FRand();
			Spheres[1].Center.Y = RandomStream.FRand();
			Spheres[1].Center.Z = RandomStream.FRand();
			Spheres[1].W = RandomStream.FRand();

			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FSphere3f>, Spheres)
			TArray<FSphere3f> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FSphere3f>>(Key).Get(StoredValue);

			for(int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FSphere3f> attribute"), StoredValue[Index].Center, RefValue[Index].Center);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FSphere3f> attribute"), StoredValue[Index].W, RefValue[Index].W);
			}
		}
		{
			TArray<FSphere3d> Spheres{FSphere3d(), FSphere3d(), FSphere3d()};
			Spheres[0].Center.X = RandomStream.FRand();
			Spheres[0].Center.Y = RandomStream.FRand();
			Spheres[0].Center.Z = RandomStream.FRand();
			Spheres[0].W = RandomStream.FRand();
			Spheres[1].Center.X = RandomStream.FRand();
			Spheres[1].Center.Y = RandomStream.FRand();
			Spheres[1].Center.Z = RandomStream.FRand();
			Spheres[1].W = RandomStream.FRand();

			ADD_ATTRIBUTE_STORAGE_NOCOMPARE(TArray<FSphere3d>, Spheres)
			TArray<FSphere3d> StoredValue;
			TestStorage.GetAttributeHandle<TArray<FSphere3d>>(Key).Get(StoredValue);

			for(int32 Index = 0; Index < StoredValue.Num(); ++Index)
			{
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FSphere3d> attribute"), StoredValue[Index].Center, RefValue[Index].Center);
				TestEqual(TEXT("`AttributeStorage` must handle add and retrieve TArray<FSphere3d> attribute"), StoredValue[Index].W, RefValue[Index].W);
			}
		}
	}

	//Get the Hash we will use it later to do some test
	FGuid HashGuidRef = TestStorage.GetStorageHash();

	//Test hashing twice give the same result
	{
		FGuid HashGuidTest = TestStorage.GetStorageHash();
		if (HashGuidRef != HashGuidTest)
		{
			AddError(FString("`AttributeStorage` hash must be deterministic when calculating it twice."));
		}
	}

	//Removing...
	{
		if (!IsAttributeStorageResultSuccess(TestStorage.UnregisterAttribute(TestInt32KeyName)))
		{
			AddError(FString("`AttributeStorage` must handle removing attribute"));
		}

		//This will kick a defrag of the storage
		if (!IsAttributeStorageResultSuccess(TestStorage.UnregisterAttribute(BigArrayKey)))
		{
			AddError(FString("`AttributeStorage` must handle removing attribute with defrag"));
		}
	}

	//Re-adding the negative integer value
	{
		if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(TestInt32KeyName, NegativeValueRef)))
		{
			AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
		}
		int32 StoredValue = 0;
		TestStorage.GetAttributeHandle<int32>(TestInt32KeyName).Get(StoredValue);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve int32 attribute"), StoredValue, NegativeValueRef);
	}

	//Test hash must be deterministic even if the order of attributes have change
	{
		FGuid HashGuidTest = TestStorage.GetStorageHash();
		if (HashGuidRef != HashGuidTest)
		{
			AddError(FString("`AttributeStorage` hash must be deterministic even if the attributes order differ."));
		}
	}

	//Get the random stream after the defrag and re-adding
	{
		FRandomStream StoredRandomStream;
		TestStorage.GetAttributeHandle<FRandomStream>(RandomStreamKey).Get(StoredRandomStream);
		//Equal random stream should give the same result when asking fraction
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve FRandomStream attribute"), StoredRandomStream.RandHelper(RandHelperUInt8), TestStoredSeed);
	}

	//Overriding value
	{
		int32 StoredValue = 0;
		FAttributeStorage::TAttributeHandle<int32> TestInt32Handle = TestStorage.GetAttributeHandle<int32>(TestInt32KeyName);
		TestInt32Handle.Get(StoredValue);
		TestEqual(TEXT("`AttributeStorage` must handle add and retrieve uint8 attribute"), StoredValue, NegativeValueRef);

		if (!IsAttributeStorageResultSuccess(TestInt32Handle.Set(OverrideValueRef)))
		{
			AddError(FString("`AttributeStorage` must handle overriding int32 attribute"));
		}
		StoredValue = 0;
		TestInt32Handle.Get(StoredValue);
		TestEqual(TEXT("`AttributeStorage` must handle overriding uint32 attribute"), StoredValue, OverrideValueRef);
	}

	//Multi thread test
	{
		//Add multithread (add BatchCount*BatchSize attributes)
		const int32 BatchSize = 10;
		const int32 BatchCount = 50;
		TArray<UE::Interchange::FAttributeKey> Keys;
		Keys.AddZeroed(BatchSize*BatchCount);
		bool ThreadError = false;
		FAttributeStorage::TAttributeHandle<FRandomStream> RandomStreamHandle = TestStorage.GetAttributeHandle<FRandomStream>(RandomStreamKey);
		for (UE::Interchange::FAttributeKey& Key : Keys)
		{
			Key = CreateUniqueKey();
		}

		//Iterate all vertex to compute normals for all vertex instance
		ParallelFor(BatchCount,
			[&TestStorage, &RandomStreamHandle, &BatchSize, &Keys, &CreateUniqueKey, &ThreadError](const int32 BatchIndex)
		{
			//Get the random stream from the storage so all thread add the number of attribute
			FRandomStream RandomStream;
			RandomStreamHandle.Get(RandomStream);
			for (int32 AttributeIndex = 0; AttributeIndex < BatchSize; ++AttributeIndex)
			{
				const uint64 RealIndex = (BatchIndex * BatchSize) + AttributeIndex;
				const FVector ValueRef = RandomStream.VRand();
				if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Keys[RealIndex], ValueRef)))
				{
					ThreadError = true;
					continue;
				}
				FVector StoredValue(0);
				TestStorage.GetAttributeHandle<FVector>(Keys[RealIndex]).Get(StoredValue);
				if (StoredValue.X != ValueRef.X)
				{
					ThreadError = true;
				}
			}
		});
		
		if (ThreadError)
		{
			AddError(FString("`AttributeStorage` Fail adding attributes in multi thread."));
		}

		//Push the defrag ratio to 0.5 to avoid calling defrag to often		
		TestStorage.SetDefragRatio(0.5);
		
		//Now remove in multithread the just added attributes, It should trigger a defrag at some point
		ParallelFor(BatchCount,
			[&TestStorage, &BatchSize, &Keys, &ThreadError](int32 BatchIndex)
		{
			for (int32 AttributeIndex = 0; AttributeIndex < BatchSize; ++AttributeIndex)
			{
				const uint64 RealIndex = BatchIndex * BatchSize + AttributeIndex;
				if (!IsAttributeStorageResultSuccess(TestStorage.UnregisterAttribute(Keys[RealIndex])))
				{
					ThreadError = true;
				}
			}
		});

		if (ThreadError )
		{
			AddError(FString("`AttributeStorage` Fail removing attributes in multi thread."));
		}
	}

	//Test compare storage
	{
		FAttributeStorage BaseStorage = FAttributeStorage();
		FAttributeStorage VersionStorage = FAttributeStorage();
		
		//Adding many FVector
		for (int32 AddedIndex = 0; AddedIndex < 3; ++AddedIndex)
		{
			const FVector ValueRef = RandomStream.VRand();
			UE::Interchange::FAttributeKey Key = CreateUniqueKey();
			if (!IsAttributeStorageResultSuccess(BaseStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding FVector attribute"));
				break;
			}
			if (!IsAttributeStorageResultSuccess(VersionStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding FVector attribute"));
				break;
			}
		}

		//Adding many int32
		for (int32 AddedIndex = 0; AddedIndex < 3; ++AddedIndex)
		{
			const int32 ValueRef = RandomStream.RandHelper(RandHelperUInt8);
			UE::Interchange::FAttributeKey Key = CreateUniqueKey();
			if (!IsAttributeStorageResultSuccess(BaseStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
				break;
			}
			if (!IsAttributeStorageResultSuccess(VersionStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
				break;
			}
		}
		TArray<UE::Interchange::FAttributeKey> RemovedKeys;
		TArray<UE::Interchange::FAttributeKey> AddedKeys;
		TArray<UE::Interchange::FAttributeKey> ModifiedKeys;
		FAttributeStorage::CompareStorage(BaseStorage, VersionStorage, RemovedKeys, AddedKeys, ModifiedKeys);
		if (RemovedKeys.Num() > 0 || AddedKeys.Num() > 0 || ModifiedKeys.Num() > 0)
		{
			AddError(FString("`AttributeStorage` Compare storage shoukld have found no difference."));
		}

		TArray<UE::Interchange::FAttributeKey> VersionKeys;
		VersionStorage.GetAttributeKeys(VersionKeys);

		//Add a integer key to the version storage
		{
			const int32 ValueRef = RandomStream.RandHelper(RandHelperUInt8);
			UE::Interchange::FAttributeKey Key(TEXT("VersionAddedInteger"));
			if (!IsAttributeStorageResultSuccess(VersionStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
			}
		}

		//Modify a key to the version storage
		if (VersionKeys.IsValidIndex(0))
		{
			const FVector ValueRef = RandomStream.VRand();
			if (!IsAttributeStorageResultSuccess(VersionStorage.GetAttributeHandle<FVector>(VersionKeys[0]).Set(ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle overriding FVector attribute"));
			}
		}

		//Remove a key to the version storage
		if(VersionKeys.IsValidIndex(3))
		{
			if (!IsAttributeStorageResultSuccess(VersionStorage.UnregisterAttribute(VersionKeys[3])))
			{
				AddError(FString("`AttributeStorage` must handle removing int32 attribute"));
			}
		}

		FAttributeStorage::CompareStorage(BaseStorage, VersionStorage, RemovedKeys, AddedKeys, ModifiedKeys);
		if (RemovedKeys.Num() != 1 || AddedKeys.Num() != 1 || ModifiedKeys.Num() != 1)
		{
			AddError(FString("`AttributeStorage` Compare storage should have found one difference for each modify/add/remove attruibutes."));
		}
	}

	//Serialization test
	{
		TArray<uint8> MemoryMocked;
		const bool bMemoryPersistent = true;
		FMemoryWriter Ar(MemoryMocked, bMemoryPersistent);
		Ar << TestStorage;

		FAttributeStorage FromMemoryMocked = FAttributeStorage();
		FMemoryReader ArRead(MemoryMocked, bMemoryPersistent);
		ArRead << FromMemoryMocked;

		TArray<UE::Interchange::FAttributeKey> RemovedKeys;
		TArray<UE::Interchange::FAttributeKey> AddedKeys;
		TArray<UE::Interchange::FAttributeKey> ModifiedKeys;

		if(TestStorage.GetStorageHash() != FromMemoryMocked.GetStorageHash())
		{
			AddError(FString("`AttributeStorage` serialization Write then Readback give different storage hash value."));
		}
		
		FAttributeStorage::CompareStorage(TestStorage, FromMemoryMocked, RemovedKeys, AddedKeys, ModifiedKeys);
		if (RemovedKeys.Num() != 0 || AddedKeys.Num() != 0 || ModifiedKeys.Num() != 0)
		{
			AddError(FString("`AttributeStorage` Compare storage should have found no difference after a serialization write follow by a read."));
		}
	}

	//Performance Test, add two million entry, 1 million int32 and 1 million FVector
	{
		FGuid StorageHash = TestStorage.GetStorageHash();
		TArray<UE::Interchange::FAttributeKey> AttributeKeys;
		const int32 TestCount = 10000;
		AttributeKeys.AddZeroed(2*TestCount);
		//Reserve 2x Testcount items (Testcount int32 and Testcount FVector) and reserve the data size
		TestStorage.Reserve(TestCount*2, TestCount*(sizeof(int32)+sizeof(FVector)));
		//Start the timer after we reserve the memory
		const double SmokeTestStartTime = FPlatformTime::Seconds();
		for (int32 AddedIndex = 0; AddedIndex < TestCount; ++AddedIndex)
		{
			const int32 ValueRef = AddedIndex;
			UE::Interchange::FAttributeKey Key = CreateUniqueKey();
			AttributeKeys[AddedIndex*2] = Key;
			if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, ValueRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding int32 attribute"));
				break;
			}
			const FVector3f VectorRef((float)AddedIndex/(float)TestCount);
			Key = CreateUniqueKey();
			AttributeKeys[(AddedIndex*2)+1] = Key;
			if (!IsAttributeStorageResultSuccess(TestStorage.RegisterAttribute(Key, VectorRef)))
			{
				AddError(FString("`AttributeStorage` must handle adding vector attribute"));
				break;
			}
		}
		const double SmokeTestWriteTime = FPlatformTime::Seconds();
		int32 MaxKeys = AttributeKeys.Num();
		for (int32 KeyIndex = 0; KeyIndex < MaxKeys; ++KeyIndex)
		{
			const UE::Interchange::FAttributeKey& Key = AttributeKeys[KeyIndex];
			if(TestStorage.GetAttributeType(Key) == EAttributeTypes::Int32)
			{
				int32 StoredValue(0);
				TestStorage.GetAttributeHandle<int32>(Key).Get(StoredValue);
			}
			else if(TestStorage.GetAttributeType(Key) == EAttributeTypes::Vector3f)
			{
				FVector3f StoredVector(0.0f);
				TestStorage.GetAttributeHandle<FVector3f>(Key).Get(StoredVector);
			}
		}
		const double SmokeTestReadTime = FPlatformTime::Seconds();
		for (int32 KeyIndex = 0; KeyIndex < MaxKeys; ++KeyIndex)
		{
			const UE::Interchange::FAttributeKey& Key = AttributeKeys[KeyIndex];
			TestStorage.UnregisterAttribute(Key);
		}
		const double SmokeTestStopTime = FPlatformTime::Seconds();
		const float TimeForWrite = (SmokeTestWriteTime - SmokeTestStartTime);
		const float TimeForRead = (SmokeTestReadTime - SmokeTestWriteTime);
		const float TimeForUnregister = (SmokeTestStopTime - SmokeTestReadTime);
		const float TimeForTest = (SmokeTestStopTime - SmokeTestStartTime);
		AddInfo(FString::Format(TEXT("AttributeStorage performance test result ({4} int32 and FVector)\n\tRegister: {0}\n\tRead: {1}\n\tUnregister: {2}\n\tTotal: {3}"), {TimeForWrite, TimeForRead, TimeForUnregister, TimeForTest, TestCount}));
		FGuid StorageHashAfter = TestStorage.GetStorageHash();
		if (StorageHashAfter != StorageHash)
		{
			AddError(FString("`AttributeStorage` hash must be deterministic even if the attributes order differ."));
		}
	}
	return true;
}


#endif //WITH_DEV_AUTOMATION_TESTS
