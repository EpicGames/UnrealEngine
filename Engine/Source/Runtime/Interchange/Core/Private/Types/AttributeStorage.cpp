// Copyright Epic Games, Inc. All Rights Reserved.

#include "Types/AttributeStorage.h"

#include "InterchangeLogPrivate.h"

#include "CoreMinimal.h"
#include "HAL/UnrealMemory.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Color.h"
#include "Math/Float16.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Math/Matrix.h"
#include "Math/Plane.h"
#include "Math/OrientedBox.h"
#include "Math/Quat.h"
#include "Math/RandomStream.h"
#include "Math/Rotator.h"
#include "Math/Sphere.h"
#include "Math/TwoVectors.h"
#include "Math/UnrealMathUtility.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector2DHalf.h"
#include "Math/Vector4.h"
#include "Misc/FrameRate.h"
#include "Misc/ScopeLock.h"
#include "Misc/ScopeLock.h"
#include "Misc/SecureHash.h"
#include "UObject/SoftObjectPath.h"

#include <type_traits>

//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		/**
		 * Stub for attribute type traits.
		 *
		 * Actual type traits need to be declared through template specialization for custom
		 * data types that are to be used internally by FAttributeStorage. Traits for the most commonly used built-in
		 * types are declared below.
		 *
		 * Complex types, such as structures and classes, can be serialized into a byte array
		 * and then assigned to an attribute. Note that you will be responsible for ensuring
		 * correct byte ordering when serializing those types.
		 *
		 * @param T The type to be used in FAttributeStorage.
		 */
		template<typename T> 
		struct TAttributeTypeTraitsInternal
		{
		};

		template<typename T>
		FString StringifyArrayAttribute(const T& Array)
		{
			static_assert(TIsTArray<T>::Value);

			FString Result = "[";
			for (const auto& Element : Array)
			{
				Result += TAttributeTypeTraitsInternal<typename T::ElementType>::ToString(Element) + TEXT(", ");
			}
			Result.RemoveFromEnd(TEXT(", "));
			Result += TEXT("]");
			return Result;
		}

		template<> 
		struct TAttributeTypeTraitsInternal<bool>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Bool; }
			static FString ToString(const bool& Value) { return Value ? TEXT("true") : TEXT("false"); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FColor>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Color; }
			static FString ToString(const FColor& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FDateTime>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::DateTime; }
			static FString ToString(const FDateTime& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<double>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Double; }
			static FString ToString(const double& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<float>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Float; }
			static FString ToString(const float& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FGuid>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Guid; }
			static FString ToString(const FGuid& Value) { return Value.ToString(EGuidFormats::DigitsLower); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<int8>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int8; }
			static FString ToString(const int8& Value) { return FString::FromInt(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<int16>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int16; }
			static FString ToString(const int16& Value) { return FString::FromInt(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<int32>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int32; }
			static FString ToString(const int32& Value) { return FString::FromInt(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<int64>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int64; }
			static FString ToString(const int64& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FIntRect>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntRect; }
			static FString ToString(const FIntRect& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FLinearColor>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::LinearColor; }
			static FString ToString(const FLinearColor& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FName>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Name; }
			static FString ToString(const FName& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FRandomStream>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::RandomStream; }
			static FString ToString(const FRandomStream& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FString>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::String; }
			static FString ToString(const FString& Value) { return Value; }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FTimespan>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Timespan; }
			static FString ToString(const FTimespan& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FTwoVectors>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::TwoVectors; }
			static FString ToString(const FTwoVectors& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<uint8>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt8; }
			static FString ToString(const uint8& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<uint16>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt16; }
			static FString ToString(const uint16& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<uint32>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt32; }
			static FString ToString(const uint32& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<uint64>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt64; }
			static FString ToString(const uint64& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector2D>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2d; }
			static FString ToString(const FVector2D& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FIntPoint>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntPoint; }
			static FString ToString(const FIntPoint& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FIntVector>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntVector; }
			static FString ToString(const FIntVector& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector2DHalf>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2DHalf; }
			static FString ToString(const FVector2DHalf& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FFloat16>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Float16; }
			static FString ToString(const FFloat16& Value) { return LexToString(Value.GetFloat()); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FOrientedBox>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::OrientedBox; }
			static FString ToString(const FOrientedBox& Value) 
			{
				return FString::Printf(
					TEXT("Center= %f, %f, %f | AxisX= %f, %f, %f | AxisY= %f, %f, %f | AxisZ= %f, %f, %f | Extents= %f, %f, %f"),
					Value.Center.X,
					Value.Center.Y,
					Value.Center.Z,
					Value.AxisX.X,
					Value.AxisX.Y,
					Value.AxisX.Z,
					Value.AxisY.X,
					Value.AxisY.Y,
					Value.AxisY.Z,
					Value.AxisZ.X,
					Value.AxisZ.Y,
					Value.AxisZ.Z,
					Value.ExtentX,
					Value.ExtentY,
					Value.ExtentZ
				);
			}
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FFrameNumber>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameNumber; }
			static FString ToString(const FFrameNumber& Value) { return LexToString(Value); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FFrameRate>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameRate; }
			static FString ToString(const FFrameRate& Value) { return Value.ToPrettyText().ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FFrameTime>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameTime; }
			static FString ToString(const FFrameTime& Value) { return LexToString(Value.AsDecimal()); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FSoftObjectPath>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::SoftObjectPath; }
			static FString ToString(const FSoftObjectPath& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FMatrix44f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Matrix44f; }
			static FString ToString(const FMatrix44f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FMatrix44d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Matrix44d; }
			static FString ToString(const FMatrix44d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FPlane4f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Plane4f; }
			static FString ToString(const FPlane4f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FPlane4d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Plane4d; }
			static FString ToString(const FPlane4d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FQuat4f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Quat4f; }
			static FString ToString(const FQuat4f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FQuat4d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Quat4d; }
			static FString ToString(const FQuat4d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FRotator3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Rotator3f; }
			static FString ToString(const FRotator3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FRotator3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Rotator3d; }
			static FString ToString(const FRotator3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FTransform3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Transform3f; }
			static FString ToString(const FTransform3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FTransform3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Transform3d; }
			static FString ToString(const FTransform3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector3f; }
			static FString ToString(const FVector3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector3d; }
			static FString ToString(const FVector3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector2f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2f; }
			static FString ToString(const FVector2f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector4f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector4f; }
			static FString ToString(const FVector4f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FVector4d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector4d; }
			static FString ToString(const FVector4d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBox2f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box2f; }
			static FString ToString(const FBox2f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBox2D>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box2D; }
			static FString ToString(const FBox2D& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBox3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box3f; }
			static FString ToString(const FBox3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBox3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box3d; }
			static FString ToString(const FBox3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBoxSphereBounds3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::BoxSphereBounds3f; }
			static FString ToString(const FBoxSphereBounds3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FBoxSphereBounds3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::BoxSphereBounds3d; }
			static FString ToString(const FBoxSphereBounds3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FSphere3f>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Sphere3f; }
			static FString ToString(const FSphere3f& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<FSphere3d>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Sphere3d; }
			static FString ToString(const FSphere3d& Value) { return Value.ToString(); }
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<bool>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::BoolArray; }
			static FString ToString(const TArray<bool>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FColor>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::ColorArray; }
			static FString ToString(const TArray<FColor>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FDateTime>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::DateTimeArray; }
			static FString ToString(const TArray<FDateTime>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<double>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::DoubleArray; }
			static FString ToString(const TArray<double>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<float>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FloatArray; }
			static FString ToString(const TArray<float>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FGuid>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::GuidArray; }
			static FString ToString(const TArray<FGuid>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<int8>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int8Array; }
			static FString ToString(const TArray<int8>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<int16>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int16Array; }
			static FString ToString(const TArray<int16>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<int32>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int32Array; }
			static FString ToString(const TArray<int32>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<int64>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Int64Array; }
			static FString ToString(const TArray<int64>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FIntRect>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntRectArray; }
			static FString ToString(const TArray<FIntRect>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FLinearColor>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::LinearColorArray; }
			static FString ToString(const TArray<FLinearColor>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FName>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::NameArray; }
			static FString ToString(const TArray<FName>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FRandomStream>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::RandomStreamArray; }
			static FString ToString(const TArray<FRandomStream>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FString>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::StringArray; }
			static FString ToString(const TArray<FString>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FTimespan>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::TimespanArray; }
			static FString ToString(const TArray<FTimespan>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FTwoVectors>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::TwoVectorsArray; }
			static FString ToString(const TArray<FTwoVectors>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<uint8>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::ByteArray; }
			static FString ToString(const TArray<uint8>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray64<uint8>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::ByteArray64; }
			static FString ToString(const TArray64<uint8>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<uint16>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt16Array; }
			static FString ToString(const TArray<uint16>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<uint32>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt32Array; }
			static FString ToString(const TArray<uint32>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<uint64>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::UInt64Array; }
			static FString ToString(const TArray<uint64>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector2D>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2dArray; }
			static FString ToString(const TArray<FVector2D>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FIntPoint>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntPointArray; }
			static FString ToString(const TArray<FIntPoint>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FIntVector>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::IntVectorArray; }
			static FString ToString(const TArray<FIntVector>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector2DHalf>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2DHalfArray; }
			static FString ToString(const TArray<FVector2DHalf>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FFloat16>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Float16Array; }
			static FString ToString(const TArray<FFloat16>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FOrientedBox>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::OrientedBoxArray; }
			static FString ToString(const TArray<FOrientedBox>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FFrameNumber>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameNumberArray; }
			static FString ToString(const TArray<FFrameNumber>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FFrameRate>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameRateArray; }
			static FString ToString(const TArray<FFrameRate>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FFrameTime>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::FrameTimeArray; }
			static FString ToString(const TArray<FFrameTime>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FSoftObjectPath>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::SoftObjectPathArray; }
			static FString ToString(const TArray<FSoftObjectPath>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FMatrix44f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Matrix44fArray; }
			static FString ToString(const TArray<FMatrix44f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FMatrix44d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Matrix44dArray; }
			static FString ToString(const TArray<FMatrix44d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FPlane4f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Plane4fArray; }
			static FString ToString(const TArray<FPlane4f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FPlane4d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Plane4dArray; }
			static FString ToString(const TArray<FPlane4d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FQuat4f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Quat4fArray; }
			static FString ToString(const TArray<FQuat4f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FQuat4d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Quat4dArray; }
			static FString ToString(const TArray<FQuat4d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FRotator3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Rotator3fArray; }
			static FString ToString(const TArray<FRotator3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FRotator3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Rotator3dArray; }
			static FString ToString(const TArray<FRotator3d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FTransform3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Transform3fArray; }
			static FString ToString(const TArray<FTransform3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FTransform3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Transform3dArray; }
			static FString ToString(const TArray<FTransform3d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector3fArray; }
			static FString ToString(const TArray<FVector3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector3dArray; }
			static FString ToString(const TArray<FVector3d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector2f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector2fArray; }
			static FString ToString(const TArray<FVector2f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector4f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector4fArray; }
			static FString ToString(const TArray<FVector4f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FVector4d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Vector4dArray; }
			static FString ToString(const TArray<FVector4d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBox2f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box2fArray; }
			static FString ToString(const TArray<FBox2f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBox2D>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box2DArray; }
			static FString ToString(const TArray<FBox2D>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBox3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box3fArray; }
			static FString ToString(const TArray<FBox3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBox3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Box3dArray; }
			static FString ToString(const TArray<FBox3d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBoxSphereBounds3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::BoxSphereBounds3fArray; }
			static FString ToString(const TArray<FBoxSphereBounds3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FBoxSphereBounds3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::BoxSphereBounds3dArray; }
			static FString ToString(const TArray<FBoxSphereBounds3d>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FSphere3f>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Sphere3fArray; }
			static FString ToString(const TArray<FSphere3f>& Value) { return StringifyArrayAttribute(Value); };
		};

		template<> 
		struct TAttributeTypeTraitsInternal<TArray<FSphere3d>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::Sphere3dArray; }
			static FString ToString(const TArray<FSphere3d>& Value) { return StringifyArrayAttribute(Value); };
		};


		template<>
		struct TAttributeTypeTraitsInternal<FInterchangeLODMeshData>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::InterchangeLODMeshData; }
			static FString ToString(const FInterchangeLODMeshData& Value) { return Value.ToString(); };
		};

		template<>
		struct TAttributeTypeTraitsInternal<TArray<FInterchangeLODMeshData>>
		{
			static constexpr EAttributeTypes GetType() { return EAttributeTypes::InterchangeLODMeshDataArray; }
			static FString ToString(const TArray<FInterchangeLODMeshData>& Value) { return StringifyArrayAttribute(Value); };
		};

		FString AttributeTypeToString(EAttributeTypes AttributeType)
		{
			FString AttributeTypeString;
			switch (AttributeType)
			{
				case EAttributeTypes::None: { AttributeTypeString = TEXT("None"); } break;

				case EAttributeTypes::Bool: { AttributeTypeString = TEXT("Bool"); } break;
				case EAttributeTypes::Color: { AttributeTypeString = TEXT("Color"); } break;
				case EAttributeTypes::DateTime: { AttributeTypeString = TEXT("DateTime"); } break;
				case EAttributeTypes::Double: { AttributeTypeString = TEXT("Double"); } break;
				case EAttributeTypes::Enum: { AttributeTypeString = TEXT("Enum"); } break;
				case EAttributeTypes::Float: { AttributeTypeString = TEXT("Float"); } break;
				case EAttributeTypes::Guid: { AttributeTypeString = TEXT("Guid"); } break;
				case EAttributeTypes::Int8: { AttributeTypeString = TEXT("Int8"); } break;
				case EAttributeTypes::Int16: { AttributeTypeString = TEXT("Int16"); } break;
				case EAttributeTypes::Int32: { AttributeTypeString = TEXT("Int32"); } break;
				case EAttributeTypes::Int64: { AttributeTypeString = TEXT("Int64"); } break;
				case EAttributeTypes::IntRect: { AttributeTypeString = TEXT("IntRect"); } break;
				case EAttributeTypes::LinearColor: { AttributeTypeString = TEXT("LinearColor"); } break;
				case EAttributeTypes::Name: { AttributeTypeString = TEXT("Name"); } break;
				case EAttributeTypes::RandomStream: { AttributeTypeString = TEXT("RandomStream"); } break;
				case EAttributeTypes::String: { AttributeTypeString = TEXT("String"); } break;
				case EAttributeTypes::Timespan: { AttributeTypeString = TEXT("Timespan"); } break;
				case EAttributeTypes::TwoVectors: { AttributeTypeString = TEXT("TwoVectors"); } break;
				case EAttributeTypes::UInt8: { AttributeTypeString = TEXT("UInt8"); } break;
				case EAttributeTypes::UInt16: { AttributeTypeString = TEXT("UInt16"); } break;
				case EAttributeTypes::UInt32: { AttributeTypeString = TEXT("UInt32"); } break;
				case EAttributeTypes::UInt64: { AttributeTypeString = TEXT("UInt64"); } break;
				case EAttributeTypes::Vector2d: { AttributeTypeString = TEXT("Vector2d"); } break;
				case EAttributeTypes::IntPoint: { AttributeTypeString = TEXT("IntPoint"); } break;
				case EAttributeTypes::IntVector: { AttributeTypeString = TEXT("IntVector"); } break;
				case EAttributeTypes::Vector2DHalf: { AttributeTypeString = TEXT("Vector2DHalf"); } break;
				case EAttributeTypes::Float16: { AttributeTypeString = TEXT("Float16"); } break;
				case EAttributeTypes::OrientedBox: { AttributeTypeString = TEXT("OrientedBox"); } break;
				case EAttributeTypes::FrameNumber: { AttributeTypeString = TEXT("FrameNumber"); } break;
				case EAttributeTypes::FrameRate: { AttributeTypeString = TEXT("FrameRate"); } break;
				case EAttributeTypes::FrameTime: { AttributeTypeString = TEXT("FrameTime"); } break;
				case EAttributeTypes::SoftObjectPath: { AttributeTypeString = TEXT("SoftObjectPath"); } break;
				case EAttributeTypes::Matrix44f: { AttributeTypeString = TEXT("Matrix44f"); } break;
				case EAttributeTypes::Matrix44d: { AttributeTypeString = TEXT("Matrix44d"); } break;
				case EAttributeTypes::Plane4f: { AttributeTypeString = TEXT("Plane4f"); } break;
				case EAttributeTypes::Plane4d: { AttributeTypeString = TEXT("Plane4d"); } break;
				case EAttributeTypes::Quat4f: { AttributeTypeString = TEXT("Quat4f"); } break;
				case EAttributeTypes::Quat4d: { AttributeTypeString = TEXT("Quat4d"); } break;
				case EAttributeTypes::Rotator3f: { AttributeTypeString = TEXT("Rotator3f"); } break;
				case EAttributeTypes::Rotator3d: { AttributeTypeString = TEXT("Rotator3d"); } break;
				case EAttributeTypes::Transform3f: { AttributeTypeString = TEXT("Transform3f"); } break;
				case EAttributeTypes::Transform3d: { AttributeTypeString = TEXT("Transform3d"); } break;
				case EAttributeTypes::Vector3f: { AttributeTypeString = TEXT("Vector3f"); } break;
				case EAttributeTypes::Vector3d: { AttributeTypeString = TEXT("Vector3d"); } break;
				case EAttributeTypes::Vector2f: { AttributeTypeString = TEXT("Vector2f"); } break;
				case EAttributeTypes::Vector4f: { AttributeTypeString = TEXT("Vector4f"); } break;
				case EAttributeTypes::Vector4d: { AttributeTypeString = TEXT("Vector4d"); } break;
				case EAttributeTypes::Box2f: { AttributeTypeString = TEXT("Box2f"); } break;
				case EAttributeTypes::Box2D: { AttributeTypeString = TEXT("Box2D"); } break;
				case EAttributeTypes::Box3f: { AttributeTypeString = TEXT("Box3f"); } break;
				case EAttributeTypes::Box3d: { AttributeTypeString = TEXT("Box3d"); } break;
				case EAttributeTypes::BoxSphereBounds3f: { AttributeTypeString = TEXT("BoxSphereBounds3f"); } break;
				case EAttributeTypes::BoxSphereBounds3d: { AttributeTypeString = TEXT("BoxSphereBounds3d"); } break;
				case EAttributeTypes::Sphere3f: { AttributeTypeString = TEXT("Sphere3f"); } break;
				case EAttributeTypes::Sphere3d: { AttributeTypeString = TEXT("Sphere3d"); } break;

				case EAttributeTypes::BoolArray: { AttributeTypeString = TEXT("BoolArray"); } break;
				case EAttributeTypes::ColorArray: { AttributeTypeString = TEXT("ColorArray"); } break;
				case EAttributeTypes::DateTimeArray: { AttributeTypeString = TEXT("DateTimeArray"); } break;
				case EAttributeTypes::DoubleArray: { AttributeTypeString = TEXT("DoubleArray"); } break;
				case EAttributeTypes::EnumArray: { AttributeTypeString = TEXT("EnumArray"); } break;
				case EAttributeTypes::FloatArray: { AttributeTypeString = TEXT("FloatArray"); } break;
				case EAttributeTypes::GuidArray: { AttributeTypeString = TEXT("GuidArray"); } break;
				case EAttributeTypes::Int8Array: { AttributeTypeString = TEXT("Int8Array"); } break;
				case EAttributeTypes::Int16Array: { AttributeTypeString = TEXT("Int16Array"); } break;
				case EAttributeTypes::Int32Array: { AttributeTypeString = TEXT("Int32Array"); } break;
				case EAttributeTypes::Int64Array: { AttributeTypeString = TEXT("Int64Array"); } break;
				case EAttributeTypes::IntRectArray: { AttributeTypeString = TEXT("IntRectArray"); } break;
				case EAttributeTypes::LinearColorArray: { AttributeTypeString = TEXT("LinearColorArray"); } break;
				case EAttributeTypes::NameArray: { AttributeTypeString = TEXT("NameArray"); } break;
				case EAttributeTypes::RandomStreamArray: { AttributeTypeString = TEXT("RandomStreamArray"); } break;
				case EAttributeTypes::StringArray: { AttributeTypeString = TEXT("StringArray"); } break;
				case EAttributeTypes::TimespanArray: { AttributeTypeString = TEXT("TimespanArray"); } break;
				case EAttributeTypes::TwoVectorsArray: { AttributeTypeString = TEXT("TwoVectorsArray"); } break;
				case EAttributeTypes::ByteArray: { AttributeTypeString = TEXT("ByteArray"); } break;
				case EAttributeTypes::ByteArray64: { AttributeTypeString = TEXT("ByteArray64"); } break;
				case EAttributeTypes::UInt16Array: { AttributeTypeString = TEXT("UInt16Array"); } break;
				case EAttributeTypes::UInt32Array: { AttributeTypeString = TEXT("UInt32Array"); } break;
				case EAttributeTypes::UInt64Array: { AttributeTypeString = TEXT("UInt64Array"); } break;
				case EAttributeTypes::Vector2dArray: { AttributeTypeString = TEXT("Vector2dArray"); } break;
				case EAttributeTypes::IntPointArray: { AttributeTypeString = TEXT("IntPointArray"); } break;
				case EAttributeTypes::IntVectorArray: { AttributeTypeString = TEXT("IntVectorArray"); } break;
				case EAttributeTypes::Vector2DHalfArray: { AttributeTypeString = TEXT("Vector2DHalfArray"); } break;
				case EAttributeTypes::Float16Array: { AttributeTypeString = TEXT("Float16Array"); } break;
				case EAttributeTypes::OrientedBoxArray: { AttributeTypeString = TEXT("OrientedBoxArray"); } break;
				case EAttributeTypes::FrameNumberArray: { AttributeTypeString = TEXT("FrameNumberArray"); } break;
				case EAttributeTypes::FrameRateArray: { AttributeTypeString = TEXT("FrameRateArray"); } break;
				case EAttributeTypes::FrameTimeArray: { AttributeTypeString = TEXT("FrameTimeArray"); } break;
				case EAttributeTypes::SoftObjectPathArray: { AttributeTypeString = TEXT("SoftObjectPathArray"); } break;
				case EAttributeTypes::Matrix44fArray: { AttributeTypeString = TEXT("Matrix44fArray"); } break;
				case EAttributeTypes::Matrix44dArray: { AttributeTypeString = TEXT("Matrix44dArray"); } break;
				case EAttributeTypes::Plane4fArray: { AttributeTypeString = TEXT("Plane4fArray"); } break;
				case EAttributeTypes::Plane4dArray: { AttributeTypeString = TEXT("Plane4dArray"); } break;
				case EAttributeTypes::Quat4fArray: { AttributeTypeString = TEXT("Quat4fArray"); } break;
				case EAttributeTypes::Quat4dArray: { AttributeTypeString = TEXT("Quat4dArray"); } break;
				case EAttributeTypes::Rotator3fArray: { AttributeTypeString = TEXT("Rotator3fArray"); } break;
				case EAttributeTypes::Rotator3dArray: { AttributeTypeString = TEXT("Rotator3dArray"); } break;
				case EAttributeTypes::Transform3fArray: { AttributeTypeString = TEXT("Transform3fArray"); } break;
				case EAttributeTypes::Transform3dArray: { AttributeTypeString = TEXT("Transform3dArray"); } break;
				case EAttributeTypes::Vector3fArray: { AttributeTypeString = TEXT("Vector3fArray"); } break;
				case EAttributeTypes::Vector3dArray: { AttributeTypeString = TEXT("Vector3dArray"); } break;
				case EAttributeTypes::Vector2fArray: { AttributeTypeString = TEXT("Vector2fArray"); } break;
				case EAttributeTypes::Vector4fArray: { AttributeTypeString = TEXT("Vector4fArray"); } break;
				case EAttributeTypes::Vector4dArray: { AttributeTypeString = TEXT("Vector4dArray"); } break;
				case EAttributeTypes::Box2fArray: { AttributeTypeString = TEXT("Box2fArray"); } break;
				case EAttributeTypes::Box2DArray: { AttributeTypeString = TEXT("Box2DArray"); } break;
				case EAttributeTypes::Box3fArray: { AttributeTypeString = TEXT("Box3fArray"); } break;
				case EAttributeTypes::Box3dArray: { AttributeTypeString = TEXT("Box3dArray"); } break;
				case EAttributeTypes::BoxSphereBounds3fArray: { AttributeTypeString = TEXT("BoxSphereBounds3fArray"); } break;
				case EAttributeTypes::BoxSphereBounds3dArray: { AttributeTypeString = TEXT("BoxSphereBounds3dArray"); } break;
				case EAttributeTypes::Sphere3fArray: { AttributeTypeString = TEXT("Sphere3fArray"); } break;
				case EAttributeTypes::Sphere3dArray: { AttributeTypeString = TEXT("Sphere3dArray"); } break;

				case EAttributeTypes::InterchangeLODMeshData: { AttributeTypeString = TEXT("InterchangeLODMeshData"); } break;
				case EAttributeTypes::InterchangeLODMeshDataArray: { AttributeTypeString = TEXT("InterchangeLODMeshDataArray"); } break;

				default:
					ensureMsgf(false, TEXT("Unsupported EAttributeTypes value!"));
					break;
			}
			return AttributeTypeString;
		}

		EAttributeTypes StringToAttributeType(const FString& AttributeTypeString)
		{
			const int32 MaxAttributetypes = static_cast<int32>(EAttributeTypes::Max);
			for (int32 AttributeTypeEnumIndex = 0; AttributeTypeEnumIndex < MaxAttributetypes; ++AttributeTypeEnumIndex)
			{
				const EAttributeTypes Attributetype = static_cast<EAttributeTypes>(AttributeTypeEnumIndex);
				if (AttributeTypeString.Equals(AttributeTypeToString(Attributetype)))
				{
					return Attributetype;
				}
			}
			return EAttributeTypes::None;
		}

		template<typename ValueType>
		FString AttributeValueToString(const ValueType& Value)
		{
			return TAttributeTypeTraitsInternal<ValueType>::ToString(Value);
		}

		/** Return the storage size of the value in bytes */
		template<typename T>
		uint64 GetValueSizeInternal(const T& Value)
		{
			if constexpr (TIsTArray<T>::Value)
			{
				// This should not be used for nested array types (arrays of strings)
				static_assert(!TIsArrayOfStringType<T>);

				return Value.NumBytes();
			}
			else if constexpr (std::is_same_v<T, FString>)
			{
				// We must add the null character '/0' terminate string
				return (Value.Len() + 1) * sizeof(TCHAR);
			}
			else if constexpr (std::is_same_v<T, FName> || std::is_same_v<T, FSoftObjectPath>)
			{
				FString ValueStr = Value.ToString();
				return GetValueSizeInternal(ValueStr);
			}
			else // Common single values (float, FRotator3f, int32, etc.)
			{
				const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();
				if (ValueType == EAttributeTypes::None)
				{
					return 0;
				}
				return sizeof(T);
			}
		}

		void LogAttributeStorageErrors(const EAttributeStorageResult Result, const FString OperationName, const FAttributeKey AttributeKey)
		{
			//////////////////////////////////////////////////////////////////////////
			//Errors
			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted))
			{
				UE_LOGF(LogInterchangeCore, Error, "Attribute storage operation [%ls] Key[%ls]: Storage is corrupted.", *OperationName, *(AttributeKey.ToString()));
			}

			//////////////////////////////////////////////////////////////////////////
			//Warning
			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_CannotFoundKey))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Cannot find attribute key.", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_CannotOverrideAttribute))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Cannot override attribute.", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_CannotRemoveAttribute))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Cannot remove an attribute.", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_WrongSize))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Stored attribute value size does not match parameter value size.", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_WrongType))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Stored attribute value type does not match parameter value type.", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_InvalidStorage))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: The storage is invalid (NULL).", *OperationName, *(AttributeKey.ToString()));
			}

			if (HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Error_InvalidMultiSizeValueData))
			{
				UE_LOGF(LogInterchangeCore, Warning, "Attribute storage operation [%ls] Key[%ls]: Cannot retrieve the multisize value data pointer.", *OperationName, *(AttributeKey.ToString()));
			}
		}

		FAttributeStorage::FAttributeStorage(const FAttributeStorage& Other)
		{
			//Lock both Storage mutex
			FScopeLock ScopeLock(&StorageMutex);
			FScopeLock ScopeLockOther(&Other.StorageMutex);

			//Copy all value
			AttributeAllocationTable = Other.AttributeAllocationTable;
			AttributeStorage = Other.AttributeStorage;
			FragmentedMemoryCost = Other.FragmentedMemoryCost;
			DefragRatio = Other.DefragRatio;
			AllocationCount = Other.AllocationCount;
		}

		FAttributeStorage& FAttributeStorage::operator=(const FAttributeStorage& Other)
		{
			//Lock both Storage mutex
			FScopeLock ScopeLock(&StorageMutex);
			FScopeLock ScopeLockOther(&Other.StorageMutex);

			//Copy all value
			AttributeAllocationTable = Other.AttributeAllocationTable;
			AttributeStorage = Other.AttributeStorage;
			FragmentedMemoryCost = Other.FragmentedMemoryCost;
			DefragRatio = Other.DefragRatio;
			AllocationCount = Other.AllocationCount;

			return *this;
		}

		template<typename T>
		FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::TAttributeHandle()
			: AttributeStorage(nullptr)
			, Key()
		{
			static_assert(TAttributeTypeTraitsInternal<T>::GetType() != EAttributeTypes::None, "Unsupported attribute type");
		}

		template<typename T>
		bool FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::IsValid() const
		{
			const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();
			return (AttributeStorage && AttributeStorage->ContainAttribute(Key) && AttributeStorage->GetAttributeType(Key) == ValueType);
		}

		template<typename T>
		EAttributeStorageResult FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::Get(T& Value) const
		{
			if (AttributeStorage)
			{
				return AttributeStorage->GetAttribute<T>(Key, Value);
			}
			return EAttributeStorageResult::Operation_Error_InvalidStorage;
		}

		template<typename T>
		EAttributeStorageResult FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::Set(const T& Value)
		{
			if (AttributeStorage)
			{
				return AttributeStorage->SetAttribute<T>(Key, Value);
			}
			return EAttributeStorageResult::Operation_Error_InvalidStorage;
		}
		
		template<typename T>
		const FAttributeKey& FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::GetKey() const
		{
			return Key;
		}
		
		template<typename T>
		FAttributeStorage::TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>::TAttributeHandle(const FAttributeKey& InKey, const FAttributeStorage* InAttributeStorage)
		{
			AttributeStorage = const_cast<FAttributeStorage*>(InAttributeStorage);
			Key = InKey;
		
			//Look for storage validity
			if (AttributeStorage == nullptr)
			{
				//Storage is null
				LogAttributeStorageErrors(EAttributeStorageResult::Operation_Error_InvalidStorage, TEXT("GetAttributeHandle"), Key);
			}
			else
			{

				if (!AttributeStorage->ContainAttribute(Key))
				{
					//Storage do not contain the key
					LogAttributeStorageErrors(EAttributeStorageResult::Operation_Error_CannotFoundKey, TEXT("GetAttributeHandle"), Key);
				}
				if (AttributeStorage->GetAttributeType(Key) != TAttributeTypeTraitsInternal<T>::GetType())
				{
					//Value Type is different from the existing key
					LogAttributeStorageErrors(EAttributeStorageResult::Operation_Error_WrongType, TEXT("GetAttributeHandle"), Key);
				}
			}
		}

		template<typename T, typename>
		EAttributeStorageResult FAttributeStorage::RegisterAttribute(const FAttributeKey& ElementAttributeKey, const T& DefaultValue, EAttributeProperty AttributeProperty)
		{
			static_assert(TAttributeTypeTraitsInternal<T>::GetType() != EAttributeTypes::None, "T is not a supported type for the attributes. Check EAttributeTypes for the supported types");
			static_assert(!TIsEnum<T>::Value && !TIsTEnumAsByte<T>::Value, "This function should only be instantiated for enum underlying types.");

			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();

			FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			if (AttributeAllocationInfo)
			{
				if (AttributeAllocationInfo->Type != ValueType)
				{
					return EAttributeStorageResult::Operation_Error_WrongType;
				}

				for (uint64 Offset : AttributeAllocationInfo->Offsets)
				{
					if (!AttributeStorage.IsValidIndex(Offset))
					{
						return EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
					}
				}
			}
			else
			{
				// Create multiple allocations in case of nested array types
				if constexpr (TIsArrayOfStringType<T>)
				{
					AttributeAllocationInfo = &AttributeAllocationTable.Add(ElementAttributeKey);
					AttributeAllocationInfo->Type = ValueType;
					AttributeAllocationInfo->Sizes.SetNumZeroed(DefaultValue.Num());
					AttributeAllocationInfo->Offsets.SetNumZeroed(DefaultValue.Num());
					AllocationCount += DefaultValue.Num();

					for (int32 Index = 0; Index < DefaultValue.Num(); ++Index)
					{
						const uint64 ElementValueSize = GetValueSizeInternal(DefaultValue[Index]);

						AttributeAllocationInfo->Sizes[Index] = ElementValueSize;
						AttributeAllocationInfo->Offsets[Index] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
					}
				}
				else if constexpr (std::is_same_v<T, FInterchangeLODMeshData>)
				{
					constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
					static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");
					
					AttributeAllocationInfo = &AttributeAllocationTable.Add(ElementAttributeKey);
					AttributeAllocationInfo->Type = ValueType;
					AttributeAllocationInfo->Sizes.SetNumZeroed(NumberOfAttributes);
					AttributeAllocationInfo->Offsets.SetNumZeroed(NumberOfAttributes);
					AllocationCount += NumberOfAttributes;

					{
						const uint64 ElementValueSize = GetValueSizeInternal(DefaultValue.MeshUid);

						AttributeAllocationInfo->Sizes[0] = ElementValueSize;
						AttributeAllocationInfo->Offsets[0] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
					}
					{
						const uint64 ElementValueSize = GetValueSizeInternal(DefaultValue.SceneNodeName);

						AttributeAllocationInfo->Sizes[1] = ElementValueSize;
						AttributeAllocationInfo->Offsets[1] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
					}
					{
						const uint64 ElementValueSize = GetValueSizeInternal(DefaultValue.Transform);

						AttributeAllocationInfo->Sizes[2] = ElementValueSize;
						AttributeAllocationInfo->Offsets[2] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
					}
					{
						const uint64 ElementValueSize = GetValueSizeInternal(DefaultValue.GeometricTransform);

						AttributeAllocationInfo->Sizes[3] = ElementValueSize;
						AttributeAllocationInfo->Offsets[3] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
					}
				}
				else if constexpr (std::is_same_v<T, TArray<FInterchangeLODMeshData>>)
				{
					constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
					static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");

					const TArray<FInterchangeLODMeshData>& InterchangeLODMeshArray = DefaultValue;
					const int32 NumOfAllocations = InterchangeLODMeshArray.Num() * NumberOfAttributes;
					AttributeAllocationInfo = &AttributeAllocationTable.Add(ElementAttributeKey);
					AttributeAllocationInfo->Type = ValueType;
					AttributeAllocationInfo->Sizes.SetNumZeroed(NumOfAllocations);
					AttributeAllocationInfo->Offsets.SetNumZeroed(NumOfAllocations);
					AllocationCount += NumOfAllocations;

					for (int32 Index = 0; Index < InterchangeLODMeshArray.Num(); ++Index)
					{
						const FInterchangeLODMeshData& InterchangeLODMesh = InterchangeLODMeshArray[Index];
						int32 MeshUidIndex = Index * NumberOfAttributes;
						int32 SceneNodeNameIndex = MeshUidIndex + 1;
						int32 TransformIndex = SceneNodeNameIndex + 1;
						int32 GeometricTransformIndex = TransformIndex + 1;
						{
							const uint64 ElementValueSize = GetValueSizeInternal(InterchangeLODMesh.MeshUid);

							AttributeAllocationInfo->Sizes[MeshUidIndex] = ElementValueSize;
							AttributeAllocationInfo->Offsets[MeshUidIndex] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
						}
						{
							const uint64 ElementValueSize = GetValueSizeInternal(InterchangeLODMesh.SceneNodeName);

							AttributeAllocationInfo->Sizes[SceneNodeNameIndex] = ElementValueSize;
							AttributeAllocationInfo->Offsets[SceneNodeNameIndex] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
						}
						{
							const uint64 ElementValueSize = GetValueSizeInternal(InterchangeLODMesh.Transform);

							AttributeAllocationInfo->Sizes[TransformIndex] = ElementValueSize;
							AttributeAllocationInfo->Offsets[TransformIndex] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
						}
						{
							const uint64 ElementValueSize = GetValueSizeInternal(InterchangeLODMesh.GeometricTransform);

							AttributeAllocationInfo->Sizes[GeometricTransformIndex] = ElementValueSize;
							AttributeAllocationInfo->Offsets[GeometricTransformIndex] = (uint64)AttributeStorage.AddZeroed(ElementValueSize);
						}
					}
				}
				// Create single allocation for all other types
				else
				{
					const uint64 ValueSize = GetValueSizeInternal(DefaultValue);

					AttributeAllocationInfo = &AttributeAllocationTable.Add(ElementAttributeKey);
					AttributeAllocationInfo->Type = ValueType;
					AttributeAllocationInfo->Sizes = {ValueSize};
					AttributeAllocationInfo->Offsets = {(uint64)AttributeStorage.AddZeroed(ValueSize)};
					AllocationCount += 1;
				}
			}

			//Force the specified attribute property
			AttributeAllocationInfo->Property = AttributeProperty;

			//Use template SetAttribute which support all the types
			const EAttributeStorageResult Result = SetAttribute(AttributeAllocationInfo, DefaultValue);
			if (!IsAttributeStorageResultSuccess(Result))
			{
				//An error occured, unregister the key from the storage
				UnregisterAttribute(ElementAttributeKey);
			}
			return Result;
		}

		EAttributeStorageResult FAttributeStorage::UnregisterAttribute(const FAttributeKey& ElementAttributeKey)
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			if (!AttributeAllocationInfo)
			{
				return EAttributeStorageResult::Operation_Error_CannotFoundKey;
			}

			for (uint64 Size : AttributeAllocationInfo->Sizes)
			{
				FragmentedMemoryCost += Size;
			}
			AllocationCount -= AttributeAllocationInfo->Sizes.Num();

			if (AttributeAllocationTable.Remove(ElementAttributeKey) == 0)
			{
				return EAttributeStorageResult::Operation_Error_CannotRemoveAttribute;
			}

			DefragInternal();

			return EAttributeStorageResult::Operation_Success;
		}

		template<typename T, typename>
		INTERCHANGECORE_API EAttributeStorageResult FAttributeStorage::GetAttribute(const FAttributeKey& ElementAttributeKey, T& OutValue) const
		{
			static_assert(TAttributeTypeTraitsInternal<T>::GetType() != EAttributeTypes::None, "T is not a supported type for the attributes. Check EAttributeTypes for the supported types");
			static_assert(!TIsEnum<T>::Value && !TIsTEnumAsByte<T>::Value, "This function should only be instantiated for enum underlying types.");

			if constexpr (TIsTArray<T>::Value)
			{
				return GenericArrayGetAttribute(ElementAttributeKey, OutValue);
			}
			else // Non-array types
			{
				//Lock the storage
				FScopeLock ScopeLock(&StorageMutex);

				const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();

				const FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
				if (AttributeAllocationInfo)
				{
					if (AttributeAllocationInfo->Type != ValueType)
					{
						return EAttributeStorageResult::Operation_Error_WrongType;
					}
					if constexpr (!std::is_same_v<T, FInterchangeLODMeshData>)
					{
						if (!ensure(AttributeAllocationInfo->Sizes.Num() == 1))
						{
							return EAttributeStorageResult::Operation_Error_WrongSize;
						}
						if constexpr (!TIsStringType<T>)
						{
							// For non-array/non-string types we should have the exact same size as the type we're retrieving
							if (AttributeAllocationInfo->Sizes[0] != GetValueSizeInternal(OutValue))
							{
								return EAttributeStorageResult::Operation_Error_WrongSize;
							}
						}
						if (!ensure(AttributeAllocationInfo->Offsets.Num() == 1) || !AttributeStorage.IsValidIndex(AttributeAllocationInfo->Offsets[0]))
						{
							return EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
						}
					}
				}
				else
				{
					//The key do not exist
					return EAttributeStorageResult::Operation_Error_CannotFoundKey;
				}

				if constexpr (TIsStringType<T>)
				{
					if (AttributeAllocationInfo->Sizes[0] <= sizeof(TCHAR))	   // Account for the null-terminator
					{
						OutValue = T();
						return EAttributeStorageResult::Operation_Success;
					}

					const FStringView ValueView = GetFStringViewAttributeFromStorage(AttributeStorage.GetData(), AttributeAllocationInfo);
					OutValue = T{ValueView};
				}
				else if constexpr (std::is_same_v<T, FInterchangeLODMeshData>)
				{
					const uint64 NumAllocations = AttributeAllocationInfo->Sizes.Num();

					constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
					static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");

					if (!ensure(NumAllocations == NumberOfAttributes))
					{
						return EAttributeStorageResult::Operation_Error_WrongSize;
					}

					const uint64 NumElements = NumAllocations;
					const uint8* StorageData = AttributeStorage.GetData();

					FInterchangeLODMeshData& InterchangeLODMesh = OutValue;

					//FString
					{
						const FStringView ValueView = GetFStringViewAttributeFromStorage(StorageData, AttributeAllocationInfo, 0);
						InterchangeLODMesh.MeshUid = ValueView;
					}

					//FString
					{
						const FStringView ValueView = GetFStringViewAttributeFromStorage(StorageData, AttributeAllocationInfo, 1);
						InterchangeLODMesh.SceneNodeName = ValueView;
					}

					//FTransform
					{
						const uint8* SourceBytes = &StorageData[AttributeAllocationInfo->Offsets[2]];
						const uint64 AllocationSize = AttributeAllocationInfo->Sizes[2];
						FMemory::Memcpy(&InterchangeLODMesh.Transform, SourceBytes, AllocationSize);
					}

					//FTransform
					{
						const uint8* SourceBytes = &StorageData[AttributeAllocationInfo->Offsets[3]];
						const uint64 AllocationSize = AttributeAllocationInfo->Sizes[3];
						FMemory::Memcpy(&InterchangeLODMesh.GeometricTransform, SourceBytes, AllocationSize);
					}
				}
				else
				{
					const uint8* StorageData = AttributeStorage.GetData();
					FMemory::Memcpy(&OutValue, &StorageData[AttributeAllocationInfo->Offsets[0]], AttributeAllocationInfo->Sizes[0]);
				}

				return EAttributeStorageResult::Operation_Success;
			}
		}

		template<typename T, typename>
		FAttributeStorage::TAttributeHandle<T> FAttributeStorage::GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const
		{
			TAttributeHandle<T> AttributeHandle(ElementAttributeKey, this);
			return AttributeHandle;
		}

		EAttributeTypes FAttributeStorage::GetAttributeType(const FAttributeKey& ElementAttributeKey) const
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);
			const FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			if (AttributeAllocationInfo)
			{
				return AttributeAllocationInfo->Type;
			}
			return EAttributeTypes::None;
		}

		bool FAttributeStorage::ContainAttribute(const FAttributeKey& ElementAttributeKey) const
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			return AttributeAllocationTable.Contains(ElementAttributeKey);
		}

		void FAttributeStorage::GetAttributeKeys(TArray<FAttributeKey>& AttributeKeys) const
		{
			FScopeLock ScopeLock(&StorageMutex);
			AttributeAllocationTable.GetKeys(AttributeKeys);
		}

		FGuid FAttributeStorage::GetAttributeHash(const FAttributeKey& ElementAttributeKey) const
		{
			FGuid AttributeHash;
			GetAttributeHash(ElementAttributeKey, AttributeHash);
			return AttributeHash;
		}

		bool FAttributeStorage::GetAttributeHash(const FAttributeKey& ElementAttributeKey, FGuid& OutGuid) const
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			const FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			if (AttributeAllocationInfo)
			{
				OutGuid = AttributeAllocationInfo->Hash;
				return true;
			}
			return false;
		}

		FGuid FAttributeStorage::GetStorageHash() const
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			TArray<FAttributeKey> OrderedKeys;
			AttributeAllocationTable.GetKeys(OrderedKeys);
			OrderedKeys.Sort();
			FSHA1 Sha;
			for (FAttributeKey Key : OrderedKeys)
			{
				const FAttributeAllocationInfo& AttributeAllocationInfo = AttributeAllocationTable.FindChecked(Key);
				//Skip Attribute that has the no hash flag
				if (HasAttributeProperty(AttributeAllocationInfo.Property, EAttributeProperty::NoHash))
				{
					continue;
				}
				uint32 GuidData[4];
				GuidData[0] = AttributeAllocationInfo.Hash.A;
				GuidData[1] = AttributeAllocationInfo.Hash.B;
				GuidData[2] = AttributeAllocationInfo.Hash.C;
				GuidData[3] = AttributeAllocationInfo.Hash.D;
				Sha.Update(reinterpret_cast<uint8*>(&GuidData[0]), 16);
			}
			Sha.Final();
			// Retrieve the hash and use it to construct a pseudo-GUID. 
			uint32 Hash[5];
			Sha.GetHash(reinterpret_cast<uint8*>(Hash));
			return FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
		}

#define COMPARE_MATH_STRUCTS( MathType ) \
	case EAttributeTypes::MathType: \
	{ \
		F##MathType BaseTypeValue; \
		ensure(GetAttributeValue(*this, BaseTypeValue)); \
		F##MathType VersionTypeValue; \
		bool bGetVersionValue = GetAttributeValue(VersionStorage, VersionTypeValue); \
		if(!bGetVersionValue) \
		{ \
			return false; \
		} \
		return BaseTypeValue.Equals(VersionTypeValue); \
	}

#define COMPARE_MATH_ARRAY( MathType ) \
	case EAttributeTypes::MathType##Array: \
	{ \
		TArray<F##MathType> BaseArray; \
		ensure(GetAttributeValue(*this, BaseArray)); \
		TArray<F##MathType> VersionArray; \
		bool bGetVersionValue = GetAttributeValue(VersionStorage, VersionArray); \
		if(!bGetVersionValue || BaseArray.Num() != VersionArray.Num()) \
		{ \
			return false; \
		} \
		for (int32 Index = 0; Index < BaseArray.Num(); ++Index) \
		{ \
			if(!BaseArray[Index].Equals(VersionArray[Index])) \
			{ \
				return false; \
			} \
		} \
		return true; \
	}

#define DECLARE_LOCAL_VARIABLES( ValueType ) \
	F##ValueType Base##ValueType; \
	ensure(GetAttributeValue(*this, Base##ValueType)); \
	F##ValueType Version##ValueType; \
	bool bGetVersionValue = GetAttributeValue(VersionStorage, Version##ValueType); \
	if (!bGetVersionValue) \
	{ \
		return false; \
	}

#define DECLARE_LOCAL_ARRAY_VARIABLES( ValueType ) \
	TArray<F##ValueType> Base##ValueType##s; \
	ensure(GetAttributeValue(*this, Base##ValueType##s)); \
	TArray<F##ValueType> Version##ValueType##s; \
	bool bGetVersionValue = GetAttributeValue(VersionStorage, Version##ValueType##s); \
	if (!bGetVersionValue || Base##ValueType##s.Num() != Version##ValueType##s.Num()) \
	{ \
		return false; \
	}

		bool FAttributeStorage::AreAllocationInfosEqual(const FAttributeKey& BaseKey, const FAttributeAllocationInfo& BaseInfo, const FAttributeStorage& VersionStorage, const FAttributeAllocationInfo& VersionInfo) const
		{
			if (BaseInfo == VersionInfo)
			{
				return true;
			}

			using namespace UE::Math;

			auto CompareTransform = []<typename T>(const TTransform<T>& Base, const TTransform<T>& Version) -> bool
			{
				bool bResult = TTransform<T>::AreTranslationsEqual(Base, Version);
				bResult &= TTransform<T>::AreScale3DsEqual(Base, Version);

				// Do not compare TQuat values but resulting TRotator values as TQuat values are eventually converted to TRotator
				// and 2 different TQuat values can generate the same TRotator value
				bResult &= Base.GetRotation().Rotator().Equals(Version.GetRotation().Rotator());

				return bResult;
			};

			auto GetAttributeValue = [&BaseKey]<typename ValueType>(const FAttributeStorage & Attributes, ValueType& OutValue) -> bool
			{
				if (!Attributes.ContainAttribute(BaseKey))
				{
					return false;
				}

				TAttributeHandle<ValueType> AttributeHandle = Attributes.GetAttributeHandle<ValueType>(BaseKey);
				if (!AttributeHandle.IsValid())
				{
					return false;
				}

				EAttributeStorageResult Result = AttributeHandle.Get(OutValue);
				if (!IsAttributeStorageResultSuccess(Result))
				{
					LogAttributeStorageErrors(Result, TEXT("FAttributeStorage::AreAllocationInfosEqual"), BaseKey);
					return false;
				}

				return true;
			};


			// Hash comparison cannot be relied on for floating point values
			switch (BaseInfo.Type)
			{
				COMPARE_MATH_STRUCTS(Vector4d)
				COMPARE_MATH_STRUCTS(Vector4f)
				COMPARE_MATH_STRUCTS(Vector3d)
				COMPARE_MATH_STRUCTS(Vector3f)
				COMPARE_MATH_STRUCTS(Vector2d)
				COMPARE_MATH_STRUCTS(Vector2f)
				COMPARE_MATH_STRUCTS(Box3d)
				COMPARE_MATH_STRUCTS(Box3f)
				COMPARE_MATH_STRUCTS(Box2D)
				COMPARE_MATH_STRUCTS(Box2f)
				COMPARE_MATH_STRUCTS(Sphere3d)
				COMPARE_MATH_STRUCTS(Sphere3f)
				COMPARE_MATH_STRUCTS(Plane4d)
				COMPARE_MATH_STRUCTS(Plane4f)
				COMPARE_MATH_STRUCTS(Quat4d)
				COMPARE_MATH_STRUCTS(Quat4f)
				COMPARE_MATH_STRUCTS(Matrix44d)
				COMPARE_MATH_STRUCTS(Matrix44f)
				COMPARE_MATH_STRUCTS(Rotator3d)
				COMPARE_MATH_STRUCTS(Rotator3f)
				COMPARE_MATH_STRUCTS(LinearColor)

				COMPARE_MATH_ARRAY(Vector4d)
				COMPARE_MATH_ARRAY(Vector4f)
				COMPARE_MATH_ARRAY(Vector3d)
				COMPARE_MATH_ARRAY(Vector3f)
				COMPARE_MATH_ARRAY(Vector2d)
				COMPARE_MATH_ARRAY(Vector2f)
				COMPARE_MATH_ARRAY(Box3d)
				COMPARE_MATH_ARRAY(Box3f)
				COMPARE_MATH_ARRAY(Box2D)
				COMPARE_MATH_ARRAY(Box2f)
				COMPARE_MATH_ARRAY(Sphere3d)
				COMPARE_MATH_ARRAY(Sphere3f)
				COMPARE_MATH_ARRAY(Plane4d)
				COMPARE_MATH_ARRAY(Plane4f)
				COMPARE_MATH_ARRAY(Quat4d)
				COMPARE_MATH_ARRAY(Quat4f)
				COMPARE_MATH_ARRAY(Matrix44d)
				COMPARE_MATH_ARRAY(Matrix44f)
				COMPARE_MATH_ARRAY(Rotator3d)
				COMPARE_MATH_ARRAY(Rotator3f)
				COMPARE_MATH_ARRAY(LinearColor)

				case EAttributeTypes::Transform3f:
				{
					DECLARE_LOCAL_VARIABLES(Transform3f )
					return CompareTransform(BaseTransform3f, VersionTransform3f);
				}

				case EAttributeTypes::Transform3fArray:
				{
					DECLARE_LOCAL_ARRAY_VARIABLES(Transform3f)

					for (int32 Index = 0; Index < BaseTransform3fs.Num(); ++Index)
					{
						if (!CompareTransform(BaseTransform3fs[Index], VersionTransform3fs[Index]))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::Transform3d:
				{
					DECLARE_LOCAL_VARIABLES(Transform3d)
					return CompareTransform(BaseTransform3d, VersionTransform3d);
				}

				case EAttributeTypes::Transform3dArray:
				{
					DECLARE_LOCAL_ARRAY_VARIABLES(Transform3d)

					for (int32 Index = 0; Index < BaseTransform3ds.Num(); ++Index)
					{
						if (!CompareTransform(BaseTransform3ds[Index], VersionTransform3ds[Index]))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::OrientedBox:
				{
					DECLARE_LOCAL_VARIABLES(OrientedBox)

					bool bIsEqual = BaseOrientedBox.AxisX.Equals(VersionOrientedBox.AxisX);
					bIsEqual &= BaseOrientedBox.AxisY.Equals(VersionOrientedBox.AxisY);
					bIsEqual &= BaseOrientedBox.AxisZ.Equals(VersionOrientedBox.AxisZ);
					bIsEqual &= BaseOrientedBox.Center.Equals(VersionOrientedBox.Center);
					bIsEqual &= FMath::IsNearlyEqual(BaseOrientedBox.ExtentX, VersionOrientedBox.ExtentX);
					bIsEqual &= FMath::IsNearlyEqual(BaseOrientedBox.ExtentY, VersionOrientedBox.ExtentY);
					bIsEqual &= FMath::IsNearlyEqual(BaseOrientedBox.ExtentZ, VersionOrientedBox.ExtentZ);

					return bIsEqual;
				}

				case EAttributeTypes::TwoVectors:
				{
					DECLARE_LOCAL_VARIABLES(TwoVectors)

					bool bIsEqual = BaseTwoVectors.v1.Equals(VersionTwoVectors.v1);
					bIsEqual &= BaseTwoVectors.v2.Equals(VersionTwoVectors.v2);

					return bIsEqual;
				}

				case EAttributeTypes::TwoVectorsArray:
				{
					DECLARE_LOCAL_ARRAY_VARIABLES(TwoVectors)

					for (int32 Index = 0; Index < BaseTwoVectorss.Num(); ++Index)
					{
						const FTwoVectors& BaseValue = BaseTwoVectorss[Index];
						const FTwoVectors& VersionValue = VersionTwoVectorss[Index];

						if (!BaseValue.v1.Equals(VersionValue.v1) || !BaseValue.v2.Equals(VersionValue.v2))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::Float16:
				{
					DECLARE_LOCAL_VARIABLES(Float16)
					return FMath::IsNearlyEqual(BaseFloat16.GetFloat(), VersionFloat16.GetFloat());
				}

				case EAttributeTypes::Float16Array:
				{
					DECLARE_LOCAL_ARRAY_VARIABLES(Float16)

					for (int32 Index = 0; Index < BaseFloat16s.Num(); ++Index)
					{
						if (!FMath::IsNearlyEqual(BaseFloat16s[Index].GetFloat(), VersionFloat16s[Index].GetFloat()))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::Vector2DHalf:
				{
					DECLARE_LOCAL_VARIABLES(Vector2DHalf)
					return FMath::IsNearlyEqual(BaseVector2DHalf.X.GetFloat(), VersionVector2DHalf.X.GetFloat())
						&& FMath::IsNearlyEqual(BaseVector2DHalf.Y.GetFloat(), VersionVector2DHalf.Y.GetFloat());
				}

				case EAttributeTypes::Vector2DHalfArray:
				{
					DECLARE_LOCAL_ARRAY_VARIABLES(Vector2DHalf)

					for (int32 Index = 0; Index < BaseVector2DHalfs.Num(); ++Index)
					{
						const FVector2DHalf& BaseValue = BaseVector2DHalfs[Index];
						const FVector2DHalf& VersionValue = VersionVector2DHalfs[Index];
						
						if (!FMath::IsNearlyEqual(BaseValue.X.GetFloat(), VersionValue.X.GetFloat()) || !FMath::IsNearlyEqual(BaseValue.Y.GetFloat(), VersionValue.Y.GetFloat()))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::Float:
				{
					float BaseFloat;
					ensure(GetAttributeValue(*this, BaseFloat));
					float VersionFloat;
					bool bVersionValue = GetAttributeValue(VersionStorage, VersionFloat);
					
					if (!bVersionValue)
					{
						return false;
					}

					return FMath::IsNearlyEqual(BaseFloat, VersionFloat);
				}

				case EAttributeTypes::FloatArray:
				{
					TArray<float> BaseFloats;
					ensure(GetAttributeValue(*this, BaseFloats));
					TArray<float> VersionFloats;
					bool bVersionValue = GetAttributeValue(VersionStorage, VersionFloats);

					if (!bVersionValue || BaseFloats.Num() != VersionFloats.Num())
					{
						return false;
					}

					for (int32 Index = 0; Index < BaseFloats.Num(); ++Index)
					{
						if (!FMath::IsNearlyEqual(BaseFloats[Index], VersionFloats[Index]))
						{
							return false;
						}
					}

					return true;
				}

				case EAttributeTypes::Double:
				{
					double BaseDouble;
					ensure(GetAttributeValue(*this, BaseDouble));
					double VersionDouble;
					bool bVersionValue = GetAttributeValue(VersionStorage, VersionDouble);

					if (!bVersionValue)
					{
						return false;
					}

					return FMath::IsNearlyEqual(BaseDouble, VersionDouble);
				}

				case EAttributeTypes::DoubleArray:
				{
					TArray<double> BaseDoubles;
					ensure(GetAttributeValue(*this, BaseDoubles));
					TArray<double> VersionDoubles;
					bool bVersionValue = GetAttributeValue(VersionStorage, VersionDoubles);

					if (!bVersionValue || BaseDoubles.Num() != VersionDoubles.Num())
					{
						return false;
					}

					for (int32 Index = 0; Index < BaseDoubles.Num(); ++Index)
					{
						if (!FMath::IsNearlyEqual(BaseDoubles[Index], VersionDoubles[Index]))
						{
							return false;
						}
					}

					return true;
				}

				default:
					break; 
			}

			return BaseInfo.Hash == VersionInfo.Hash;
		}

		void FAttributeStorage::CompareStorage(const FAttributeStorage& BaseStorage, const FAttributeStorage& VersionStorage, TArray<FAttributeKey>& RemovedAttributes, TArray<FAttributeKey>& AddedAttributes, TArray<FAttributeKey>& ModifiedAttributes)
		{
			//Lock the storage
			FScopeLock ScopeLockVersion(&VersionStorage.StorageMutex);
			//Lock the storage
			FScopeLock ScopeLockBase(&BaseStorage.StorageMutex);

			for (const auto& KvpBase : BaseStorage.AttributeAllocationTable)
			{
				FAttributeKey KeyBase = KvpBase.Key;
				const FAttributeAllocationInfo* AttributeAllocationInfoVersion = VersionStorage.AttributeAllocationTable.Find(KeyBase);
				if (!AttributeAllocationInfoVersion)
				{
					//Add the attribute to RemovedAttributes
					RemovedAttributes.Add(KeyBase);
				}
				else if (!BaseStorage.AreAllocationInfosEqual(KeyBase, KvpBase.Value, VersionStorage , *AttributeAllocationInfoVersion))
				{
					//Add the attribute to ModifiedAttributes
					ModifiedAttributes.Add(KeyBase);
				}
			}

			for (const auto& KvpVersion : VersionStorage.AttributeAllocationTable)
			{
				FAttributeKey KeyVersion = KvpVersion.Key;
				const FAttributeAllocationInfo* AttributeAllocationInfoBase = BaseStorage.AttributeAllocationTable.Find(KeyVersion);
				if (!AttributeAllocationInfoBase)
				{
					//Add the attribute to RemovedAttributes
					AddedAttributes.Add(KeyVersion);
				}
			}
		}

		template<typename T>
		void CopyStorageAttributesInternal(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<T>& AttributeKeys)
		{
			//Lock both storages
			FScopeLock SourceScopeLock(&SourceStorage.StorageMutex);
			FScopeLock DestinationScopeLock(&DestinationStorage.StorageMutex);

			constexpr bool bIsPairAttributeKey = std::is_same_v<T, TPair<FAttributeKey, FAttributeKey>>;

			for (const T& AttributeKey : AttributeKeys)
			{
				const FAttributeStorage::FAttributeAllocationInfo* SourceAttributeInfo = nullptr;

				if constexpr (bIsPairAttributeKey)
				{
					SourceAttributeInfo = SourceStorage.AttributeAllocationTable.Find(AttributeKey.Key);
				}
				else
				{
					SourceAttributeInfo = SourceStorage.AttributeAllocationTable.Find(AttributeKey);
				}

				if (SourceAttributeInfo)
				{
					bool bSkipAttribute = false;
					for (uint64 Offset : SourceAttributeInfo->Offsets)
					{
						if (!SourceStorage.AttributeStorage.IsValidIndex(Offset))
						{
							// EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
							bSkipAttribute = true;
							break;
						}
					}
					if (bSkipAttribute)
					{
						continue;
					}

					const EAttributeTypes SourceValueType = SourceAttributeInfo->Type;
					const TArray<uint64>& SourceValueSizes = SourceAttributeInfo->Sizes;

					// Search first for the original key
					FAttributeStorage::FAttributeAllocationInfo* DestinationAttributeInfo = nullptr;
					if constexpr (bIsPairAttributeKey)
					{
						DestinationAttributeInfo = DestinationStorage.AttributeAllocationTable.Find(AttributeKey.Key);

						// Search for the new key
						if (!DestinationAttributeInfo)
						{
							DestinationAttributeInfo = DestinationStorage.AttributeAllocationTable.Find(AttributeKey.Value);
						}
					}
					else
					{
						DestinationAttributeInfo = DestinationStorage.AttributeAllocationTable.Find(AttributeKey);
					}

					if (DestinationAttributeInfo)
					{
						if (DestinationAttributeInfo->Type != SourceValueType)
						{
							//EAttributeStorageResult::Operation_Error_WrongType;
							continue;
						}
						if (DestinationAttributeInfo->Sizes != SourceValueSizes)
						{
							//EAttributeStorageResult::Operation_Error_WrongSize;
							continue;
						}

						for (uint64 Offset : DestinationAttributeInfo->Offsets)
						{
							if (!DestinationStorage.AttributeStorage.IsValidIndex(Offset))
							{
								// EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
								bSkipAttribute = true;
								break;
							}
						}
						if (bSkipAttribute)
						{
							continue;
						}
					}
					else
					{
						if constexpr (bIsPairAttributeKey)
						{
							DestinationAttributeInfo = &DestinationStorage.AttributeAllocationTable.Add(AttributeKey.Value);
						}
						else
						{
							DestinationAttributeInfo = &DestinationStorage.AttributeAllocationTable.Add(AttributeKey);
						}
						DestinationAttributeInfo->Type = SourceValueType;
						DestinationAttributeInfo->Sizes = SourceValueSizes;

						DestinationAttributeInfo->Offsets.Reset(DestinationAttributeInfo->Sizes.Num());
						for (uint64 Size : DestinationAttributeInfo->Sizes)
						{
							DestinationAttributeInfo->Offsets.Add(DestinationStorage.AttributeStorage.AddZeroed(Size));
						}
					}

					//Force the specified attribute property
					DestinationAttributeInfo->Property = SourceAttributeInfo->Property;
					//Save the hash no need to compute it
					DestinationAttributeInfo->Hash = SourceAttributeInfo->Hash;

					//Memcpy source to destination storage
					for (int32 AllocIndex = 0; AllocIndex < SourceValueSizes.Num(); ++AllocIndex)
					{
						const uint8* SourceStorageData = SourceStorage.AttributeStorage.GetData();
						uint8* DestinationStorageData = DestinationStorage.AttributeStorage.GetData();
						FMemory::Memcpy(
							&DestinationStorageData[DestinationAttributeInfo->Offsets[AllocIndex]], 
							&SourceStorageData[SourceAttributeInfo->Offsets[AllocIndex]], 
							SourceValueSizes[AllocIndex]
						);
					}
				}
			}

			DestinationStorage.UpdateAllocationCount();
		}

		void FAttributeStorage::CopyStorageAttributes(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<TPair<FAttributeKey, FAttributeKey>>& AttributeKeys)
		{
			CopyStorageAttributesInternal(SourceStorage, DestinationStorage, AttributeKeys);
		}

		void FAttributeStorage::CopyStorageAttributes(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<FAttributeKey>& AttributeKeys)
		{
			CopyStorageAttributesInternal(SourceStorage, DestinationStorage, AttributeKeys);
		}

		void FAttributeStorage::SetDefragRatio(const float InDefragRatio)
		{
			FScopeLock ScopeLock(&StorageMutex);
			DefragRatio = InDefragRatio;
			DefragInternal();
		}

		void FAttributeStorage::Reserve(int64 NewAttributeCount, int64 NewStorageSize)
		{
			if (NewAttributeCount > 0)
			{
				const int64 ReserveCount = AttributeAllocationTable.Num() + NewAttributeCount;
				AttributeAllocationTable.Reserve(ReserveCount);
			}
			if (NewStorageSize > 0)
			{
				const int64 ReserveCount = AttributeStorage.Num() + NewStorageSize;
				AttributeStorage.Reserve(ReserveCount);
			}
		}

		template<typename T>
		EAttributeStorageResult FAttributeStorage::SetAttribute(const FAttributeKey& ElementAttributeKey, const T& Value)
		{
			FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			return SetAttribute(AttributeAllocationInfo, Value);
		}

		const FStringView FAttributeStorage::GetFStringViewAttributeFromStorage(const uint8* StorageData, const FAttributeAllocationInfo* AttributeAllocationInfo, int32 ElementIndex) const
		{
			if (!ensure(AttributeAllocationInfo->Sizes.IsValidIndex(ElementIndex)) || !ensure(AttributeAllocationInfo->Offsets.IsValidIndex(ElementIndex)))
			{
				return {};
			}

			const uint64 NumberOfChar = AttributeAllocationInfo->Sizes[ElementIndex] / sizeof(TCHAR);
			check(NumberOfChar > 0);

			// Null terminator is included in the size
			if (NumberOfChar <= 1)
			{
				return FStringView();
			}

			const TCHAR* StringContents = reinterpret_cast<const TCHAR*>(&StorageData[AttributeAllocationInfo->Offsets[ElementIndex]]);
			return FStringView(StringContents, int32(NumberOfChar - 1));
		}

		/**
		 * Set a multisize(TArray<uint8>, FString ...) attribute value into the storage at the allocation with TargetAllocationIndex.
		 * Return success if the attribute was properly set.
		 *
		 * WARNING: Expects the caller to lock the StorageMutex
		 */
		template<typename MultiSizeType>
		EAttributeStorageResult FAttributeStorage::MultiSizeSetAttribute(
			FAttributeAllocationInfo* AttributeAllocationInfo,
			int32 TargetAllocationIndex,
			const MultiSizeType& Value,
			const void* SourceDataPtr,
			bool& bOutNeedsDefrag
		)
		{
			const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<MultiSizeType>::GetType();
			const uint64 ValueSize = GetValueSizeInternal(Value);

			if (!AttributeAllocationInfo)
			{
				return EAttributeStorageResult::Operation_Error_CannotFoundKey;
			}

			//For all string-related types (listed below) we'll be setting their individual values with individual FStrings, whether
			//they're arrays or just a single value
			if constexpr (TIsStringType<MultiSizeType>)
			{
				//We should never be trying to set with actual FNames / FSoftObjectPaths in here, only with FStrings
				static_assert(std::is_same_v<MultiSizeType, FString>);

				const static TSet<EAttributeTypes> AllowedTypes = {
					EAttributeTypes::String,
					EAttributeTypes::StringArray,
					EAttributeTypes::Name,
					EAttributeTypes::NameArray,
					EAttributeTypes::SoftObjectPath,
					EAttributeTypes::SoftObjectPathArray,
					EAttributeTypes::InterchangeLODMeshData,
					EAttributeTypes::InterchangeLODMeshDataArray
				};
				if (!AllowedTypes.Contains(ValueType))
				{
					return EAttributeStorageResult::Operation_Error_WrongType;
				}
			}
			//InterchangeLODMeshData and InterchangeLODMeshDataArray will call this method on its properties in which case the ValueType (of the property) won't match the AttributeAllocationInfo->Type which is expected
			else if (AttributeAllocationInfo->Type != ValueType && !(AttributeAllocationInfo->Type == EAttributeTypes::InterchangeLODMeshData || AttributeAllocationInfo->Type == EAttributeTypes::InterchangeLODMeshDataArray))
			{
				return EAttributeStorageResult::Operation_Error_WrongType;
			}

			//We must have a valid data pointer if we have something to copy into the storage.
			//This case support empty array or empty string because the ValueSize will be 0.
			if (!SourceDataPtr && ValueSize > 0)
			{
				return EAttributeStorageResult::Operation_Error_InvalidMultiSizeValueData;
			}

			//Make sure we have the desired allocation index
			if (TargetAllocationIndex >= AttributeAllocationInfo->Sizes.Num())
			{
				AttributeAllocationInfo->Sizes.SetNumZeroed(TargetAllocationIndex + 1);
				AttributeAllocationInfo->Offsets.SetNumZeroed(TargetAllocationIndex + 1);
			}

			//If the new size is greater then the old size for the target allocation index, we have to create a new allocation entry and delete the old one
			if (ValueSize > AttributeAllocationInfo->Sizes[TargetAllocationIndex])
			{
				FragmentedMemoryCost += AttributeAllocationInfo->Sizes[TargetAllocationIndex];
				bOutNeedsDefrag = true;

				AttributeAllocationInfo->Sizes[TargetAllocationIndex] = ValueSize;
				AttributeAllocationInfo->Offsets[TargetAllocationIndex] = AttributeStorage.AddZeroed(ValueSize);
			}
			else
			{
				//In case we reuse the allocation table, simply adjust the waste memory counter
				FragmentedMemoryCost += AttributeAllocationInfo->Sizes[TargetAllocationIndex] - ValueSize;
				bOutNeedsDefrag = true;

				AttributeAllocationInfo->Sizes[TargetAllocationIndex] = ValueSize;
			}

			if (ValueSize > 0)
			{
				uint8* StorageData = AttributeStorage.GetData();
				FMemory::Memcpy(
					&StorageData[AttributeAllocationInfo->Offsets[TargetAllocationIndex]],
					SourceDataPtr,
					ValueSize
				);
			}

			return EAttributeStorageResult::Operation_Success;
		}

		template<typename ArrayType>
		EAttributeStorageResult FAttributeStorage::GenericArrayGetAttribute(const FAttributeKey& ElementAttributeKey, ArrayType& OutValue) const
		{
			//Lock the storage
			FScopeLock ScopeLock(&StorageMutex);

			static_assert(TIsTArray<ArrayType>::Value);

			const FAttributeAllocationInfo* AttributeAllocationInfo = AttributeAllocationTable.Find(ElementAttributeKey);
			if (AttributeAllocationInfo)
			{
				if (AttributeAllocationInfo->Type != TAttributeTypeTraitsInternal<ArrayType>::GetType())
				{
					return EAttributeStorageResult::Operation_Error_WrongType;
				}
				if (AttributeAllocationInfo->Offsets.Num() != AttributeAllocationInfo->Sizes.Num())
				{
					// For now we expect all generic array types (i.e. non-string array types) to have a single allocation
					return EAttributeStorageResult::Operation_Error_WrongSize;
				}
				for (uint64 Offset : AttributeAllocationInfo->Offsets)
				{
					if (!AttributeStorage.IsValidIndex(Offset))
					{
						return EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
					}
				}
			}
			else
			{
				//The key does not exist
				return EAttributeStorageResult::Operation_Error_CannotFoundKey;
			}

			constexpr bool bOneAllocationPerElement = TIsArrayOfStringType<ArrayType>;
			if constexpr (bOneAllocationPerElement)
			{
				const uint64 NumAllocations = AttributeAllocationInfo->Sizes.Num();
				const uint64 NumElements = NumAllocations;

				const uint8* StorageData = AttributeStorage.GetData();

				OutValue.SetNum(NumElements);
				for (int32 AllocIndex = 0; AllocIndex < NumAllocations; ++AllocIndex)
				{
					if constexpr (TIsStringType<typename ArrayType::ElementType>)
					{
						const FStringView ValueView = GetFStringViewAttributeFromStorage(StorageData, AttributeAllocationInfo, AllocIndex);
						OutValue[AllocIndex] = typename ArrayType::ElementType{ValueView};
					}
					else
					{
						// This should never happen for now as we only have "nested arrays" of string types
						ensureMsgf(false, TEXT("Actual nested TArrays are not supported!"));
					}
				}
			}
			else if constexpr (std::is_same_v<ArrayType, TArray<FInterchangeLODMeshData>>)
			{
				const uint64 NumAllocations = AttributeAllocationInfo->Sizes.Num();

				const constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
				static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");
				ensure(NumAllocations % NumberOfAttributes == 0);

				const uint64 NumElements = NumAllocations / NumberOfAttributes;
				OutValue.SetNum(NumElements);

				const uint8* StorageData = AttributeStorage.GetData();

				for (int32 ElementIndex = 0; ElementIndex < NumElements; ++ElementIndex)
				{
					//FString
					{
						const FStringView ValueView = GetFStringViewAttributeFromStorage(StorageData, AttributeAllocationInfo, ElementIndex * NumberOfAttributes);
						OutValue[ElementIndex].MeshUid = ValueView;
					}

					//FString
					{
						const FStringView ValueView = GetFStringViewAttributeFromStorage(StorageData, AttributeAllocationInfo, ElementIndex * NumberOfAttributes + 1);
						OutValue[ElementIndex].SceneNodeName = ValueView;
					}

					//FTransform
					{
						const int32 AllocationIndex = (ElementIndex * NumberOfAttributes) + 2;
						const uint8* SourceBytes = &StorageData[AttributeAllocationInfo->Offsets[AllocationIndex]];
						const uint64 AllocationSize = AttributeAllocationInfo->Sizes[AllocationIndex];
						FMemory::Memcpy(&OutValue[ElementIndex].Transform, SourceBytes, AllocationSize);
					}

					//FTransform
					{
						const int32 AllocationIndex = (ElementIndex * NumberOfAttributes) + 3;
						const uint8* SourceBytes = &StorageData[AttributeAllocationInfo->Offsets[AllocationIndex]];
						const uint64 AllocationSize = AttributeAllocationInfo->Sizes[AllocationIndex];
						FMemory::Memcpy(&OutValue[ElementIndex].GeometricTransform, SourceBytes, AllocationSize);
					}
				}
			}
			else // One allocation for the whole array (so most types, like TArray<float>, TArray<FTransform3f>, etc.)
			{
				const uint64 NumAllocations = AttributeAllocationInfo->Sizes.Num();
				ensure(NumAllocations <= 1);

				if (NumAllocations == 0 || AttributeAllocationInfo->Sizes[0] == 0)
				{
					OutValue.Empty();
					return EAttributeStorageResult::Operation_Success;
				}

				const uint64 AllocationSize = AttributeAllocationInfo->Sizes[0];
				const uint64 NumElements = AllocationSize / ArrayType::GetTypeSize();
				OutValue.SetNumUninitialized(NumElements);

				const uint8* StorageData = AttributeStorage.GetData();
				const uint8* SourceBytes = &StorageData[AttributeAllocationInfo->Offsets[0]];
				FMemory::Memcpy(OutValue.GetData(), SourceBytes, AllocationSize);
			}

			return EAttributeStorageResult::Operation_Success;
		}

		// WARNING: This may invalidate references to elements of the AttributeAllocationTable, as it will compact its pairs
		void FAttributeStorage::DefragInternal()
		{
			//Defrag if the fragmented memory cost is bigger then the DefragRatio.
			//TODO: Why 11? Maybe use a cvar?
			if (FragmentedMemoryCost < 11 || FragmentedMemoryCost < AttributeStorage.Num() * DefragRatio)
			{
				return;
			}

			//Linearize our allocations, since FAttributeAllocationInfo may now hold multiple allocations each, that could
			//all be interleaved with other attributes'
			struct FLinearAllocation
			{
				FAttributeKey Key;
				uint32 Index;
				uint64 Offset;
				uint64 Size;
			};
			TArray<FLinearAllocation> LinearAllocations;
			LinearAllocations.Reserve(AttributeAllocationTable.Num());
			for (const TPair<FAttributeKey, FAttributeAllocationInfo>& Pair : AttributeAllocationTable)
			{
				const FAttributeKey& AllocKey = Pair.Key;
				const FAttributeAllocationInfo& AllocInfo = Pair.Value;

				int32 NumAllocs = AllocInfo.Sizes.Num();
				if (!ensure(NumAllocs == AllocInfo.Offsets.Num()))
				{
					return;
				}

				for (int32 AllocIndex = 0; AllocIndex < NumAllocs; ++AllocIndex)
				{
					FLinearAllocation& LinearAlloc = LinearAllocations.Emplace_GetRef();
					LinearAlloc.Key = AllocKey;
					LinearAlloc.Index = AllocIndex;
					LinearAlloc.Offset = AllocInfo.Offsets[AllocIndex];
					LinearAlloc.Size = AllocInfo.Sizes[AllocIndex];
				}
			}

			//Sorts the allocation table per offset since we want to defrag using memory block
			LinearAllocations.Sort([](const FLinearAllocation& A, const FLinearAllocation& B)
			{
				if (A.Offset == B.Offset) 
				{
					return A.Size < B.Size;
				}

				return A.Offset < B.Offset;
			});

			uint8* StorageData = AttributeStorage.GetData();
			//Number of allocations we have handled
			uint64 HandledAllocations = 0;
			//Current storage offset we want to defrag
			uint64 CurrentOffset = 0;
			//Are we build a defrag block
			bool bBuildBlock = false;
			//Defrag block start
			uint64 MemmoveBlockStart = 0;
			//Replace CurrentOffset when we are moving a block
			uint64 MemmoveBlockCurrentOffset = 0;
			const uint64 NumAllocations = LinearAllocations.Num();
			for (FLinearAllocation& LinearAllocation : LinearAllocations)
			{
				HandledAllocations++;
				const bool bLast = HandledAllocations == NumAllocations;

				if (LinearAllocation.Offset == CurrentOffset)
				{
					CurrentOffset += LinearAllocation.Size;
					continue;
				}
				check(CurrentOffset < LinearAllocation.Offset);
				if (!bBuildBlock)
				{
					bBuildBlock = true;
					MemmoveBlockStart = LinearAllocation.Offset;
					LinearAllocation.Offset = CurrentOffset;
					MemmoveBlockCurrentOffset = MemmoveBlockStart + LinearAllocation.Size;
				}
				else
				{
					if (LinearAllocation.Offset == MemmoveBlockCurrentOffset)
					{
						LinearAllocation.Offset -= MemmoveBlockStart - CurrentOffset;
						MemmoveBlockCurrentOffset += LinearAllocation.Size;
					}
					else
					{
						const uint64 BlockSize = MemmoveBlockCurrentOffset - MemmoveBlockStart;
						//Memmove support overlap
						FMemory::Memmove(&StorageData[CurrentOffset], &StorageData[MemmoveBlockStart], BlockSize);
						CurrentOffset += BlockSize;

						MemmoveBlockStart = LinearAllocation.Offset;
						LinearAllocation.Offset = CurrentOffset;
						MemmoveBlockCurrentOffset = MemmoveBlockStart + LinearAllocation.Size;
					}
				}

				//We have to move the block if we are the last attribute
				if (bLast)
				{
					const uint64 BlockSize = MemmoveBlockCurrentOffset - MemmoveBlockStart;
					//Memmove support overlap
					FMemory::Memmove(&StorageData[CurrentOffset], &StorageData[MemmoveBlockStart], BlockSize);
					const uint64 RemoveStartIndex = CurrentOffset + BlockSize;
					const uint64 RemoveCount = AttributeStorage.Num() - RemoveStartIndex;
					//Remove the Last items, allow shrinking so we are compact
					AttributeStorage.RemoveAt(RemoveStartIndex, RemoveCount);
				}
			}

			//Update our allocation count since it's basically free to do right now
			AllocationCount = LinearAllocations.Num();

			//Reset the fragmented cost
			FragmentedMemoryCost = 0;

			//Remove holes in the TMap
			AttributeAllocationTable.Compact();

			//Update our allocation table with the defragged offsets / sizes
			for (const FLinearAllocation& Alloc : LinearAllocations)
			{
				FAttributeAllocationInfo* FoundInfo = AttributeAllocationTable.Find(Alloc.Key);
				if (ensure(FoundInfo))
				{
					if (ensure(FoundInfo->Sizes.IsValidIndex(Alloc.Index) && FoundInfo->Offsets.IsValidIndex(Alloc.Index)))
					{
						FoundInfo->Sizes[Alloc.Index] = Alloc.Size;
						FoundInfo->Offsets[Alloc.Index] = Alloc.Offset;
					}
				}
			}
		}

		template<typename T>
		EAttributeStorageResult FAttributeStorage::SetAttribute(FAttributeAllocationInfo* AttributeAllocationInfo, const T& Value)
		{
			static_assert(TAttributeTypeTraitsInternal<T>::GetType() != EAttributeTypes::None, "T is not a supported type for the attributes. Check EAttributeTypes for the supported types");
			static_assert(!TIsEnum<T>::Value && !TIsTEnumAsByte<T>::Value, "This function should only be instantiated for enum underlying types.");

			if (!AttributeAllocationInfo)
			{
				return EAttributeStorageResult::Operation_Error_CannotFoundKey;
			}

			FScopeLock ScopeLock(&StorageMutex);

			bool bNeedsDefrag = false;

			EAttributeStorageResult Result = EAttributeStorageResult::Operation_Success;
			if constexpr (TIsArrayOfStringType<T>)
			{
				AllocationCount -= AttributeAllocationInfo->Sizes.Num();
				AllocationCount += Value.Num();

				// We only need as many allocations as Value has elements: Discard the extra ones
				for (int32 Index = AttributeAllocationInfo->Sizes.Num() - 1; Index >= Value.Num(); --Index)
				{
					FragmentedMemoryCost += AttributeAllocationInfo->Sizes[Index];
					bNeedsDefrag = true;
				}
				AttributeAllocationInfo->Sizes.SetNum(Value.Num());
				AttributeAllocationInfo->Offsets.SetNum(Value.Num());

				// Set one allocation for each Value element
				for (int32 Index = 0; Index < Value.Num(); ++Index)
				{
					if constexpr (std::is_same_v<typename T::ElementType, FString>)
					{
						const FString& Element = Value[Index];
						Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index, Element, (uint8*)(*Element), bNeedsDefrag);
					}
					else
					{
						const FString& Element = Value[Index].ToString();
						Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index, Element, (uint8*)(*Element), bNeedsDefrag);
					}

					if (Result != EAttributeStorageResult::Operation_Success)
					{
						break;
					}
				}
			}
			else if constexpr (TIsTArray<T>::Value)
			{
				if constexpr (std::is_same_v<T, TArray<FInterchangeLODMeshData>>)
				{
					const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();

					if (AttributeAllocationInfo->Type != ValueType)
					{
						Result = EAttributeStorageResult::Operation_Error_WrongType;
					}
					if (Result == EAttributeStorageResult::Operation_Success)
					{
						constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
						static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");

						const int32 NumOfAllocations = Value.Num() * NumberOfAttributes;
						AllocationCount -= AttributeAllocationInfo->Sizes.Num();
						AllocationCount += NumOfAllocations;

						// We only need as many allocations as Value has elements: Discard the extra ones
						for (int32 Index = AttributeAllocationInfo->Sizes.Num() - 1; Index >= NumOfAllocations; --Index)
						{
							FragmentedMemoryCost += AttributeAllocationInfo->Sizes[Index];
							bNeedsDefrag = true;
						}
						AttributeAllocationInfo->Sizes.SetNum(NumOfAllocations);
						AttributeAllocationInfo->Offsets.SetNum(NumOfAllocations);

						int32 Index = 0;
						for (const FInterchangeLODMeshData& InterchangeLODMesh : Value)
						{
							//FString
							Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index++, InterchangeLODMesh.MeshUid, (uint8*)(*InterchangeLODMesh.MeshUid), bNeedsDefrag);

							//FString
							if (Result == EAttributeStorageResult::Operation_Success)
							{
								Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index++, InterchangeLODMesh.SceneNodeName, (uint8*)(*InterchangeLODMesh.SceneNodeName), bNeedsDefrag);
							}

							//FTransform:
							if (Result == EAttributeStorageResult::Operation_Success)
							{
								Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index++, InterchangeLODMesh.Transform, &InterchangeLODMesh.Transform, bNeedsDefrag);
							}

							//FTransform:
							if (Result == EAttributeStorageResult::Operation_Success)
							{
								Result = MultiSizeSetAttribute(AttributeAllocationInfo, Index++, InterchangeLODMesh.GeometricTransform, &InterchangeLODMesh.GeometricTransform, bNeedsDefrag);
							}
						}
					}
				}
				else
				{
					const int32 TargetAllocationIndex = 0;
					Result = MultiSizeSetAttribute(AttributeAllocationInfo, TargetAllocationIndex, Value, (uint8*)(Value.GetData()), bNeedsDefrag);
				}
			}
			else if constexpr (std::is_same_v<T, FString>)
			{
				const int32 TargetAllocationIndex = 0;
				Result = MultiSizeSetAttribute(AttributeAllocationInfo, TargetAllocationIndex, Value, (uint8*)(*Value), bNeedsDefrag);
			}
			else if constexpr (std::is_same_v<T, FName> || std::is_same_v<T, FSoftObjectPath>)
			{
				//Value must be stored as a FString for persistence
				const int32 TargetAllocationIndex = 0;
				FString ValueStr = Value.ToString();
				Result = MultiSizeSetAttribute(AttributeAllocationInfo, TargetAllocationIndex, ValueStr, (uint8*)(*ValueStr), bNeedsDefrag);
			}
			else if constexpr (std::is_same_v<T, FInterchangeLODMeshData>)
			{
				const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();

				if (AttributeAllocationInfo->Type != ValueType)
				{
					Result = EAttributeStorageResult::Operation_Error_WrongType;
				}
				if (Result == EAttributeStorageResult::Operation_Success)
				{
					const constexpr int32 NumberOfAttributes = FInterchangeLODMeshData::NumberOfAttributes();
					static_assert(NumberOfAttributes == 4, "Unexpected number of attributes on FInterchangeLODMeshData. Please update attribute parsing in scope.");
					
					AllocationCount -= AttributeAllocationInfo->Sizes.Num();
					AllocationCount += NumberOfAttributes;

					// We only need as many allocations as Value has elements: Discard the extra ones
					for (int32 Index = AttributeAllocationInfo->Sizes.Num() - 1; Index >= NumberOfAttributes; --Index)
					{
						FragmentedMemoryCost += AttributeAllocationInfo->Sizes[Index];
						bNeedsDefrag = true;
					}
					AttributeAllocationInfo->Sizes.SetNum(NumberOfAttributes);
					AttributeAllocationInfo->Offsets.SetNum(NumberOfAttributes);

					//FString
					Result = MultiSizeSetAttribute(AttributeAllocationInfo, 0, Value.MeshUid, (uint8*)(*Value.MeshUid), bNeedsDefrag);
					if (Result == EAttributeStorageResult::Operation_Success)
					{
						Result = MultiSizeSetAttribute(AttributeAllocationInfo, 1, Value.SceneNodeName, (uint8*)(*Value.SceneNodeName), bNeedsDefrag);
					}

					//FTransform:
					if (Result == EAttributeStorageResult::Operation_Success)
					{
						Result = MultiSizeSetAttribute(AttributeAllocationInfo, 2, Value.Transform, &Value.Transform, bNeedsDefrag);
					}

					//FTransform:
					if (Result == EAttributeStorageResult::Operation_Success)
					{
						Result = MultiSizeSetAttribute(AttributeAllocationInfo, 3, Value.GeometricTransform, &Value.GeometricTransform, bNeedsDefrag);
					}
				}
			}
			else //Other non-string elements (float, FGuid, FTransform3f, etc.)
			{
				const EAttributeTypes ValueType = TAttributeTypeTraitsInternal<T>::GetType();
				const uint64 ValueSize = GetValueSizeInternal(Value);

				if (AttributeAllocationInfo->Type != ValueType)
				{
					Result = EAttributeStorageResult::Operation_Error_WrongType;
				}
				if (!ensure(AttributeAllocationInfo->Sizes.Num() == 1) || AttributeAllocationInfo->Sizes[0] != ValueSize)
				{
					Result = EAttributeStorageResult::Operation_Error_WrongSize;
				}
				if (!ensure(AttributeAllocationInfo->Offsets.Num() == 1) || !AttributeStorage.IsValidIndex(AttributeAllocationInfo->Offsets[0]))
				{
					Result = EAttributeStorageResult::Operation_Error_AttributeAllocationCorrupted;
				}

				if (Result == EAttributeStorageResult::Operation_Success)
				{
					uint8* StorageData = AttributeStorage.GetData();
					FMemory::Memcpy(&StorageData[AttributeAllocationInfo->Offsets[0]], &Value, ValueSize);
				}
			}

			UpdateAllocationInfoHash(*AttributeAllocationInfo);

			// We can only defrag after we're done since it may potentially invalidate our AttributeAllocationInfo pointer
			// as it will internally compact the allocation table
			if (bNeedsDefrag)
			{
				DefragInternal();
			}

			return Result;
		}

		void FAttributeStorage::UpdateAllocationInfoHash(FAttributeStorage::FAttributeAllocationInfo& AllocationInfo)
		{
			ensure(AllocationInfo.Offsets.Num() == AllocationInfo.Sizes.Num());

			const uint8* StorageData = AttributeStorage.GetData();

			FSHA1 Sha;
			for (int32 AllocIndex = 0; AllocIndex < AllocationInfo.Sizes.Num(); ++AllocIndex)
			{
				Sha.Update(
					&StorageData[AllocationInfo.Offsets[AllocIndex]], 
					AllocationInfo.Sizes[AllocIndex]
				);
			}
			Sha.Final();

			// Retrieve the hash and use it to construct a pseudo-GUID.
			uint32 Hash[5];
			Sha.GetHash(reinterpret_cast<uint8*>(Hash));
			AllocationInfo.Hash = FGuid(Hash[0] ^ Hash[4], Hash[1], Hash[2], Hash[3]);
		}

		void FAttributeStorage::UpdateAllocationCount()
		{
			AllocationCount = 0;
			for (const TPair<FAttributeKey, FAttributeAllocationInfo>& Pair : AttributeAllocationTable)
			{
				AllocationCount += Pair.Value.Sizes.Num();
			}
		}

#define SPECIALIZE(X) \
			template INTERCHANGECORE_API EAttributeStorageResult FAttributeStorage::RegisterAttribute<X>(const FAttributeKey&, const X&, EAttributeProperty);\
			template INTERCHANGECORE_API EAttributeStorageResult FAttributeStorage::GetAttribute<X>(const FAttributeKey&, X&) const;\
			template INTERCHANGECORE_API FAttributeStorage::TAttributeHandle<X>::TAttributeHandle();\
			template INTERCHANGECORE_API FAttributeStorage::TAttributeHandle<X>::TAttributeHandle(const FAttributeKey& InKey, const FAttributeStorage* InAttributeStorage);\
			template INTERCHANGECORE_API bool FAttributeStorage::TAttributeHandle<X>::IsValid() const;\
			template INTERCHANGECORE_API EAttributeStorageResult FAttributeStorage::TAttributeHandle<X>::Get(X& Value) const;\
			template INTERCHANGECORE_API EAttributeStorageResult FAttributeStorage::TAttributeHandle<X>::Set(const X& Value);\
			template INTERCHANGECORE_API const FAttributeKey& FAttributeStorage::TAttributeHandle<X>::GetKey() const;\
			template INTERCHANGECORE_API FAttributeStorage::TAttributeHandle<X> FAttributeStorage::GetAttributeHandle<X>(const FAttributeKey&) const;\
			template INTERCHANGECORE_API FString UE::Interchange::AttributeValueToString(const X&);

		SPECIALIZE(bool);
		SPECIALIZE(FColor);
		SPECIALIZE(FDateTime);
		SPECIALIZE(double);
		SPECIALIZE(float);
		SPECIALIZE(FGuid);
		SPECIALIZE(int8);
		SPECIALIZE(int16);
		SPECIALIZE(int32);
		SPECIALIZE(int64);
		SPECIALIZE(FIntRect);
		SPECIALIZE(FLinearColor);
		SPECIALIZE(FName);
		SPECIALIZE(FRandomStream);
		SPECIALIZE(FTimespan);
		SPECIALIZE(FString);
		SPECIALIZE(FTwoVectors);
		SPECIALIZE(uint8);
		SPECIALIZE(uint16);
		SPECIALIZE(uint32);
		SPECIALIZE(uint64);
		SPECIALIZE(FVector2D);
		SPECIALIZE(FIntPoint);
		SPECIALIZE(FIntVector);
		SPECIALIZE(FVector2DHalf);
		SPECIALIZE(FFloat16);
		SPECIALIZE(FOrientedBox);
		SPECIALIZE(FFrameNumber);
		SPECIALIZE(FFrameRate);
		SPECIALIZE(FFrameTime);
		SPECIALIZE(FSoftObjectPath);
		SPECIALIZE(FMatrix44f);
		SPECIALIZE(FMatrix44d);
		SPECIALIZE(FPlane4f);
		SPECIALIZE(FPlane4d);
		SPECIALIZE(FQuat4f);
		SPECIALIZE(FQuat4d);
		SPECIALIZE(FRotator3f);
		SPECIALIZE(FRotator3d);
		SPECIALIZE(FTransform3f);
		SPECIALIZE(FTransform3d);
		SPECIALIZE(FVector3f);
		SPECIALIZE(FVector3d);
		SPECIALIZE(FVector2f);
		SPECIALIZE(FVector4f);
		SPECIALIZE(FVector4d);
		SPECIALIZE(FBox2f);
		SPECIALIZE(FBox2D);
		SPECIALIZE(FBox3f);
		SPECIALIZE(FBox3d);
		SPECIALIZE(FBoxSphereBounds3f);
		SPECIALIZE(FBoxSphereBounds3d);
		SPECIALIZE(FSphere3f);
		SPECIALIZE(FSphere3d);

		SPECIALIZE(TArray<bool>);
		SPECIALIZE(TArray<FColor>);
		SPECIALIZE(TArray<FDateTime>);
		SPECIALIZE(TArray<double>);
		SPECIALIZE(TArray<float>);
		SPECIALIZE(TArray<FGuid>);
		SPECIALIZE(TArray<int8>);
		SPECIALIZE(TArray<int16>);
		SPECIALIZE(TArray<int32>);
		SPECIALIZE(TArray<int64>);
		SPECIALIZE(TArray<FIntRect>);
		SPECIALIZE(TArray<FLinearColor>);
		SPECIALIZE(TArray<FName>);
		SPECIALIZE(TArray<FRandomStream>);
		SPECIALIZE(TArray<FTimespan>);
		SPECIALIZE(TArray<FString>);
		SPECIALIZE(TArray<FTwoVectors>);
		SPECIALIZE(TArray<uint8>);
		SPECIALIZE(TArray64<uint8>);
		SPECIALIZE(TArray<uint16>);
		SPECIALIZE(TArray<uint32>);
		SPECIALIZE(TArray<uint64>);
		SPECIALIZE(TArray<FVector2D>);
		SPECIALIZE(TArray<FIntPoint>);
		SPECIALIZE(TArray<FIntVector>);
		SPECIALIZE(TArray<FVector2DHalf>);
		SPECIALIZE(TArray<FFloat16>);
		SPECIALIZE(TArray<FOrientedBox>);
		SPECIALIZE(TArray<FFrameNumber>);
		SPECIALIZE(TArray<FFrameRate>);
		SPECIALIZE(TArray<FFrameTime>);
		SPECIALIZE(TArray<FSoftObjectPath>);
		SPECIALIZE(TArray<FMatrix44f>);
		SPECIALIZE(TArray<FMatrix44d>);
		SPECIALIZE(TArray<FPlane4f>);
		SPECIALIZE(TArray<FPlane4d>);
		SPECIALIZE(TArray<FQuat4f>);
		SPECIALIZE(TArray<FQuat4d>);
		SPECIALIZE(TArray<FRotator3f>);
		SPECIALIZE(TArray<FRotator3d>);
		SPECIALIZE(TArray<FTransform3f>);
		SPECIALIZE(TArray<FTransform3d>);
		SPECIALIZE(TArray<FVector3f>);
		SPECIALIZE(TArray<FVector3d>);
		SPECIALIZE(TArray<FVector2f>);
		SPECIALIZE(TArray<FVector4f>);
		SPECIALIZE(TArray<FVector4d>);
		SPECIALIZE(TArray<FBox2f>);
		SPECIALIZE(TArray<FBox2D>);
		SPECIALIZE(TArray<FBox3f>);
		SPECIALIZE(TArray<FBox3d>);
		SPECIALIZE(TArray<FBoxSphereBounds3f>);
		SPECIALIZE(TArray<FBoxSphereBounds3d>);
		SPECIALIZE(TArray<FSphere3f>);
		SPECIALIZE(TArray<FSphere3d>);

		SPECIALIZE(FInterchangeLODMeshData);
		SPECIALIZE(TArray<FInterchangeLODMeshData>);

		#undef SPECIALIZE
	} //ns Interchange
} //ns UE