// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

// Custom serialization version for changes to variant manager objects
struct FInterchangeCustomVersion
{
	enum Type
	{
		// Roughly corresponds to 5.2
		BeforeCustomVersionWasAdded = 0,

		SerializedInterchangeObjectStoring,

		MultipleAllocationsPerAttributeInStorage,

		// The change that implemented the previous version had to be backed out to fix a serialization issue
		MultipleAllocationsPerAttributeInStorageFixed,

		// Replaced 'DoNotSearch' Material's search location enum value with a boolean property
		AdditionOfReuseExistingMaterialsBool,

		// Moved mesh conversion setting from UInterchangeGenericCommonSkeletalMeshesAndAnimationsProperties to UInterchangeGenericCommonMeshesProperties
		SkeletalAndStaticMeshOptionsCleanUp,

		// -----<new versions can be added above this line>-------------------------------------------------
		VersionPlusOne,
		LatestVersion = VersionPlusOne - 1
	};

	// The GUID for this custom version number
	INTERCHANGECORE_API const static FGuid GUID;

private:
	FInterchangeCustomVersion() {}
};