// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeTaskSystem.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"

#include "InterchangeFactoryBase.generated.h"

class ULevel;
class UInterchangeBaseNodeContainer;
class UInterchangePipelineBase;
class UInterchangeSourceData;
class UInterchangeTranslatorBase;

UENUM(BlueprintType)
enum class EInterchangeFactoryAssetType : uint8
{
	None = 0,
	Textures,
	Materials,
	Meshes,
	Animations,
	Physics,
	Grooms,
	Sounds,
	Levels,
	Variants,
	Custom
};

/**
 * Base class for post import task.
 * Post import tasks are execute by the InterchangeManager when there is no import task to execute.
 * See more detail in the Interchange manager header.
 */
class FInterchangePostImportTask
{
public:
	virtual void Execute() {};
};

/**
 * Asset factory implementation:
 * 
 * The first three steps use the Interchange factory node to import or reimport the UObject:
 * 
 * 1. BeginImportAsset_GameThread - Create the asset UObject. You can also import source data (retrieve payloads) and set up properties on the game thread.
 * 2. ImportAsset_Async - Import source data (retrieve payloads) and set up properties asynchronously on any thread.
 * 3. EndImportAsset_GameThread - Anything you need to do on the game thread to finalize the import source data and set up properties. For example, conflict resolution that needs UI.
 * 
 * The last three steps can modify the created UObject
 * 
 * 4. SetupObject_GameThread - Do any UObject setup required before the build, the UObject dependencies should exist and have all the source data and properties imported.
 * 5. BuildObject_GameThread - Build the asset if it can be built.
 * 6. FinalizeObject_GameThread - Do any final UObject setup after the build. Note that the build of an asset can be asynchronous and this function will be call after the async build is done.
 * 
 * Scene factory implementation:
 * 
 * 1. ImportSceneObject_GameThread - Create an actor in a level.
 */

UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UInterchangeFactoryBase : public UObject
{
	GENERATED_BODY()
public:

	/**
	 * Return the UClass this factory can create.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Factory")
	virtual UClass* GetFactoryClass() const
	{
		return nullptr;
	}

	/**
	 * Return the asset type this factory can create.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Factory")
	virtual EInterchangeFactoryAssetType GetFactoryAssetType() { return EInterchangeFactoryAssetType::None; }

	/**
	 * Parameters to pass to the CreateAsset function.
	 */
	struct FImportAssetObjectParams
	{
		/** The package where the asset should be created. If null, it will be put in the transient package. */
		UObject* Parent = nullptr;

		/** The name to give to the created asset. */
		FString AssetName = FString();

		/** The base node that describes how to create the asset. */
		UInterchangeFactoryBaseNode* AssetNode = nullptr;

		/** The translator is used to retrieve the payLoad data in case the factory needs it. */
		const UInterchangeTranslatorBase* Translator = nullptr;

		/** The source data. TODO: we have to refactor UAssetImportData, the source data should be the base class for this now */
		const UInterchangeSourceData* SourceData = nullptr;

		/** The node container associated with the current source index. */
		const UInterchangeBaseNodeContainer* NodeContainer = nullptr;

		/**
		 * If when we try to create the package we find out the asset already exists, this field will contain
		 * the asset we want to reimport. The reimport should just change the source data and not any asset settings.
		 */
		UObject* ReimportObject = nullptr;
	};

	struct FImportAssetResult
	{
		//If the factory sets this to true, the Interchange task import object should skip this asset.
		//An asset can be skipped if it already exists and the factory isn't doing a reimport. For example, if we import a static mesh and we find an existing material.
		//We don't want to override the material in this case because the UE material often contains content related to gameplay.
		bool bIsFactorySkipAsset = false;
		//Return the UObject imported or reimported by the factory, or leave it set to nullptr if there was an error.
		UObject* ImportedObject = nullptr;
	};

	UE_DEPRECATED(5.3, "This function is replaced by BeginImportAsset_GameThread.")
	virtual UObject* ImportAssetObject_GameThread(const FImportAssetObjectParams& Arguments)
	{
		return nullptr;
	}

	virtual void CreatePayloadTasks(const FImportAssetObjectParams& Arguments, bool bAsync, TArray<TSharedPtr<UE::Interchange::FInterchangeTaskBase>>& PayloadTasks)
	{
		return;
	}

	/**
	 * Override this function to import/reimport source data and configure the properties synchronously.
	 * Create the asset package on the game thread because it's not thread-safe.
	 *
	 * @param Arguments - The structure that contains all necessary arguments to import the UObject and the package. See the structure definition for its documentation.
	 * @return the FImportAssetResult. See the structure to access its documentation.
	 *
	 * @Note Mandatory to override this function to create the asset UObject package. Not mandatory for level actor; use CreateSceneObject instead.
	 */
	virtual FImportAssetResult BeginImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
	{
		FImportAssetResult ImportAssetResult;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ImportAssetResult.ImportedObject = ImportAssetObject_GameThread(Arguments);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return ImportAssetResult;
	}

	UE_DEPRECATED(5.3, "This function is replaced by ImportAsset_Async.")
	virtual UObject* ImportAssetObject_Async(const FImportAssetObjectParams& Arguments)
	{
		//By default simply return the UObject created by BeginImportAsset_GameThread that was store in the asset node.
		FSoftObjectPath ReferenceObject;
		if (Arguments.AssetNode && Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
		{
			return ReferenceObject.TryLoad();
		}
		return nullptr;
	}

	/**
	 * Override this function to import/reimport the UObject source data and configure the properties asynchronously. Do not create the asset UObject asynchronously.
	 * Helpful to get all the payloads in parallel or do any long import tasks.
	 *
	 * @param Arguments - The structure that contains all necessary data to import the UObject. See the structure definition for its documentation.
	 * @return the FImportAssetResult. See the structure to access its documentation.
	 */
	virtual FImportAssetResult ImportAsset_Async(const FImportAssetObjectParams& Arguments)
	{
		FImportAssetResult ImportAssetResult;
		PRAGMA_DISABLE_DEPRECATION_WARNINGS
		ImportAssetResult.ImportedObject = ImportAssetObject_Async(Arguments);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS
		return ImportAssetResult;
	}

	/**
	 * Override this function to end import/reimport on the game thread. Helpful if you need to display UI (for example, reimport material conflict resolution) or if you
	 * need to do anything not thread-safe to complete the import/reimport.
	 *
	 * @param Arguments - The structure that contains all necessary data to import the UObject. See the structure definition for its documentation.
	 * @return the FImportAssetResult. See the structure to access its documentation.
	 */
	virtual FImportAssetResult EndImportAsset_GameThread(const FImportAssetObjectParams& Arguments)
	{
		FImportAssetResult ImportAssetResult;
		//By default simply return the UObject created by BeginImportAssetObject_GameThread that was store in the asset node.
		FSoftObjectPath ReferenceObject;
		if (Arguments.AssetNode && Arguments.AssetNode->GetCustomReferenceObject(ReferenceObject))
		{
			ImportAssetResult.ImportedObject = ReferenceObject.TryLoad();
		}
		return ImportAssetResult;
	}

	/**
	 * Parameters to pass to the SpawnActor function.
	 */
	struct FImportSceneObjectsParams
	{
		/** The level in which to create the scene objects. */
		ULevel* Level = nullptr;

		/** The name to give to the created actor. */
		FString ObjectName;

		/** The base node that describes how to create the asset. */
		UInterchangeFactoryBaseNode* FactoryNode = nullptr;

		/** The node container associated with the current source index. */
		const UInterchangeBaseNodeContainer* NodeContainer = nullptr;

		/** The source data. TODO: we have to refactor UAssetImportData, the source data should be the base class for this now */
		const UInterchangeSourceData* SourceData = nullptr;

		/**
		 * If not null, the factory must perform a reimport of the scene node.
		 */
		UObject* ReimportObject = nullptr;

		/**
		 * Factory base node associated with the reimported scene node.
		 */
		const UInterchangeFactoryBaseNode* ReimportFactoryNode = nullptr;

		/**
		 * true if we are re-importing
		 */
		bool bIsReimport = false;

		/*
		 * This represent the async helper unique id in case a factory need to retrieve the asynchelper associate to the import task
		 * 
		 * TODO create a base class for FAsyncHelper in Interchange/Core so we can reference it here instead of using the unique identifier.
		 * The asynchelper is currently declare in Interchange/Engine so we can't refer to it here.
		 */
		int32 AsyncHelperUid = INDEX_NONE;
	};

	/**
	 * Creates the scene object from a Scene Node data.
	 *
	 * @param Arguments - The structure that contains all necessary arguments. See the structure definition for its documentation.
	 * @return The scene object, or nullptr if the operation was unsuccessful.
	 */
	virtual UObject* ImportSceneObject_GameThread(const FImportSceneObjectsParams& Arguments)
	{
		return nullptr;
	}

	/** Call when the user cancels the operation. */
	virtual void Cancel() {}

	/**
	 * Parameters to pass to the CreateAsset function.
	 */
	struct FSetupObjectParams
	{
		/** The source data, mainly used to set the asset import data file. TODO: we have to refactor UAssetImportData. The source data should be the base class for this now. */
		const UInterchangeSourceData* SourceData = nullptr;
		UInterchangeFactoryBaseNode* FactoryNode = nullptr;


		/** The UObject we want to execute code on. */
		UObject* ImportedObject = nullptr;
		FString NodeUniqueID;
		UInterchangeBaseNodeContainer* NodeContainer = nullptr;
		TArray<UInterchangePipelineBase*> Pipelines;
		TArray<UObject*> OriginalPipelines;
		UInterchangeTranslatorBase* Translator;
		
		bool bIsReimport  = false;
	};

	/*
	 * Do any UObject setup required before the build and after all dependency UObjects have been imported.
	 * @note - This function is called when starting the pre-completion task (before asset build is called for the asset).
	 */
	virtual void SetupObject_GameThread(const FSetupObjectParams& Arguments)
	{
		check(IsInGameThread());
	}

	/*
	 * Build the asset if it can be built.
	 * 
	 * @Param Arguments - The setup object data.
	 * @Param OutPostEditchangeCalled - Set it to true if your implementation has called PostEditChange. Set it to false, interchange will call PostEditChange at the end of the import.
	 * @note - This function is called when starting the pre-completion task.
	 * @note - The default implementation for asset that do not have any build step is to call PostEditChange and set OutPostEditchangeCalled to true.
	 */
	virtual void BuildObject_GameThread(const FSetupObjectParams& Arguments, bool& OutPostEditchangeCalled)
	{
		check(IsInGameThread());
#if WITH_EDITOR
		if (Arguments.ImportedObject)
		{
			Arguments.ImportedObject->PostEditChange();
			OutPostEditchangeCalled = true;
		}
#endif //WITH_EDITOR
	}

	/*
	 * Do any final UObject setup after the asset is built
	 * @note - This function is called after asset is built in FTaskWaitAssetCompilation
	 */
	virtual void FinalizeObject_GameThread(const FSetupObjectParams& Arguments)
	{
		check(IsInGameThread());
	}

	/**
	 * Fills the OutSourceFilenames array with the list of source files contained in the asset source data.
	 * Returns true if the operation was successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Factory")
	virtual bool GetSourceFilenames(const UObject* Object, TArray<FString>& OutSourceFilenames) const
	{
		return false;
	}

	/**
	 * Sets the object's source at the specified index to the given SourceFileName.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Factory")
	virtual bool SetSourceFilename(const UObject* Object, const FString& SourceFilename, int32 SourceIndex) const
	{
		return false;
	}

	/**
	 * Backups SourceData from the AssetData.
	 */
	virtual void BackupSourceData(const UObject* Object) const
	{
	}

	/**
	 * Reinstates the backedup SourceData from the Backup to the AssetData.
	 */
	virtual void ReinstateSourceData(const UObject* Object) const
	{
	}

	/**
	 * Clears the backedup SourceData from the Backup to the AssetData.
	 */
	virtual void ClearBackupSourceData(const UObject* Object) const
	{
	}

	/**
	 * Set the object's reimport source at the specified index value.
	 */
	virtual bool SetReimportSourceIndex(const UObject* Object, int32 SourceIndex) const
	{
		return false;
	}

	/**
	 * This function is used to add the given message object directly into the results for this operation.
	 */
	template <typename T>
	T* AddMessage()
	{
		check(Results != nullptr);
		T* Item = Results->Add<T>();
		return Item;
	}


	void AddMessage(UInterchangeResult* Item)
	{
		check(Results != nullptr);
		Results->Add(Item);
	}


	void SetResultsContainer(UInterchangeResultsContainer* InResults)
	{
		Results = InResults;
	}

	/**
	 * Called before resetting object properties. 
	 * Restores all the tracked custom attribute delegates.
	 */
	void PreResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
		// Restore delegates and force make the re-import strategy to apply pipeline properties.
		FactoryNode->RestoreAllCustomAttributeDelegates();
		
		ExecutePreResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);
	}

	/**
	 * Re-apply all the custom attributes that were set on the Factory Node to the Object.
	 * Useful when resetting the object to its original imported state, unless the factory node's attributes have been changed.
	 */
	void ResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
		// Force use of ApplyPipelineProperties Reimport Strategy to apply attributes through the attribute delegates.
		EReimportStrategyFlags CachedReimportStrategyFlags = FactoryNode->GetReimportStrategyFlags();
		FactoryNode->SetReimportStrategyFlags(EReimportStrategyFlags::ApplyPipelineProperties);
		
		ExecuteResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);
		
		// Restore Previous State
		FactoryNode->SetReimportStrategyFlags(CachedReimportStrategyFlags);
	}

	/**
	 * Called after resetting object properties. 
	 * Useful for doing any cleanup required.
	 */
	void PostResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
		ExecutePostResetObjectProperties(BaseNodeContainer, FactoryNode, ImportedObject);		
		FactoryNode->RemoveAllCustomAttributeDelegates();
	}

	/**
	* Acquires the Object to be re-imported.
	* Function is called from FFactoryCommon::GetObjectToReiomport.
	* Which in turn is called from the InterchangeTask related functions to acquire the Object to be Reimported.
	* Allows Factories to override, in order to provide the desired ReimportObject based on Factory requirements.
	*/
	virtual UObject* GetObjectToReimport(UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
	{
		return ReimportObject;
	}

protected:

	/**
	 * Override this method in the child classes, to support custom behavior for PreResetObjectProperties
	 */
	virtual void ExecutePreResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
	}

	/**
	 * Override this method in the child classes if the Custom Attributes are not directly applied to the imported object
	 * E.g. Actor Factories need root component/other components in the hierarchy. Actor Factory overrides this method for custom behavior
	 */
	virtual void ExecuteResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
		FactoryNode->ApplyAllCustomAttributeToObject(ImportedObject);
	}

	/**
	 * Override this method in the child classes, to support custom behavior for PostResetObjectProperties
	 */
	virtual void ExecutePostResetObjectProperties(const UInterchangeBaseNodeContainer* BaseNodeContainer, UInterchangeFactoryBaseNode* FactoryNode, UObject* ImportedObject)
	{
	}

public:
	UPROPERTY()
	TObjectPtr<UInterchangeResultsContainer> Results;
};
