// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"

namespace UE::Interchange
{
	/**
	 * Sanitize InOutName so it's both filename and FName compatible.
	 * @param InOutName The name to sanitize in place, will be replaced by "Null" if empty or invalid for FName.
	 */
	INTERCHANGECORE_API void SanitizeName(FString& InOutName, bool bIsJoint = false);

	/**
	 * Returns a sanitized copy of InName, that is both filename and FName compatible.
	 * Returns "Null" if empty or invalid for FName.
	 */
	INTERCHANGECORE_API FString MakeSanitizedName(const FString& InName, bool bIsJoint = false);

	UE_DEPRECATED(5.7, "Use MakeSanitizedName instead.")
	INTERCHANGECORE_API FString MakeName(const FString& InName, bool bIsJoint = false);

	/**
	 * Try to compute a char budget for asset names, including FNames constraints, OS constraints,
	 * parent package, and user defined limitation.
	 *
	 * @param ParentPackage destination of the asset (Package path consume a part of the budget)
	 * @return An estimation of the budget for asset names
	 */
	INTERCHANGECORE_API int32 GetAssetNameMaxCharCount(const FString& ParentPackage);

	/**
	 * Generate a new name for DesiredAssetName based on a character budget. Any characters out of the budget limit will be stripped out
	 * @param DesiredAssetName - the name of the package that needs to fulfill the OS constraints
	 * @param CharBudget - characters limit for the new generated name (if 0, the string will be returned with a warning, meaning we couldn't generate a new name)
	 * @param CharReplacement - the character replacement of the trimmed substring
	 * @return The new generated name if out of budget, or just the string if it's in the budget limitation
	 */
	INTERCHANGECORE_API FString GetAssetNameWBudget(const FString& ParentPackage, int32 CharBudget, TCHAR CharReplacement = TEXT('_'));

	class FScopedLambda
	{
	public:
		FScopedLambda(TFunction<void(void)>&& Lambda)
			: _Lambda(Lambda)
		{}

		~FScopedLambda()
		{
			if (_Lambda)
			{
				_Lambda();
			}
		}

	private:
		TFunction<void(void)> _Lambda;
	};
};