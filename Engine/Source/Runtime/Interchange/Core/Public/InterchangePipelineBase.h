// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSourceData.h"
#include "Misc/AssertionMacros.h"
#include "Misc/OptionalFwd.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"
#include "Widgets/SCompoundWidget.h"
#include "Widgets/SWindow.h"

#include "InterchangePipelineBase.generated.h"

#define UE_API INTERCHANGECORE_API

class FText;
class UClass;
class UInterchangeBaseNodeContainer;
class UInterchangeResult;
class UInterchangeSourceData;
struct FAnalyticsEventAttribute;
struct FFrame;

UENUM(BlueprintType)
enum class EInterchangePipelineTask : uint8
{
	PostTranslator,
	PostFactory,
	PostImport,
	Export
};

UENUM()
enum class EInterchangePipelineContext : uint8
{
	None, //Default pipeline instance we refer in the project settings pipeline stack. This context should allow editing of the properties states
	AssetImport,
	AssetReimport,
	SceneImport,
	SceneReimport,
	AssetCustomLODImport,
	AssetCustomLODReimport,
	AssetAlternateSkinningImport,
	AssetAlternateSkinningReimport,
	AssetCustomMorphTargetImport, //Import the content has a combine static mesh so we can add a custom morph target to a skeletal mesh
	AssetCustomMorphTargetReImport,
};

USTRUCT()
struct FInterchangePipelineContextParams
{
	GENERATED_BODY()

	UPROPERTY()
	EInterchangePipelineContext ContextType = EInterchangePipelineContext::None;

	UPROPERTY()
	TObjectPtr<UClass> ImportObjectType = nullptr;

	UPROPERTY()
	TObjectPtr<UObject> ReimportAsset = nullptr;

	UPROPERTY()
	TObjectPtr<const UInterchangeBaseNodeContainer> BaseNodeContainer = nullptr;

	friend FArchive& operator<<(FArchive& Ar, FInterchangePipelineContextParams& Params);
};

USTRUCT(BlueprintType)
struct FInterchangePipelinePropertyStatePerContext
{
	GENERATED_BODY()

	/** If true, the property is visible. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Property States")
	bool bVisible = true;

	friend FArchive& operator<<(FArchive& Ar, FInterchangePipelinePropertyStatePerContext& Params);
};

USTRUCT(BlueprintType)
struct FInterchangePipelinePropertyStates
{
	GENERATED_BODY()

	/** Return true if the property is locked. */
	bool IsPropertyLocked() const
	{
		return bLocked;
	}
	
	void SetPropertyLocked(const bool bLockValue)
	{
		bLocked = bLockValue;
	}

	/** Return true if the property is locked. */
	bool IsPropertyPreDialogReset() const
	{
		return bPreDialogReset;
	}

	void SetPropertyPreDialogReset(const bool bPreDialogResetValue)
	{
		bPreDialogReset = bPreDialogResetValue;
	}


	/** Return true if the property is visible for the specified context. */
	bool IsPropertyVisibleInShowEssentials() const
	{
		return BasicLayoutStates.bVisible;
	}

	/** Return true if the property is visible for the specified context. */
	bool IsPropertyVisible(const bool bIsReimportContext, const bool bIsShowEssentials) const
	{
		bool bVisible = bIsReimportContext ? ReimportStates.bVisible : ImportStates.bVisible;
		if (bVisible)
		{
			bVisible = bIsShowEssentials ? BasicLayoutStates.bVisible : true;
		}
		return bVisible;
	}

	void SetPropertyImportVisibility(const bool bVisibleValue)
	{
		ImportStates.bVisible = bVisibleValue;
	}

	void SetPropertyReimportVisibility(const bool bVisibleValue)
	{
		ReimportStates.bVisible = bVisibleValue;
	}

	void SetPropertyShowEssentialsVisibility(const bool bVisibleValue)
	{
		BasicLayoutStates.bVisible = bVisibleValue;
	}

	/** If true, the property is locked. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Property States")
	bool bLocked = false;

	/** If true, the property will be reset to default when loading the import dialog. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Property States")
	bool bPreDialogReset = false;

	/** The property states for the import context. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Context Properties States")
	FInterchangePipelinePropertyStatePerContext BasicLayoutStates;

	/** The property states for the import context */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Context Properties States")
	FInterchangePipelinePropertyStatePerContext ImportStates;

	/** The property states for the reimport context. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Context Properties States")
	FInterchangePipelinePropertyStatePerContext ReimportStates;

	friend FArchive& operator<<(FArchive& Ar, FInterchangePipelinePropertyStates& Params);
};

struct FInterchangeConflictInfo
{
	FString DisplayName;
	FString Description;
	FGuid UniqueId;
	TObjectPtr<UInterchangePipelineBase> Pipeline = nullptr;
	TSet<UClass*> AffectedAssetClasses;
};

class SInterchangeBaseConflictWidget : public SCompoundWidget
{
public:
	SLATE_BEGIN_ARGS(SInterchangeBaseConflictWidget)
		: _WidgetWindow(nullptr)
		{}

		SLATE_ARGUMENT(TSharedPtr<SWindow>, WidgetWindow)

	SLATE_END_ARGS()

	virtual void SetWidgetWindow(TSharedPtr<SWindow> InWidgetWindow)
	{
		WidgetWindow = InWidgetWindow;
	}

	FVector2D GetMinimumSize(float ApplicationScale) const
	{
		return ComputeDesiredSize(ApplicationScale);
	}
protected:
	TSharedPtr<SWindow> WidgetWindow = nullptr;
};

/**
 * Pipeline implementation:
 *
 * 1. ExecutePipeline - Create the factory nodes from the translated nodes. This is where the logic is execute to create the unreal asset via the factory node. Called after the translation
 * 2. ExecutePostFactoryPipeline - Called after the factory has create the unreal asset with the associate factory node, but before calling PostEditChange.
 * 3. ExecutePostImportPipeline - Called after the asset PostEditChange is done. If the asset use the async build framework, the asset build should be completed.
 * 4. ExecutePostBroadcastPipeline - Called after the asset was registered to the registry manager and all broadcast calls have been done.
 */

UCLASS(BlueprintType, Blueprintable, editinlinenew, Abstract, MinimalAPI)
class UInterchangePipelineBase : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * This function is call when we want to list pipeline in the import dialog. If not override the default behavior of this function will search if
	 * the pipeline have a FString UPROPERTY named "PipelineDisplayName" and return the property value. If there is no FString UPROPERTY call "PipelineDisplayName" it will
	 * return the name of the pipeline asset (UObject::GetName).
	 * 
	 * When creating a pipeline (c++, python or blueprint) you can simply add a UPROPERTY name "PipelineDisplayName" to your pipeline, you do not need to override the function.
	 * Use the same category has your other options and put it on the top.
	 * The meta tag StandAlonePipelineProperty will hide your PROPERTY if your pipeline is a sub object of another pipeline when showing the import dialog.
	 * The meta tag PipelineInternalEditionData make sure the property will be show only when we edit the pipeline object (hidden when showing the import dialog).
	 * 
	 *
	 * UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Textures", meta = (StandAlonePipelineProperty = "True", PipelineInternalEditionData = "True"))
	 * FString PipelineDisplayName;
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API FString ScriptedGetPipelineDisplayName() const;
	/** The default implementation (call if the blueprint do not have any implementation) will call the virtual ExecutePipeline */
	FString ScriptedGetPipelineDisplayName_Implementation() const
	{
		//By default we call the virtual GetPipelineDisplayName
		return GetPipelineDisplayName();
	}

	/**
	 * ScriptedExecutePipeline, is call after the translation and before we parse the graph to call the factory.
	 * This is where factory node should be created by the pipeline.
	 * Each factory node represent an unreal asset create that will be create by an interchange factory.
	 * @note - the FTaskPipeline is calling this function not the virtual one that is call by the default implementation.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API void ScriptedExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath);
	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ExecutePipeline(). */
	void ScriptedExecutePipeline_Implementation(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
	{
		//By default we call the virtual import pipeline execution
		ExecutePipeline(BaseNodeContainer, SourceDatas, ContentBasePath);
	}

	/**
	 * ScriptedExecutePostFactoryPipeline is called after the factory creates an Unreal asset, but before it calls PostEditChange.
	 * @note - The FTaskPreCompletion task calls this function, not the virtual one that is called by the default implementation.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API void ScriptedExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport);
	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ExecutePostFactoryPipeline(). */
	void ScriptedExecutePostFactoryPipeline_Implementation(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
		//By default we call the virtual import pipeline execution
		ExecutePostFactoryPipeline(BaseNodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);
	}
	/**
	 * ScriptedExecutePostImportPipeline is called after an asset is completely imported, after PostEditChange has already been called.
	 * This can be useful if you need build data for one asset to finish setting up another asset.
	 * @example - A PhysicsAsset needs skeletal mesh render data to be built properly.
	 * @note - the FTaskPostImport calls this function not the virtual one that is call by the default implementation.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API void ScriptedExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport);
	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ExecutePostImportPipeline(). */
	void ScriptedExecutePostImportPipeline_Implementation(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
		//By default we call the virtual import pipeline execution
		ExecutePostImportPipeline(BaseNodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);
	}

	/**
	 * ScriptedExecutePostBroadcastPipeline is called after an asset is completely imported and the broadcast have been called.
	 * This can be useful if you need to unload the asset for any reason (Level reference by level instance need to be unload).
	 * @note - the FTaskCompletion_GameThread calls this function not the virtual one that is call by the default implementation.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API void ScriptedExecutePostBroadcastPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport);
	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ExecutePostBroadcastPipeline(). */
	void ScriptedExecutePostBroadcastPipeline_Implementation(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& FactoryNodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
		//By default we call the virtual import pipeline execution
		ExecutePostBroadcastPipeline(BaseNodeContainer, FactoryNodeKey, CreatedAsset, bIsAReimport);
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function.
	 * The Interchange manager calls this function, not the virtual one that is called by the default implementation.
	 */
	UFUNCTION(BlueprintCallable, BlueprintNativeEvent, Category = "Interchange | Pipeline")
	UE_API void ScriptedExecuteExportPipeline(UInterchangeBaseNodeContainer* BaseNodeContainer);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ExecuteExportPipeline(). */
	void ScriptedExecuteExportPipeline_Implementation(UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		ExecuteExportPipeline(BaseNodeContainer);
	}

	UE_DEPRECATED(5.4, "This function will be remove, call CanExecuteOnAnyThread function")
	bool ScriptedCanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask)
	{
		return CanExecuteOnAnyThread(PipelineTask);
	}

	/**
	 * This function tells the Interchange manager if this pipeline can be executed in async mode. If it returns false, the ScriptedExecuteImportPipeline
	 * will be called on the main thread (GameThread). If it returns true, the pipeline will be run in a background thread and possibly in parallel if there are multiple
	 * import processes at the same time.
	 *
	 */
	virtual bool CanExecuteOnAnyThread(EInterchangePipelineTask PipelineTask)
	{
		return true;
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function.
	 * the Interchange framework calls this function, not the virtual one that is called by the default implementation.
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API void ScriptedSetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual SetReimportContentFromSourceIndex(). */
	void ScriptedSetReimportSourceIndex_Implementation(UClass* ReimportObjectClass, const int32 SourceFileIndex)
	{
		SetReimportSourceIndex(ReimportObjectClass, SourceFileIndex);
	}

	/**
	 * A non-scripted class should return false here. The default is set to true because scripted classes cannot override
	 * this function since it can be called in an asynchronous thread, which is not possible for Python.
	 *
	 * We cannot call ScriptedCanExecuteOnAnyThread for a scripted Python pipeline from the task parsing async thread.
	 * This function allows us to not call it and force the ScriptedExecutePostImportPipeline to execute on the game thread.
	 */
	virtual bool IsScripted()
	{
		return true;
	}

	UE_API static FString GetDefaultConfigFileName();
	
	UE_API void LoadSettings(const FName PipelineStackName, bool bResetPreDialog = false);

	UE_API void SaveSettings(const FName PipelineStackName);

	/**
	 * This function is called before we show the pipeline dialog. Pipelines that override it can change the existing settings according to the reimport type.
	 * The function is also called when we import or reimport custom LOD and alternate skinning.
	 *
	 * @Note - The function will set the context of the pipeline.
	 * @Param ContextParams - Give all the context information to the pipeline for the current import.
	 */
	UE_API virtual void AdjustSettingsForContext(const FInterchangePipelineContextParams& ContextParams);
	UE_API virtual void AdjustSettingsFromCache();

	/** Transfer the source pipeline adjust settings to this pipeline. */
	UE_API void TransferAdjustSettings(const UInterchangePipelineBase* SourcePipeline);

	void SetShowEssentialsMode(bool bShowEssentialsModeValue)
	{
		bIsShowEssentials = bShowEssentialsModeValue;
	}

	/*
	 * Set to true if this pipeline is from a re-import or an override pipelines stack
	 */
	void SetFromReimportOrOverride(bool bInFromReimportOrOverride)
	{
		bFromReimportOrOverride = bInFromReimportOrOverride;
	}

	/*
	 * Return true if the pipeline was created for a re-import or an override pipelines stack
	 */
	bool IsFromReimportOrOverride() const
	{
		return bFromReimportOrOverride;
	}
	
	/**
	 * This function is called before showing the import dialog. It is not called when doing a reimport.
	 */
	virtual void PreDialogCleanup(const FName PipelineStackName) {}

	/**
	 * This function should return true if all the pipeline settings are in a valid state to start the import.
	 * The Interchange Pipeline Configuration Dialog will call this to know if the Import button can be enabled.
	 */
	virtual bool IsSettingsAreValid(TOptional<FText>& OutInvalidReason) const
	{
		return true;
	}

#if WITH_EDITOR
	/**
	 * Filter the pipeline properties from the translated data. This function is called by the import dialog after having duplicated and loaded the settings of the pipeline.
	 * If some specific options change, the UI must refresh the filter. See the IsPropertyChangeNeedRefresh function.
	 */
	virtual void FilterPropertiesFromTranslatedData(UInterchangeBaseNodeContainer* InBaseNodeContainer)
	{
		//The base class function do not have anything to filter
	}

	/**
	 * The import dialog will call this function when the user changes a specific property. Return true if the pipeline UI should be refreshed.
	 * A refresh will call the FilterPropertiesFromTranslatedData function.
	 */
	virtual bool IsPropertyChangeNeedRefresh(const FPropertyChangedEvent& PropertyChangedEvent) const
	{
		return false;
	}

	/** Fill the list of all asset this pipeline can create */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline")
	virtual void GetSupportAssetClasses(TArray<UClass*>& PipelineSupportAssetClasses) const
	{
		return;
	}

#endif //WITH_EDITOR

	/**
	 * Return all conflict the pipeline find in the translated data. This function is call by the import dialog.
	 * If some specific options change, the UI must refresh the conflicts, see function IsPropertyChangeNeedRefresh.
	 */
	virtual TArray<FInterchangeConflictInfo> GetConflictInfos(UObject* ReimportObject, UInterchangeBaseNodeContainer* InBaseNodeContainer, UInterchangeSourceData* SourceData)
	{
		//The base class function do not have any conflict
		ConflictInfos.Empty();
		return ConflictInfos;
	}

	virtual void ShowConflictDialog(const FGuid& ConflictUniqueId)
	{
		return;
	}

	/**
	 * This function is used to add the given message object directly into the results for this operation.
	 */
	template <typename T>
	T* AddMessage() const
	{
		check(Results != nullptr);
		T* Item = Results->Add<T>();
		return Item;
	}

	void AddMessage(UInterchangeResult* Item) const
	{
		check(Results != nullptr);
		Results->Add(Item);
	}
	
	void SetResultsContainer(UInterchangeResultsContainer* InResults)
	{
		Results = InResults;
	}

	/**
	 * Return a const property states pointer. Return nullptr if the property does not exist.
	 */
	UE_API const FInterchangePipelinePropertyStates* GetPropertyStates(const FName PropertyPath) const;

	/**
	 * Return a mutable property states pointer. Return nullptr if the property does not exist.
	 */
	UE_API FInterchangePipelinePropertyStates* GetMutablePropertyStates(const FName PropertyPath);

	/**
	 * Return true if the property has valid states, or false if no states were set for the property.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API bool DoesPropertyStatesExist(const FName PropertyPath) const;

	/**
	 * Return a mutable property states reference. Add the property states if it doesn't exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline")
	UE_API FInterchangePipelinePropertyStates& FindOrAddPropertyStates(const FName PropertyPath);

	/**
	 * Return the property name of the properties states map.
	 */
	static UE_API FName GetPropertiesStatesPropertyName();

	static UE_API FName GetResultsPropertyName();

	bool CanEditPropertiesStates() { return bAllowPropertyStatesEdition; }

	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline")
	bool IsReimportContext() { return bIsReimportContext; }

	bool IsShowEssentials() { return bIsShowEssentials; }

#if WITH_EDITOR
	/*
	 * FName or FString properties can have a dynamic set of possible values. When displaying 
	 * a pipeline property in the property editor, this function will be call for all FString and FName
	 * properties and will generate a combo box with the PossibleValues if the function return true.
	 */
	virtual bool GetPropertyPossibleValues(const FName PropertyPath, TArray<FString>& PossibleValues)
	{
		return false;
	}
#endif //WITH_EDITOR

	/*
	 * Return true if the pipeline is not a sub-pipeline. Standalone pipelines can have some extra properties.
	 * The customize detail of the pipeline will call this function to show or hide those properties.
	 * The pipeline itself can call the function to know whether a property must be used.
	 * Example: the override asset name property.
	 */
	bool IsStandAlonePipeline() const
	{
		return this == GetMostPipelineOuter();
	};

	/*
	* Update/reset the TWeakObjectPtrs to point to their original TObjectPtr.
	* Works based on Variable naming.
	*/
	UE_API void UpdateWeakObjectPtrs();

	/** Begin UObject overrides*/
	UE_API virtual void PostDuplicate(bool bDuplicateForPIE) override;
	UE_API virtual void Serialize(FArchive& Ar) override;
	/** End UObject overrides */

	/*
	 * If this returns true, this pipeline will be saved in the asset import data.
	 * We will reuse this pipeline when reimporting the asset.
	 * If false, it's probably a debug helper pipeline that we do not want to save into assets.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Pipeline")
	virtual bool SupportReimport() const { return true; }

	/*
	 * The Unreal import system has an option to force a name if we import only one main asset (one texture, one mesh or one animation).
	 * The generic asset pipeline uses this information to behave as expected.
	 */
	UPROPERTY()
	FString DestinationName;

	/*
	 * The content path where asset should be created.
	 */
	UPROPERTY()
	FString ContentImportPath;

#if WITH_EDITORONLY_DATA
	/**
	 * Path of the pipeline object used to create an instance of the pipeline.
	 * Useful for creating section names for caching the properties in ini file.
	*/
	UPROPERTY()
	FSoftObjectPath OriginalPipelinePath;
#endif

	UE_API UInterchangePipelineBase* GetMostPipelineOuter() const;

	UE_DEPRECATED(5.8, "Use the version with PropertyValueIterator instead.")
	virtual bool CanAddPropertyToAnalytics(FProperty* Property) const
	{
		return false;
	}

	UE_API virtual bool CanAddPropertyToAnalytics(TPropertyValueIterator<FProperty>& PropertyValueIterator) const;

	UE_DEPRECATED(5.8, "Use the version with PropertyValueIterator instead.")
	virtual void AddPropertyAnalytics(FProperty* Property, TArray<FAnalyticsEventAttribute>& PipelineAttributes)
	{
	}

	UE_API virtual void AddPropertyAnalytics(TPropertyValueIterator<FProperty>& PropertyValueIterator, TArray<FAnalyticsEventAttribute>& PipelineAttributes);


protected:

	UE_API virtual FString GetCategoryNameForAnalytics(const TArray<const FProperty*>& PropertyChain) const;

	UE_API virtual FString GetPipelineDisplayName() const;

	virtual void ExecutePipeline(UInterchangeBaseNodeContainer* BaseNodeContainer, const TArray<UInterchangeSourceData*>& SourceDatas, const FString& ContentBasePath)
	{
	}

	/**
	 * This function is called after the factory creates an asset, but before PostEditChange is called.
	 * It is always called on the game thread. Code inside this function should not wait on other threads. Do not stall the game thread.
	 * This is the place to make any changes to the Unreal asset before PostEditChange is called.
	 */
	virtual void ExecutePostFactoryPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
	}

	/**
	 * This function is called after the Unreal asset is completely imported and PostEditChange was called.
	 * @Note: Some Unreal assets have asynchronous build operations. It's possible they are still compiling.
	 */
	virtual void ExecutePostImportPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
	}

	/**
	 * This function is called after the Unreal asset is completely imported. PostEditChange and all broadcast have been called.
	 * @Note: Some Unreal assets have asynchronous build operations. It's possible they are still compiling.
	 */
	virtual void ExecutePostBroadcastPipeline(const UInterchangeBaseNodeContainer* BaseNodeContainer, const FString& NodeKey, UObject* CreatedAsset, bool bIsAReimport)
	{
	}

	virtual void SetReimportSourceIndex(UClass* ReimportObjectClass, const int32 SourceFileIndex)
	{
	}

	/** This function can modify the BaseNodeContainer to create a pipeline that will set/validate the graph nodes hierarchy and options.*/
	virtual void ExecuteExportPipeline(UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
	}

	UE_API void LoadSettingsInternal(const FName PipelineStackName, const FString& ConfigFilename, TMap<FName, FInterchangePipelinePropertyStates>& ParentPropertiesStates, bool bResetPreDialog, bool& bOutRequiresSaving);

	UE_API void SaveSettingsInternal(const FName PipelineStackName, const FString& ConfigFilename);

#if WITH_EDITOR
	static UE_API void InternalToggleVisibilityPropertiesOfMetaDataValue(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, bool bDoTransientSubPipeline, const FString& MetaDataKey, const FString& MetaDataValue, const bool bVisibilityState);
	static UE_API void HidePropertiesOfCategory(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FString& HideCategoryName, bool bDoTransientSubPipeline = false);
	static UE_API void HidePropertiesOfSubCategory(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FString& HideSubCategoryName, bool bDoTransientSubPipeline = false);
	static UE_API void HideProperty(UInterchangePipelineBase* OuterMostPipeline, UInterchangePipelineBase* Pipeline, const FName& HidePropertyName);
#endif //WITH_EDITOR
	
	/**
	 * If true, the property editor for this pipeline instance will allow editing property states.
	 * If false, the property editor for this pipeline instance will apply the property states.
	 *
	 * Note: If you open a pipeline asset in the Content Browser, you will be able to edit the property states.
	 *       If you import a file with Interchange, the import dialog will apply property states.
	 */
	UPROPERTY(Transient)
	bool bAllowPropertyStatesEdition = true;

	/**
	 * If true, this pipeline instance is used for reimport.
	 * If false, this pipeline instance is used for import.
	 *
	 * Note: This context must be set by the owner instancing this pipeline. This context will be used to determine whether to hide some properties.
	 */
	UPROPERTY(Transient)
	bool bIsReimportContext = false;

	/**
	 * If true, this pipeline instance is use for essentials settings layout.
	 * If false, this pipeline instance is use for normal layout.
	 *
	 * Note: This layout must be set by the owner instancing this pipeline. This layout will be use to hide or not some properties.
	 */
	UPROPERTY(Transient)
	bool bIsShowEssentials = false;

	/*
	 * If true, this pipeline was create to re-import an asset or override the project settings pipelines.
	 * That kind of pipeline will not be treat like project settings pipeline in the UI. PredialogCleanup will not be called.
	 */
	UPROPERTY()
	bool bFromReimportOrOverride = false;

	UPROPERTY()
	TObjectPtr<UInterchangeResultsContainer> Results;

	/**
	 * Map of property path and lock status. Any properties that have a true lock status will be readonly when showing the import dialog.
	 * Use the API to Get and Set the properties states.
	*/
	UPROPERTY()
	TMap<FName, FInterchangePipelinePropertyStates> PropertiesStates;

	UPROPERTY(Transient)
	mutable TMap<FName, FInterchangePipelinePropertyStates> CachePropertiesStates;

	UPROPERTY(Transient)
	mutable FInterchangePipelineContextParams CacheContextParam;

	TArray<FInterchangeConflictInfo> ConflictInfos; 
};

#undef UE_API