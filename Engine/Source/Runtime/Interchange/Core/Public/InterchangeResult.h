// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "Internationalization/Text.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeResult.generated.h"

class UClass;
class UInterchangePipelineBase;


UENUM()
enum class EInterchangeResultType
{
	Success,
	Warning,
	Error
};


/**
 * Base class for the result from an Interchange operation.
 */
UCLASS(Experimental, MinimalAPI)
class UInterchangeResult : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Return the severity of the message: success, warning, error.
	 */
	virtual EInterchangeResultType GetResultType() const { return EInterchangeResultType::Success; }

	/**
	 * Return a user-readable message.
	 */
	virtual FText GetText() const { return FText(); }

	/**
	 * Return a message to be displayed in the MessageLog window.
	 */
	INTERCHANGECORE_API FText GetMessageLogText() const;

	/**
	 * Serialize this UInterchangeResult object to Json.
	 */
	INTERCHANGECORE_API FString ToJson();

	/**
	 * Create a new UInterchangeResult-derived object from a Json representation.
	 */
	static INTERCHANGECORE_API UInterchangeResult* FromJson(const FString& JsonString);


	UPROPERTY()
	FString SourceAssetName;

	UPROPERTY()
	FString DestinationAssetName;

	UPROPERTY()
	FString AssetFriendlyName;

	UPROPERTY()
	TObjectPtr<const UClass> AssetType;

	UPROPERTY()
	FString InterchangeKey;
};


/**
 * Class representing a succesful result.
 */
UCLASS(MinimalAPI)
class UInterchangeResultSuccess : public UInterchangeResult
{
	GENERATED_BODY()

public:

	virtual EInterchangeResultType GetResultType() const override { return EInterchangeResultType::Success; }
	virtual FText GetText() const override { return FText(); }
};


/**
 * Base class representing a warning result.
 */
UCLASS(MinimalAPI)
class UInterchangeResultWarning : public UInterchangeResult
{
	GENERATED_BODY()

public:

	virtual EInterchangeResultType GetResultType() const override { return EInterchangeResultType::Warning; }
};


/**
 * Base class representing a error result.
 */
UCLASS(MinimalAPI)
class UInterchangeResultError : public UInterchangeResult
{
	GENERATED_BODY()

public:

	virtual EInterchangeResultType GetResultType() const override { return EInterchangeResultType::Error; }
};


UCLASS(MinimalAPI)
class UInterchangeResultWarning_Generic : public UInterchangeResultWarning
{
	GENERATED_BODY()

public:

	virtual FText GetText() const override { return Text; }

	UPROPERTY()
	FText Text;
};


UCLASS(MinimalAPI)
class UInterchangeResultError_Generic : public UInterchangeResultError
{
	GENERATED_BODY()

public:

	virtual FText GetText() const override { return Text; }

	UPROPERTY()
	FText Text;
};

UCLASS(MinimalAPI)
class UInterchangeResultError_ReimportFail : public UInterchangeResultError
{
	GENERATED_BODY()

public:

	virtual FText GetText() const override { return NSLOCTEXT("InterchangeResultNS", "UInterchangeResultError_ReimportFail_GetText", "Re-import Fail, see log for more detail."); }
};

/**
* Used for Successful imports with messages with lower than Warning priorities (for ep Display)
*/
UCLASS(MinimalAPI)
class UInterchangeResultDisplay_Generic : public UInterchangeResultSuccess
{
	GENERATED_BODY()

public:

	virtual FText GetText() const override { return Text; }

	UPROPERTY()
	FText Text;
};