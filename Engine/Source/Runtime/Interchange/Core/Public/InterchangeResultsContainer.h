// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "HAL/CriticalSection.h"
#include "HAL/PlatformCrt.h"
#include "InterchangeResult.h"
#include "Misc/ScopeLock.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/Package.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeResultsContainer.generated.h"


UCLASS(Experimental, MinimalAPI)
class UInterchangeResultsContainer : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Returns whether the results container is empty or not
	 */
	bool IsEmpty() const
	{
		FScopeLock ScopeLock(&Lock);
		return Results.IsEmpty();
	}

	/**
	 * Empties the results container
	 */
	INTERCHANGECORE_API void Empty();

	/**
	 * Appends the given results container to this one
	 */
	INTERCHANGECORE_API void Append(UInterchangeResultsContainer* Other);

	/**
	 * Creates a UInterchangeResult of the given type, adds it to the container and returns it.
	 */
	template <typename T>
	T* Add()
	{
		FScopeLock ScopeLock(&Lock);
		T* Item = NewObject<T>(GetTransientPackage());
		Results.Add(Item);
		return Item;
	}

	/**
	 * Adds the given UInterchangeResult to the container.
	 */
	void Add(UInterchangeResult* Item)
	{
		FScopeLock ScopeLock(&Lock);
		Results.Add(Item);
	}

	/**
	 * Finalizes the container, prior to passing it to the UI display
	 */
	INTERCHANGECORE_API void Finalize();

	/**
	 * Return the contained array (by value, for thread safety).
	 */
	TArray<UInterchangeResult*> GetResults() const
	{
		FScopeLock ScopeLock(&Lock);
		return Results;
	}

	/**
	 * Removes the given UInterchangeResult from the container.
	 */
	void RemoveResult(UInterchangeResult* Item)
	{
		FScopeLock ScopeLock(&Lock);
		Results.Remove(Item);
	}

private:

	mutable FCriticalSection Lock;

	UPROPERTY()
	TArray<TObjectPtr<UInterchangeResult>> Results;
};
