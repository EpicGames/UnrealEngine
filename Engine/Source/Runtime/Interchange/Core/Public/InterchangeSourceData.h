// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/Paths.h"
#include "Misc/SecureHash.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSourceData.generated.h"

/*
 * Helper class to be able to read different source data
 * File on disk
 * HTTP URL (TODO)
 * Memory buffer (TODO)
 * Stream (TODO)
 */

UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UInterchangeSourceData : public UObject
{
	 GENERATED_BODY()

public:
	INTERCHANGECORE_API UInterchangeSourceData();

	INTERCHANGECORE_API UInterchangeSourceData(FString InFilename);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	inline FString GetFilename() const
	{
		return Filename;
	}

	/** Return the hash of the content point by Filename. */
	inline TOptional<FMD5Hash> GetFileContentHash() const
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeSourceData::GetFileContentHash);
		if(!FileContentHashCache.IsSet())
		{
			ComputeFileContentHashCache();
		}
		return FileContentHashCache;
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	inline bool SetFilename(const FString& InFilename)
	{
		Filename = FPaths::ConvertRelativePathToFull(InFilename);
		//Reset the cache
		FileContentHashCache.Reset();
		return true;
	}

	/** Return a easy to read source description string, this is mainly use for logging or UI. */
	inline FString ToDisplayString() const
	{
		if (!Filename.IsEmpty())
		{
			bool bFileExist = FPaths::FileExists(Filename);
			FString CleanFilename = FPaths::GetCleanFilename(Filename);
			FString BasePath = FPaths::GetPath(Filename);
			//limit the base path to 20 character do ellipsis in the middle
			if (BasePath.Len() > 43)
			{
				FString RightPath = BasePath.Right(20);
				FString LeftPath = BasePath.Left(20);
				BasePath = LeftPath + TEXT("...") + RightPath;
			}
			FString DisplayString = BasePath + TEXT("/") + CleanFilename;
			return DisplayString;
		}
		return FString();
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	INTERCHANGECORE_API UObject* GetContextObjectByTag(const FString& Tag) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	INTERCHANGECORE_API void SetContextObjectByTag(const FString& Tag, UObject* Object) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	INTERCHANGECORE_API TArray<FString> GetAllContextObjectTags() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	INTERCHANGECORE_API void RemoveAllContextObjects() const;

private:
	void INTERCHANGECORE_API ComputeFileContentHashCache() const;

	UPROPERTY()
	FString Filename;

	/**
	 * Hash cache for the file content.
	 * We use mutable because the cache is computed when we use the get, which is a const function.
	 * The cache is computed only in the GetFileContentHash() to let the client control in which thread it will compute the cache.
	 * It also makes sure we do not waste CPU computing a cache in case no client uses GetFileContentHash().
	 */
	mutable TOptional<FMD5Hash> FileContentHashCache;

	/**
	 * UObjects that are accessible by the translators, pipelines, and the caller of the Interchange import.
	 *
	 * Use this to transmit additional information that is beneficial to reuse but can't be serialized, such as
	 * external SDK memory objects, external assets, or large cached data.
	 */
	UPROPERTY()
	mutable TMap<FString, TObjectPtr<UObject>> ContextObjectsByTag;
};
