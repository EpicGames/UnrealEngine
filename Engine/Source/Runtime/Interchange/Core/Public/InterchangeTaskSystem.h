// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/Ticker.h"
#include "HAL/CriticalSection.h"
#include "Misc/ScopeLock.h"

#define INTERCHANGE_INVALID_TASK_ID 0xFFFFFFFFFFFFFFFF

DECLARE_MULTICAST_DELEGATE(FOnInterchangeTaskSystemTick)

namespace UE::Interchange
{
	//Forward declare to add it has friend
	class FInterchangeTaskSystem;

	enum class EInterchangeTaskStatus : uint8
	{
		Waiting, //Task is queue for execution when prerequisite are terminate and resource are available
		Executing, //Task is being executing
		Done //Task was execute and is now terminate
	};

	enum class EInterchangeTaskThread : uint8
	{
		GameThread, //Task will be tick in a ticker in the engine tick on the game thread. This is a safe place to manipulate UObject
		AsyncThread //Task will be call on asynchronous thread, code running inside those task must be thread safe.
	};

	class FInterchangeTaskBase
	{
	public:
		FInterchangeTaskBase()
		{}

		virtual ~FInterchangeTaskBase()
		{
			if (GetTaskStatus() == EInterchangeTaskStatus::Executing)
			{
				if (Future.IsValid())
				{
					Future.Get();
				}
			}
		}

		uint64 GetTaskId() const
		{
			return TaskId;
		}

		virtual EInterchangeTaskThread GetTaskThread() const
		{
			return EInterchangeTaskThread::GameThread;
		}

		const TArray<uint64>& GetPrerequisiteTasks() const
		{
			return PrerequisiteTasks;
		}

		EInterchangeTaskStatus GetTaskStatus() const
		{
			FScopeLock TaskStatusLock(&TaskStatusCriticalSection);
			return TaskStatus;
		}

		/**
		 * This function must return the task id to fill the promise of the execution.
		 * 
		 * If the caller want to re-enqueue the task and not run the subsequent tasks after
		 * this function, It must set the task status to Waiting and the task will be
		 * re-execute later on another tick. This is the best way to implement a async wait.
		 */
		virtual void Execute() = 0;

		/**
		 * Wait can be call only on game thread. Async thread wait can create dead lock by using all the thread resources.
		 * 
		 * The caller have some restriction to follow
		 *    - This must be a safe place for any game thread task that will be executing during the wait.
		 *        - Wait should be call from a engine tick and no lock (like the global UObject locks) should be taken before calling wait.
		 *        - Slate ui callback are safe
		 *        - Engine tick is safe, use FTSTicker::GetCoreTicker() when you can
		 *    - Wait should not happen from a package/asset load call stack
		 *    - Wait should not happen from a save package/asset call stack
		 *    - Wait should not happen from a garbage collect call stack
		 * 
		 * If your system need to wait and you already have some locks or you are under a unsafe place you need to redesign your system to
		 * move the wait before you get to a unsafe call stack place to wait.
		 */
		INTERCHANGECORE_API void Wait() const;
	protected:

		//Only friend FInterchangeTaskSystem can set a new status
		INTERCHANGECORE_API bool SetTaskStatus(EInterchangeTaskStatus NewTaskStatus);

		INTERCHANGECORE_API void SetPrerequisites(const TArray<uint64>& InPrerequisiteTasks);

	private:

		mutable FCriticalSection TaskStatusCriticalSection;
		uint64 TaskId = INTERCHANGE_INVALID_TASK_ID;
		EInterchangeTaskStatus TaskStatus = EInterchangeTaskStatus::Waiting;
		TArray<uint64> PrerequisiteTasks;

		//If a task is asynchronous the future will be set.
		//The future will contain the task id when it will be ready.
		TFuture<uint64> Future;

		friend FInterchangeTaskSystem;
	};

	/**
	 * This class is use to run a lambda with the interchange task system.
	 */
	class FInterchangeTaskLambda : public FInterchangeTaskBase
	{
	private:
		EInterchangeTaskThread TaskThread = EInterchangeTaskThread::GameThread;
		TFunction<void()> ExecuteLambda;
		

	public:
		FInterchangeTaskLambda(EInterchangeTaskThread InTaskThread, TFunction<void()> InExecuteLambda)
			: TaskThread(InTaskThread)
			, ExecuteLambda(MoveTemp(InExecuteLambda))
		{
		}

		virtual EInterchangeTaskThread GetTaskThread() const override
		{
			return TaskThread;
		}

		virtual void Execute()
		{
			ExecuteLambda();
		}
	};

	class FInterchangeTaskSystem
	{
	public:
		/**
		 * The constructor should not be called directly, an ensure will be set if its the case
		 * Use the static Get function to acces the singleton.
		 */
		FInterchangeTaskSystem();

		//The interchange task system is a singleton
		static INTERCHANGECORE_API FInterchangeTaskSystem& Get();

		/**
		 * Add a task with some prerequisites. Return the task id.
		 * If there is an issue the returned task id will be INTERCHANGE_INVALID_TASK_ID
		 * The task will start when all prerequisites are done and some thread resource will be available.
		 */
		INTERCHANGECORE_API uint64 AddTask(TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task, TArray<uint64>& TaskPrerequisites);

		/**
		 * Add a task without prerequisite. Return the task id.
		 * If there is an issue the returned task id will be INTERCHANGE_INVALID_TASK_ID
		 * The task will start when some thread resource will be available.
		 */
		INTERCHANGECORE_API uint64 AddTask(TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe> Task);

		/** Return the current task status */
		INTERCHANGECORE_API EInterchangeTaskStatus GetTaskStatus(const uint64 TaskId) const;

		/**
		 * Cancel a task, you can control if you cancel also the prerequisites, and subsequents.
		 * Canceling a task that is waiting will simply push it state to Done which will allow subsequent to be executed.
		 */
		INTERCHANGECORE_API void CancelTask(const uint64 TaskId, const bool bCancelPrerequisites);

		/**
		 * Wait can be call only on game thread. Async thread wait can create dead lock by using all the thread resources.
		 *
		 * The caller have some restriction to follow
		 *    - This must be a safe place for any game thread task that will be executing during the wait.
		 *        - Wait should be call from a engine tick and no lock (like the global UObject locks) should be taken before calling wait.
		 *        - Slate ui callback are safe
		 *        - Engine tick is safe, use FTSTicker::GetCoreTicker() when you can
		 *    - Wait should not happen from a package/asset load call stack
		 *    - Wait should not happen from a save package/asset call stack
		 *    - Wait should not happen from a garbage collect call stack
		 *
		 * If your system need to wait and you already have some locks or you are under a unsafe place you need to redesign your system to
		 * move the wait before you get to a unsafe call stack place to wait.
		 */
		INTERCHANGECORE_API void WaitUntilTasksComplete(const TArray<uint64>& TasksToComplete);

		FOnInterchangeTaskSystemTick& OnTaskSystemTickDelegate()
		{
			return OnTaskSystemTick;
		}

	private:

		static bool bIsCreatingSingleton;

		FOnInterchangeTaskSystemTick OnTaskSystemTick;

		//Internal cancel request with no lock, lock is handle by the caller (CancelTask)
		void InternalAddCancelRequestNoLock(const uint64 TaskId, bool bCancelPrerequisites);

		/**
		 * The tick is where we start task execution and where we update the tasks status
		 */
		void Tick();

		//The ticker we use to tick in the engine tick
		FTSTicker::FDelegateHandle	TickTickerHandle;

		// Store the tasks per id
		TMap<uint64, TSharedPtr<FInterchangeTaskBase, ESPMode::ThreadSafe>> TaskPerIdMap;
		mutable FCriticalSection TaskPerIdMapCriticalSection;

		//When a task is done, we add it here and release the task. Prerequisite are search in the list and here.
		//A clean up is done when the TaskPerIdMap is empty.
		TSet<uint64> ReleaseAndDoneTasks;

		//The priority value is a counter that is increment for each task added to the system to create the task id
		//The lower task id are execute earlier if possible.
		uint64 PriorityValue = 0;
		FCriticalSection PriorityValueCriticalSection;

		TArray<uint64> CancelTaskRequests;
		FCriticalSection CancelTaskRequestsCriticalSection;
	};
} //namespace UE::Interchange
