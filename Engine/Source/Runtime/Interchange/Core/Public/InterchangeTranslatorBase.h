// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "InterchangeResult.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSourceData.h"
#include "Misc/AssertionMacros.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectPtr.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeTranslatorBase.generated.h"

class UInterchangeBaseNodeContainer;
class UInterchangeAnalyticsHandlerBase;

/**
 * Base class for translator settings, the UInterchangeAssetImportData will store this data if the source translator exist
 */
UCLASS(BlueprintType, Blueprintable, editinlinenew, Abstract, MinimalAPI)
class UInterchangeTranslatorSettings : public UObject
{
	GENERATED_BODY()

public:
	UFUNCTION(BlueprintCallable, Category = "Interchange")
	INTERCHANGECORE_API void LoadSettings();

	UFUNCTION(BlueprintCallable, Category = "Interchange")
	INTERCHANGECORE_API void SaveSettings();
};

UENUM(BlueprintType)
enum class EInterchangeTranslatorType : uint8
{
	Invalid = 0,
	Assets = 1 << 1,
	Actors = 1 << 2,
	Scenes = Assets | Actors
};

UENUM(BlueprintType)
enum class EInterchangeTranslatorAssetType : uint8
{
	None = 0x0,
	Textures = 0x1 << 0,
	Materials = 0x1 << 1,
	Meshes = 0x1 << 2,
	Animations = 0x1 << 3,
	Sounds = 0x1 << 4,
	Grooms = 0x1 << 5,
};
ENUM_CLASS_FLAGS(EInterchangeTranslatorAssetType);

UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UInterchangeTranslatorBase : public UObject
{
	GENERATED_BODY()
public:

	/** return true if the translator can translate the given source data. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	INTERCHANGECORE_API virtual bool CanImportSourceData(const UInterchangeSourceData* InSourceData) const;

	/**
	 * Return true if the translator can be instantiate more then one time.
	 * Some translator are not thread safe and cannot translate two files in same time.
	 * A non thread safe translator could use the InterchangeWorker program to run in a separate process and return true.
	 */
	virtual bool IsThreadSafe() const { return true; }

	/** Specifies the capabilities of a translator. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	virtual EInterchangeTranslatorType GetTranslatorType() const { return EInterchangeTranslatorType::Assets; }

	bool DoesSupportAssetType(EInterchangeTranslatorAssetType AssetType) const { return EnumHasAllFlags(GetSupportedAssetTypes(), AssetType); }

	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	virtual EInterchangeTranslatorAssetType GetSupportedAssetTypes() const PURE_VIRTUAL(UInterchangeTranslatorBase::GetSupportedAssetTypes, return EInterchangeTranslatorAssetType::None;);

	/** List of formats supported by the translator. Each entry is of the form "ext;Description" where ext is the file extension. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	virtual TArray<FString> GetSupportedFormats() const PURE_VIRTUAL(UInterchangeTranslatorBase::GetSupportedExtensions, return TArray<FString>(););

	/**
	 * Translate the associated source data into node(s) that are hold in the specified nodes container.
	 *
	 * @param BaseNodeContainer - The nodes container where to put the translated source data result.
	 * @return true if the translator can translate the source data, false otherwise.
	 */
	virtual bool Translate(UInterchangeBaseNodeContainer& BaseNodeContainer) const
	{
		return false;
	}

	/**
	 * Release source is called when we do not need anymore the translator source and also early in the cancel process.
	 * This is where out of process translator will send the stop command.
	 * A good example of why its useful to release the source is when the dispatcher delete the worker, the join on the
	 * thread will be very short and will not block the main thread. If the worker stop command was send before the completion task.
	 */
	virtual void ReleaseSource()
	{
		return;
	}

	/**
	 * This function is call when the import is done (FImportAsyncHelper::CleanUp) and we are cleaning the data.
	 * Use it to free resource that need to be release before the next garbage collector pass.
	 */
	virtual void ImportFinish()
	{
		return;
	}

	/**
	 * This function is used to add the given message object directly into the results for this operation.
	 */
	template <typename T>
	T* AddMessage() const
	{
		if (ensure(Results) && ensure(SourceData))
		{
			T* Item = Results->Add<T>();
			Item->SourceAssetName = SourceData->GetFilename();
			return Item;
		}
		return nullptr;
	}

	void AddMessage(UInterchangeResult* Item) const
	{
		if (ensure(Results) && ensure(SourceData))
		{
			Results->Add(Item);
			Item->SourceAssetName = SourceData->GetFilename();
		}
	}
	
	void SetResultsContainer(UInterchangeResultsContainer* InResults)
	{
		Results = InResults;
	}

	/**
	 * Get the associated source data for this translator.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	const UInterchangeSourceData* GetSourceData() const
	{
		return SourceData;
	}

	/**
	 * Return this translator settings
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	virtual UInterchangeTranslatorSettings* GetSettings() const { return nullptr; }

	/**
	 * Reimport will set the settings if they exist in UInterchangeAssetImportData
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Translator")
	virtual void SetSettings(const UInterchangeTranslatorSettings* InterchangeTranslatorSettings) { }

	UPROPERTY()
	TObjectPtr<UInterchangeResultsContainer> Results;

	UPROPERTY()
	TObjectPtr<const UInterchangeSourceData> SourceData;

	TObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler = nullptr;
};
