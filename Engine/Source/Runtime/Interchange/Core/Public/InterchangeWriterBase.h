// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeWriterBase.generated.h"

UCLASS(BlueprintType, Blueprintable, Abstract, Experimental, MinimalAPI)
class UInterchangeWriterBase : public UObject
{
	GENERATED_BODY()
public:

	/**
	 * Export an nodes container (graph)
	 * @param BaseNodeContainer - The container holding nodes that describe what to export
	 * @return true if the writer can export the Nodes, false otherwise.
	 */
	virtual bool Export(UInterchangeBaseNodeContainer* BaseNodeContainer) const {return false;}
};
