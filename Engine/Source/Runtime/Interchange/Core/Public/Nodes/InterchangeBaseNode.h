// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/StringFwd.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "Delegates/Delegate.h"
#include "HAL/PlatformCrt.h"
#include "Math/Color.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Guid.h"
#include "Misc/TVariant.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "Templates/SharedPointer.h"
#include "Types/AttributeStorage.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Class.h"
#include "UObject/Field.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/UnrealType.h"

#include "InterchangeBaseNode.generated.h"

class FArchive;
class FAssetRegistryTagsContext;
class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
struct FFrame;

/**
 * Internal Helper to get set custom property for class that derive from UInterchangeBaseNode. This is used by the macro IMPLEMENT_UOD_ATTRIBUTE.
 */
namespace InterchangePrivateNodeBase
{
	/**
	 * Retrieve a custom attribute if the attribute exists.
	 *
	 * @param Attributes - The attribute storage you want to query for the custom attribute.
	 * @param AttributeKey - The storage key for the attribute.
	 * @param OperationName - The name of the operation, in case there is an error.
	 * @param OutAttributeValue - This is where we store the value we retrieve from the storage.
	 *
	 * @return - returns true if the attribute exists in the storage and was queried without error.
	 *           returns false if the attribute does not exist or there is an error retrieving it from the storage.
	 */
	template<typename ValueType>
	bool GetCustomAttribute(const UE::Interchange::FAttributeStorage& Attributes, const UE::Interchange::FAttributeKey& AttributeKey, const FString& OperationName, ValueType& OutAttributeValue)
	{
		if (!Attributes.ContainAttribute(AttributeKey))
		{
			return false;
		}
		UE::Interchange::FAttributeStorage::TAttributeHandle<ValueType> AttributeHandle = Attributes.GetAttributeHandle<ValueType>(AttributeKey);
		if (!AttributeHandle.IsValid())
		{
			return false;
		}
		UE::Interchange::EAttributeStorageResult Result = AttributeHandle.Get(OutAttributeValue);
		if (!UE::Interchange::IsAttributeStorageResultSuccess(Result))
		{
			UE::Interchange::LogAttributeStorageErrors(Result, OperationName, AttributeKey);
			return false;
		}
		return true;
	}

	/**
	 * Add or update a custom attribute value in the specified storage.
	 *
	 * @param Attributes - The attribute storage in which you want to add or update the custom attribute.
	 * @param AttributeKey - The storage key for the attribute.
	 * @param OperationName - The name of the operation, in case there is an error.
	 * @param AttributeValue - The value to add or update in the storage.
	 */
	template<typename ValueType>
	bool SetCustomAttribute(UE::Interchange::FAttributeStorage& Attributes, const UE::Interchange::FAttributeKey& AttributeKey, const FString& OperationName, const ValueType& AttributeValue)
	{
		UE::Interchange::EAttributeStorageResult Result = Attributes.RegisterAttribute(AttributeKey, AttributeValue);
		if (!UE::Interchange::IsAttributeStorageResultSuccess(Result))
		{
			UE::Interchange::LogAttributeStorageErrors(Result, OperationName, AttributeKey);
			return false;
		}
		return true;
	}

	/**
	 * Finds a property by name in Outer and supports looking into FStructProperties (embedded structs) with a '.' separating the property names.
	 * 
	 * @param Container - The container for the property values. If the final property is inside a UScriptStruct, the container will be set to the UScriptStruct instance address.
	 * @param Outer - The UStruct containing the top level property.
	 * @param PropertyPath - A dot separated chain of properties. Doesn't support going through external objects.
	 * @return The Property matching the last name in the PropertyPath.
	 */
	INTERCHANGECORE_API FProperty* FindPropertyByPathChecked(TVariant<UObject*, uint8*>& Container, UStruct* Outer, FStringView PropertyPath);
}

/**
 * Use these macros to create Get/Set/ApplyToAsset for an attribute you want to support in your custom UOD node that derive from UInterchangeBaseNode.
 * The attribute key will be declared under the private access modifier, and the functions are declared under the public access modifier.
 * The class access modifier is public after calling this macro.
 *
 * @note - The Get will return false if the attribute was never set.
 * @note - The Set will add the attribute to the storage or update the value if the storage already has this attribute.
 * @note - The Apply will set a member variable of the AssetType instance. It will apply the value only if the storage contains the attribute key
 *
 * @param AttributeName - The name of the Get/Set functions. For example, if you pass Foo you will end up with GetFoo and SetFoo functions.
 * @param AttributeType - This is to specify the type of the attribute: bool, float, FString, or anything else supported by the FAttributeStorage.
 * @param AssetType - This is the asset you want to apply the storage value.
 * @param EnumType - Optional. Specify only if the AssetType member is an enum so we can type cast it in the apply function (we use uint8 to store the enum value).
 */
#define IMPLEMENT_NODE_ATTRIBUTE_KEY(AttributeName)																				\
const UE::Interchange::FAttributeKey Macro_Custom##AttributeName##Key = UE::Interchange::FAttributeKey(TEXT(#AttributeName));


#define IMPLEMENT_NODE_ATTRIBUTE_GETTER(AttributeName, AttributeType)																					\
	FString OperationName = GetTypeName() + TEXT(".Get" #AttributeName);																				\
	return InterchangePrivateNodeBase::GetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, OperationName, AttributeValue);

#define IMPLEMENT_NODE_ATTRIBUTE_SETTER_NODELEGATE(AttributeName, AttributeType)																		\
	FString OperationName = GetTypeName() + TEXT(".Set" #AttributeName);																				\
	return InterchangePrivateNodeBase::SetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, OperationName, AttributeValue);


#define INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(ValueType)																												\
	UE::Interchange::FAttributeStorage::TAttributeHandle<ValueType> Handle = RegisterAttribute<ValueType>(UE::Interchange::FAttributeKey(NodeAttributeKey), Value);	\
	return Handle.IsValid();

#define INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(ValueType)																							\
	if(HasAttribute(UE::Interchange::FAttributeKey(NodeAttributeKey)))																			\
	{																																			\
		UE::Interchange::FAttributeStorage::TAttributeHandle<ValueType> Handle = GetAttributeHandle<ValueType>(UE::Interchange::FAttributeKey(NodeAttributeKey));	\
		if (Handle.IsValid())																													\
		{																																		\
			return (Handle.Get(OutValue) == UE::Interchange::EAttributeStorageResult::Operation_Success);										\
		}																																		\
	}																																			\
	return false;

//Interchange namespace
namespace UE::Interchange
{
	/**
		* Helper struct used to declare static const data we use in the UInterchangeBaseNode.
		* Nodes that derive from UInterchangeBaseNode can also add a struct that derives from this one to add static data.
		* @note: The static data are mainly for holding Attribute keys. All attributes that are always available for a node should be in this class or a derived class.
		*/
	struct FBaseNodeStaticData
	{
		static INTERCHANGECORE_API const FAttributeKey& UniqueIDKey();
		static INTERCHANGECORE_API const FAttributeKey& DisplayLabelKey();
		static INTERCHANGECORE_API const FAttributeKey& NamespaceKey();
		static INTERCHANGECORE_API const FAttributeKey& ParentIDKey();
		static INTERCHANGECORE_API const FAttributeKey& DesiredChildIndexKey();
		static INTERCHANGECORE_API const FAttributeKey& IsEnabledKey();
		static INTERCHANGECORE_API const FAttributeKey& TargetAssetIDsKey();
		static INTERCHANGECORE_API const FAttributeKey& ClassTypeAttributeKey();
		static INTERCHANGECORE_API const FAttributeKey& AssetNameKey();
		static INTERCHANGECORE_API const FAttributeKey& NodeContainerTypeKey();
	};

} //ns UE::Interchange

UENUM(BlueprintType)
enum class EInterchangeNodeContainerType : uint8
{
	None,
	TranslatedScene,
	TranslatedAsset,
	FactoryData
};

UENUM(BlueprintType)
enum class EInterchangeNodeUserInterfaceContext : uint8
{
	None,
	Preview //When we want to preview the node data. A preview is normally read-only and should not show internal data.
};

/**
 * This struct is used to store and retrieve key-value attributes. The attributes are stored in a generic FAttributeStorage that serializes the values in a TArray64<uint8>.
 * See UE::Interchange::EAttributeTypes to know the supported template types.
 * This is an abstract class. This is the base class of the interchange node graph format; all classes in this format should derive from this class.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeBaseNode : public UObject
{
	GENERATED_BODY()

public:
	static constexpr const TCHAR* HierarchySeparator = TEXT("\\");

	INTERCHANGECORE_API UInterchangeBaseNode();

	/**
	 * Initialize the base data of the node.
	 * @param UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 * If the Node to initialize also will get a Parent, then the UInterchangeBaseNodeContainer's SetupNode is better suited.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API void InitializeNode(const FString& UniqueID, const FString& DisplayLabel, const EInterchangeNodeContainerType NodeContainerType);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API virtual FString GetTypeName() const;

	/**
	 * Icon name, used to retrieve the brush when we display the node in any UI.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API virtual FName GetIconName() const;

#if WITH_EDITOR
	/**
	 * UI that inspects node attributes calls this to give a readable name to an attribute key.
	 */
	INTERCHANGECORE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const;

	/**
	 * UI that inspects node attributes calls this to display or hide an attribute.
	 */
	INTERCHANGECORE_API virtual bool ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const;

	/**
	 * UI that inspects node attributes calls this to display the attribute under the returned category.
	 */
	INTERCHANGECORE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const;

#endif //WITH_EDITOR

#if WITH_EDITORONLY_DATA
	/**
	 * Temporary property set by UI to have the context for ShouldHideAttribute. We use this because the property editor cannot add this custom context when it calls CustomizeDetails.
	 */
	UPROPERTY(Transient, DuplicateTransient)
	EInterchangeNodeUserInterfaceContext UserInterfaceContext = EInterchangeNodeUserInterfaceContext::None;
#endif

	/**
	 * Add an attribute to the node.
	 * @param NodeAttributeKey - The key of the attribute.
	 * @param Value - The attribute value.
	 *
	 */
	template<typename T>
	UE::Interchange::FAttributeStorage::TAttributeHandle<T> RegisterAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey, const T& Value)
	{
		const UE::Interchange::EAttributeStorageResult Result = Attributes->RegisterAttribute(NodeAttributeKey, Value);
		
		if (IsAttributeStorageResultSuccess(Result))
		{
			return Attributes->GetAttributeHandle<T>(NodeAttributeKey);
		}
		LogAttributeStorageErrors(Result, TEXT("RegisterAttribute"), NodeAttributeKey);
		return UE::Interchange::FAttributeStorage::TAttributeHandle<T>();
	}

	/**
	 * Return true if the node contains an attribute with the specified Key.
	 * @param nodeAttributeKey - The key of the searched attribute.
	 *
	 */
	INTERCHANGECORE_API virtual bool HasAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const;

	/**
	 * This function returns an attribute type for the specified Key. Returns type None if the key is invalid.
	 *
	 * @param NodeAttributeKey - The key of the attribute.
	 */
	INTERCHANGECORE_API virtual UE::Interchange::EAttributeTypes GetAttributeType(const UE::Interchange::FAttributeKey& NodeAttributeKey) const;

	/**
	 * This function returns an attribute handle for the specified Key.
	 * If there is an issue with the KEY or storage, the method will trip a check. Always make sure you have a valid key before calling this.
	 * 
	 * @param NodeAttributeKey - The key of the attribute.
	 */
	template<typename T>
	UE::Interchange::FAttributeStorage::TAttributeHandle<T> GetAttributeHandle(const UE::Interchange::FAttributeKey& NodeAttributeKey) const
	{
		return Attributes->GetAttributeHandle<T>(NodeAttributeKey);
	}

	/**
	 * Query all the node attribute keys.
	 */
	INTERCHANGECORE_API void GetAttributeKeys(TArray<UE::Interchange::FAttributeKey>& AttributeKeys) const;

	/**
	 * Remove the specified attribute from this node. Returns false if it could not be removed. If the attribute does not exist, returns true.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool RemoveAttribute(const FString& NodeAttributeKey);

	/**
	 * Add a Boolean attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddBooleanAttribute(const FString& NodeAttributeKey, const bool& Value);
	
	/**
	 * Get a Boolean attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetBooleanAttribute(const FString& NodeAttributeKey, bool& OutValue) const;

	/**
	 * Add a int32 attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddInt32Attribute(const FString& NodeAttributeKey, const int32& Value);
	
	/**
	 * Get a int32 attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetInt32Attribute(const FString& NodeAttributeKey, int32& OutValue) const;

	/**
	 * Add a float attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddFloatAttribute(const FString& NodeAttributeKey, const float& Value);
	
	/**
	 * Get a float attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetFloatAttribute(const FString& NodeAttributeKey, float& OutValue) const;

	/**
	 * Add a double attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddDoubleAttribute(const FString& NodeAttributeKey, const double& Value);
	
	/**
	 * Get a double attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetDoubleAttribute(const FString& NodeAttributeKey, double& OutValue) const;

	/**
	 * Add a string attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddStringAttribute(const FString& NodeAttributeKey, const FString& Value);
	
	/**
	 * Get a string attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetStringAttribute(const FString& NodeAttributeKey, FString& OutValue) const;

	/**
	 * Add a GUID attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddGuidAttribute(const FString& NodeAttributeKey, const FGuid& Value);

	/**
	 * Get a GUID attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetGuidAttribute(const FString& NodeAttributeKey, FGuid& OutValue) const;

	/**
	 * Add an FLinearColor attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddLinearColorAttribute(const FString& NodeAttributeKey, const FLinearColor& Value);
	
	/**
	 * Get an FLinearColor attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetLinearColorAttribute(const FString& NodeAttributeKey, FLinearColor& OutValue) const;

	/**
	 * Add a Vector2 attribute to this node. Returns false if the attribute does not exist or if it cannot be added.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddVector2Attribute(const FString& NodeAttributeKey, const FVector2f& Value);

	/**
	 * Get a Vector2 attribute from this node. Returns false if the attribute does not exist.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetVector2Attribute(const FString& NodeAttributeKey, FVector2f& OutValue) const;

	template<typename AttributeType>
	AttributeType GetAttributeChecked(const FString& NodeAttributeKey) const
	{
		AttributeType Value = AttributeType();
		check(HasAttribute(UE::Interchange::FAttributeKey(NodeAttributeKey)));
		UE::Interchange::FAttributeStorage::TAttributeHandle<AttributeType> Handle = GetAttributeHandle<AttributeType>(UE::Interchange::FAttributeKey(NodeAttributeKey));
		check(Handle.IsValid());
		check(Handle.Get(Value) == UE::Interchange::EAttributeStorageResult::Operation_Success);
		return Value;
	}

	template<typename AttributeType>
	bool GetAttribute(const FString& NodeAttributeKey, AttributeType& OutValue) const
	{
		INTERCHANGE_BASE_NODE_GET_ATTRIBUTE(AttributeType);
	}
	template<typename AttributeType>
	bool SetAttribute(const FString& NodeAttributeKey, const AttributeType& Value)
	{
		INTERCHANGE_BASE_NODE_ADD_ATTRIBUTE(AttributeType);
	}

	/**
	 * Return the unique ID passed in the constructor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API FString GetUniqueID() const;

	/**
	 * Return the display label.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API FString GetDisplayLabel() const;

	/**
	 * Change the display label.
     * @param InDisplayLabel - Desired display label text. Sanitized to "Null" if empty or if equals "None".
    */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetDisplayLabel(const FString& InDisplayLabel);

	/**
	 * Namespace is a prefix added when calling GetUniqueId().
	 * If the attribute exist it will fill Namespace and return true.
	 * If the attribute do not exist, it will not touch OutNamespace and return false.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetNamespace(FString& Namespace) const;

	/**
	 * Return the parent unique ID. If the attribute does not exist, returns InvalidNodeUid().
	 * 
	 * Use UInterchangeBaseNodeContainer::SetNodeParentUid function to create a parent hierarchy. This will ensure the container cache is always in sync
	 * It's suggested to use SetupNode (that will Iintialize, Add to container and setup the Parent hierarchy as well).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API FString GetParentUid() const;

	/**
	 * Return the desired child index. If the attribute do not exist it will return INDEX_NONE.
	 * The interchange node container will use this to re-order children if attribute are specified.
	 * 
	 * Use UInterchangeBaseNodeContainer::SetNodeDesiredChildIndex function to set this node child index. This will ensure the container cache is always in sync
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API int32 GetDesiredChildIndex() const;

	/**
	 * Get the number of target assets relating to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API int32 GetTargetNodeCount() const;

	/**
	 * Get the target assets relating to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API void GetTargetNodeUids(TArray<FString>& OutTargetAssets) const;

	/**
	 * Add an asset node UID relating to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddTargetNodeUid(const FString& AssetUid) const;

	/**
	 * Remove an asset node UID relating to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool RemoveTargetNodeUid(const FString& AssetUid) const;

	/**
	 * If true, the node is imported or exported. If false, it is discarded.
	 * Returns false if the node was disabled. Returns true if the attribute is not there or if it was enabled.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool IsEnabled() const;

	/**
	 * Determine whether this node should be part of the import or export process.
	 * @param bIsEnabled - If true, this node is imported or exported. If false, it is discarded.
	 * @return true if the attribute was set, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetEnabled(const bool bIsEnabled);

	/**
	 * Return the node container type that defines the purpose of the node (factory node, translated scene node, or translated asset node).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API EInterchangeNodeContainerType GetNodeContainerType() const;

	/**
	 * Return an FGuid built from the FSHA1 of all the attribute data contained in the node.
	 *
	 * @note the attribute are sorted by key when building the FSHA1 data. The hash will be deterministic for the same data whatever
	 * the order we add the attributes.
	 * This function interface is pure virtual.
	 */
	INTERCHANGECORE_API virtual FGuid GetHash() const;

	/**
	 * Optional. Any node that can import or export an asset should set the desired name for the asset.
	 * If the attribute was never set, returns GetDisplayLabel().
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API virtual FString GetAssetName() const;

	/**
	 * Set the name for the imported asset this node represents. The asset factory will call GetAssetName().
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API virtual bool SetAssetName(const FString& AssetName);

	/** Return the invalid unique ID. */
	static INTERCHANGECORE_API FString InvalidNodeUid();

	/**
	 * Serialize the node. By default, only the attribute storage is serialized for a node.
	 */
	INTERCHANGECORE_API virtual void Serialize(FArchive& Ar) override;

	static INTERCHANGECORE_API void CompareNodeStorage(const UInterchangeBaseNode* NodeA, const UInterchangeBaseNode* NodeB, TArray<UE::Interchange::FAttributeKey>& RemovedAttributes, TArray<UE::Interchange::FAttributeKey>& AddedAttributes, TArray<UE::Interchange::FAttributeKey>& ModifiedAttributes);
	
	static INTERCHANGECORE_API void CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UInterchangeBaseNode* DestinationNode, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys);
	static INTERCHANGECORE_API void CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys);
	static INTERCHANGECORE_API void CopyStorageAttributes(const UE::Interchange::FAttributeStorage& SourceStorage, UInterchangeBaseNode* DestinationNode, const TArray<UE::Interchange::FAttributeKey>& AttributeKeys);	
	static INTERCHANGECORE_API void CopyStorageAttributes(const UInterchangeBaseNode* SourceNode, UE::Interchange::FAttributeStorage& DestinationStorage, const TArray<TPair<UE::Interchange::FAttributeKey, UE::Interchange::FAttributeKey>>& SrcDestAttributeKeys);

	static INTERCHANGECORE_API void CopyStorage(const UInterchangeBaseNode* SourceNode, UInterchangeBaseNode* DestinationNode);
	
	virtual void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) const
	{
	}

	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
	{
	}

protected:

	/**
	 * Return true if the Namespace attribute is added, false otherwise.
	 */
	bool SetNamespace(const FString& InNamespace);

	/**
	 * Set the desired child index. Only the container should set the desired child index
	 */
	bool SetDesiredChildIndex(const int32& ChildIndex);

	/**
	 * Set the parent unique ID.
	 * Use UInterchangeBaseNodeContainer::SetNodeParentUid function to create a parent hierarchy. This will ensure the container cache is always in sync
	 * It's suggested to use SetupNode (that will Iintialize, Add to container and setup the Parent hierarchy as well).
	 */
	bool SetParentUid(const FString& ParentUid);

	/** The storage use to store the Key value attribute for this node. */
	TSharedPtr<UE::Interchange::FAttributeStorage, ESPMode::ThreadSafe> Attributes;

	bool bIsInitialized = false;

	/**
	 * This tracks the IDs of asset nodes which are the target of factories
	 */
	mutable UE::Interchange::TArrayAttributeHelper<FString> TargetNodes;

	friend UInterchangeBaseNodeContainer;
};
