// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Misc/FileHelper.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Templates/Casts.h"
#include "Templates/Function.h"
#include "Templates/UniquePtr.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeBaseNodeContainer.generated.h"

class FArchive;
class UClass;
class UInterchangeBaseNode;
class UInterchangeFactoryBaseNode;
struct FFrame;
template <typename T> struct TObjectPtr;


/**
 * The Interchange UInterchangeBaseNode graph is a format used to feed factories and writers when they import, reimport, and
 * export an asset or scene.
 *
 * This container holds a flat list of all nodes that have been translated from the source data.
 * Translators fill this container, and the import/export managers read it to execute the import/export process.
 */
 UCLASS(BlueprintType, MinimalAPI)
class UInterchangeBaseNodeContainer : public UObject
{
	GENERATED_BODY()
public:
	INTERCHANGECORE_API UInterchangeBaseNodeContainer();

	/**
	 * Empty the container.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	void Reset()
	{
		Nodes.Reset();
		ChildrenCache.Reset();
	}

	/**
	 * Removes node from Nodes map with the given NodeUniqueID.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	void RemoveNode(const FString& NodeUniqueID)
	{
		Nodes.Remove(NodeUniqueID);
	}

	/**
	 * Add a node to the container. The node is added into a TMap.
	 *
	 * @param Node - a pointer on the node you want to add
	 * @return: return the node unique ID of the added item. If the node already exist it will return the existing ID. Return InvalidNodeUid if the node cannot be added.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API FString AddNode(UInterchangeBaseNode* Node);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool ReplaceFactoryNode(const FString& NodeUniqueID, UInterchangeFactoryBaseNode* NewNode);

	UE_DEPRECATED(5.8, "Use ReplaceFactoryNode instead.")
	void ReplaceNode(const FString& NodeUniqueID, UInterchangeFactoryBaseNode* NewNode)
	{
		ReplaceFactoryNode(NodeUniqueID, NewNode);
	}

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool RemoveFactoryNode(const FString& NodeUniqueID);

	/** Return true if the node unique ID exists in the container. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool IsNodeUidValid(const FString& NodeUniqueID) const;

	/**
	 * Set a namespace to all node of the target class in this container. A valid node namespace is prefix to the unique ID in UInterchangeBaseNode::GetUniqueId().
	 * After adding the namespace this function will find any string attribute in all the node that reference the node unique ID and replace the attribute value with the new unique id.
	 * The last step is to remap the node container with the new Ids.
	 * 
	 * @Param Namespace - Is the new namespace you want to set. Pass an empty string to remove an existing namespace.
	 * @Param TargetClass - Optional, this parameter represent the node class we want to apply the namespace on. If null all node will be tagged with the namespace
	 * 
	 * @Note - Changing all node namespace wont work since some node class use a combination of the unique ID in there attributes and we cannot change those attribute
	 *         in a generic way.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void SetNamespace(const FString& Namespace, UClass* TargetClass);

	/** Unordered iteration of the all nodes. */
	INTERCHANGECORE_API void IterateNodes(TFunctionRef<void(const FString&, UInterchangeBaseNode*)> IterationLambda) const;

	template <typename T>
	void IterateNodesOfType(TFunctionRef<void(const FString&, T*)> IterationLambda) const
	{
		for (const TPair<FString, TObjectPtr<UInterchangeBaseNode>>& NodeKeyValue : Nodes)
		{
			if (T* Node = Cast<T>(NodeKeyValue.Value))
			{
				IterationLambda(NodeKeyValue.Key, Node);
			}
		}
	}

	template <typename T>
	void GetNodesOfType(TArray<T*>& OutNodes) const
	{
		OutNodes.Empty();

		for (const TPair<FString, TObjectPtr<UInterchangeBaseNode>>& NodeKeyValue : Nodes)
		{
			if (T* Node = Cast<T>(NodeKeyValue.Value))
			{
				OutNodes.Add(Node);
			}
		}
	}

	template <typename T>
	void GetNodeUIDsOfType(TArray<FString>& OutNodeUIDs) const
	{
		for (const TPair<FString, TObjectPtr<UInterchangeBaseNode>>& NodeKeyValue : Nodes)
		{
			if (T* Node = Cast<T>(NodeKeyValue.Value))
			{
				OutNodeUIDs.Add(NodeKeyValue.Key);
			}
		}
	}

	/**
	 * Recursively traverse the hierarchy starting with the specified node unique ID.
	 * If the iteration lambda returns true, the iteration will stop. If the iteration lambda returns false, the iteration will continue.
	 */
	void IterateNodeChildren(const FString& NodeUniqueID, TFunctionRef<void(const UInterchangeBaseNode*)> IterationLambda) const
	{
		if (const UInterchangeBaseNode* Node = GetNode(NodeUniqueID))
		{
			IterationLambda(Node);
			const TArray<FString> ChildrenIds = GetNodeChildrenUids(NodeUniqueID);
			for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
			{
				IterateNodeChildren(ChildrenIds[ChildIndex], IterationLambda);
			}
		}
	}

	/**
	 * Recursively traverse the hierarchy starting with the specified node unique ID.
	 * If the iteration lambda returns true, the iteration will stop. If the iteration lambda returns false, the iteration will continue.
	 *
	 * @return - Return true if the iteration was broken, or false otherwise.
	 */
	bool BreakableIterateNodeChildren(const FString& NodeUniqueID, TFunctionRef<bool(const UInterchangeBaseNode*)> IterationLambda) const
	{
		if (const UInterchangeBaseNode* Node = GetNode(NodeUniqueID))
		{
			if (IterationLambda(Node))
			{
				return true;
			}
			const TArray<FString> ChildrenIds = GetNodeChildrenUids(NodeUniqueID);
			for (int32 ChildIndex = 0; ChildIndex < ChildrenIds.Num(); ++ChildIndex)
			{
				if (BreakableIterateNodeChildren(ChildrenIds[ChildIndex], IterationLambda))
				{
					return true;
				}
			}
		}
		return false;
	}

	/** Unordered iteration of the all nodes, but can be stopped early by returning true. */
	INTERCHANGECORE_API void BreakableIterateNodes(TFunctionRef<bool(const FString&, UInterchangeBaseNode*)> IterationLambda) const;

	template <typename T>
	void BreakableIterateNodesOfType(TFunctionRef<bool(const FString&, T*)> IterationLambda) const
	{
		for (const TPair<FString, TObjectPtr<UInterchangeBaseNode>>& NodeKeyValue : Nodes)
		{
			if (T* Node = Cast<T>(NodeKeyValue.Value))
			{
				if (IterationLambda(NodeKeyValue.Key, Node))
				{
					break;
				}
			}
		}
	}

	/** Return all nodes that do not have any parent. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void GetRoots(TArray<FString>& RootNodes) const;

	/** Return all nodes that are of the ClassNode type. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void GetNodes(const UClass* ClassNode, TArray<FString>& OutNodes) const;

	/** Get a node pointer. Once added to the container, nodes are considered const. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API const UInterchangeBaseNode* GetNode(const FString& NodeUniqueID) const;

	/** Get a factory node pointer. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API UInterchangeFactoryBaseNode* GetFactoryNode(const FString& NodeUniqueID) const;

	/** Set the ParentUid of the node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool SetNodeParentUid(const FString& NodeUniqueID, const FString& NewParentNodeUid);

	/* Remove the node's ParentUid, making it into a top-level node */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool ClearNodeParentUid(const FString& NodeUniqueID);

	/** Set the desired child index of the node. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API bool SetNodeDesiredChildIndex(const FString& NodeUniqueID, const int32& NewNodeDesiredChildIndex);

	/** Get the number of children the node has. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API int32 GetNodeChildrenCount(const FString& NodeUniqueID) const;

	/** Get the UIDs of all the node's children. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API TArray<FString> GetNodeChildrenUids(const FString& NodeUniqueID) const;

	INTERCHANGECORE_API TArray<FString>* GetCachedNodeChildrenUids(const FString& NodeUniqueID) const;

	/** Get the nth const child of the node */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API UInterchangeBaseNode* GetNodeChildren(const FString& NodeUniqueID, int32 ChildIndex);

	/** Get the nth const child of the node. Const version. */
	INTERCHANGECORE_API const UInterchangeBaseNode* GetNodeChildren(const FString& NodeUniqueID, int32 ChildIndex) const;

	/**
	 * This function serializes the node container and all node sub-objects it points .
	 * Out-of-process translators like FBX will dump a file containing this data, and the editor
	 * will read the file and regenerate the container from the saved data.
	 */
	INTERCHANGECORE_API void SerializeNodeContainerData(FArchive& Ar);

	/* Serialize the node container into the specified file. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void SaveToFile(const FString& Filename);

	/* Serialize the node container from the specified file. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void LoadFromFile(const FString& Filename);

	/* Fill the cache of children UIDs to optimize the GetNodeChildrenUids call. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	INTERCHANGECORE_API void ComputeChildrenCache();

	/* Sets the children cache from an incoming data set. */
	INTERCHANGECORE_API void SetChildrenCache(const TMap<FString, TArray<FString>>& InChildrenCache);

	/* Gets the children cache. */
	INTERCHANGECORE_API TMap<FString, TArray<FString>>& GetChildrenCache();

	/* Reset the cache of children UIDs. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node Container")
	void ResetChildrenCache()
	{
		ChildrenCache.Reset();
	}

	/**
	 * Checks if ParentNodeUID is an ancestor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetIsAncestor(const FString& NodeUniqueID, const FString& AncestorUID) const;

	INTERCHANGECORE_API void SetupNode(UInterchangeBaseNode* Node, const FString& NodeUID,
		const FString& DisplayLabel, EInterchangeNodeContainerType ContainerType,
		const FString& ParentNodeUID = TEXT(""));

	INTERCHANGECORE_API void SetupAndReplaceFactoryNode(UInterchangeFactoryBaseNode* Node, const FString& NodeUID,
		const FString& DisplayLabel, EInterchangeNodeContainerType ContainerType,
		const FString& OldNodeUID,
		const FString& ParentNodeUID = TEXT(""));

	template<typename PREDICATE_CLASS>
	inline void SortNodes(const PREDICATE_CLASS& Predicate)
	{
		Nodes.ValueSort(Predicate);
	}

private:

	void InternalReorderChildren(TArray<FString>& Children) const;

	UInterchangeBaseNode* GetNodeChildrenInternal(const FString& NodeUniqueID, int32 ChildIndex);

	/** Flat List of the nodes. Since the nodes are variable size, we store a pointer. */
	UPROPERTY(VisibleAnywhere, Category = "Interchange | Node Container")
	TMap<FString, TObjectPtr<UInterchangeBaseNode> > Nodes;

	mutable TMap<FString, TArray<FString> > ChildrenCache;
};
