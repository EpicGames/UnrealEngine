// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

//Interchange namespace
namespace UE
{
namespace Interchange
{
	static const FString& ArrayIndexKey()
	{
		static FString IndexKeyString = "_NameIndex_";
		return IndexKeyString;
	}

	static FAttributeKey BuildArrayKeyFromIndex(const FString& BaseKeyName, uint32 Index)
	{
		FString DepIndexKeyString = BaseKeyName + ArrayIndexKey() + FString::FromInt(Index);
		return FAttributeKey(DepIndexKeyString);
	}

	static FString BuildDisplayNameForArrayKey(const FString& BaseDisplayName, const FString& ArrayAttributeKey)
	{
		static FString IndexKeyString = ArrayIndexKey();
		const int32 IndexPosition = ArrayAttributeKey.Find(IndexKeyString);
		if (IndexPosition != INDEX_NONE)
		{
			const int32 ChopIndex = IndexPosition + IndexKeyString.Len();
			if (ChopIndex < ArrayAttributeKey.Len())
			{
				return BaseDisplayName + ArrayAttributeKey.RightChop(ChopIndex);
			}
		}

		return BaseDisplayName;
	}


/**
 * Helper class to manage an array of items inside an attribute storage
 */
template<typename ItemType>
class TArrayAttributeHelper
{
public:
	~TArrayAttributeHelper()
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		Attributes = nullptr;
		KeyCount = NAME_None;
	}

	void Initialize(const TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe>& InAttributes, const FString& BaseKeyName)
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		check(InAttributes.IsValid());
		Attributes = InAttributes;
		KeyCount = BaseKeyName;
	}

	int32 GetCount() const
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			return 0;
		}
		int32 ItemCount = 0;
		if (AttributePtr->ContainAttribute(GetKeyCount()))
		{
			FAttributeStorage::TAttributeHandle<int32> Handle = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
			if (Handle.IsValid())
			{
				Handle.Get(ItemCount);
			}
		}
		return ItemCount;
	}

	void GetItem(const int32 Index, ItemType& OutItem) const
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			return;
		}
		int32 ItemCount = 0;
		if (!AttributePtr->ContainAttribute(GetKeyCount()))
		{
			return;
		}

		FAttributeStorage::TAttributeHandle<int32> Handle = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
		if (!Handle.IsValid())
		{
			return;
		}
		Handle.Get(ItemCount);
		if (Index >= ItemCount)
		{
			return;
		}
		FAttributeKey DepIndexKey = GetIndexKey(Index);
		FAttributeStorage::TAttributeHandle<ItemType> HandleItem = AttributePtr->GetAttributeHandle<ItemType>(DepIndexKey);
		if (!HandleItem.IsValid())
		{
			return;
		}
		HandleItem.Get(OutItem);
	}

	void GetItems(TArray<ItemType>& OutItems) const
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			OutItems.Empty();
			return;
		}
		int32 ItemCount = 0;
		if (!AttributePtr->ContainAttribute(GetKeyCount()))
		{
			OutItems.Empty();
			return;
		}

		FAttributeStorage::TAttributeHandle<int32> Handle = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
		if (!Handle.IsValid())
		{
			return;
		}
		Handle.Get(ItemCount);

		//Reuse as much memory we can to avoid allocation
		OutItems.Reset(ItemCount);
		for (int32 NameIndex = 0; NameIndex < ItemCount; ++NameIndex)
		{
			FAttributeKey DepIndexKey = GetIndexKey(NameIndex);
			FAttributeStorage::TAttributeHandle<ItemType> HandleItem = AttributePtr->GetAttributeHandle<ItemType>(DepIndexKey);
			if (!HandleItem.IsValid())
			{
				continue;
			}
			ItemType& OutItem = OutItems.AddDefaulted_GetRef();
			HandleItem.Get(OutItem);
		}
	}

	bool AddItem(const ItemType& Item)
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			return false;
		}

		if (!AttributePtr->ContainAttribute(GetKeyCount()))
		{
			const int32 DependencyCount = 0;
			EAttributeStorageResult Result = AttributePtr->RegisterAttribute<int32>(GetKeyCount(), DependencyCount);
			if (!IsAttributeStorageResultSuccess(Result))
			{
				LogAttributeStorageErrors(Result, TEXT("TArrayAttributeHelper.AddName"), GetKeyCount());
				return false;
			}
		}
		FAttributeStorage::TAttributeHandle<int32> Handle = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
		if (!ensure(Handle.IsValid()))
		{
			return false;
		}
		int32 ItemIndex = 0;
		Handle.Get(ItemIndex);
		FAttributeKey ItemIndexKey = GetIndexKey(ItemIndex);
		EAttributeStorageResult AddItemResult = AttributePtr->RegisterAttribute<ItemType>(ItemIndexKey, Item);
		if (!IsAttributeStorageResultSuccess(AddItemResult))
		{
			LogAttributeStorageErrors(AddItemResult, TEXT("TArrayAttributeHelper.AddName"), ItemIndexKey);
			return false;
		}

		//Success, increment the item counter
		ItemIndex++;
		Handle.Set(ItemIndex);
		return true;
	}

	bool RemoveItem(const ItemType& ItemToDelete)
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			return false;
		}

		int32 ItemCount = 0;
		if (!AttributePtr->ContainAttribute(GetKeyCount()))
		{
			return false;
		}
		FAttributeStorage::TAttributeHandle<int32> Handle = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
		if (!Handle.IsValid())
		{
			return false;
		}
		Handle.Get(ItemCount);
		int32 DecrementKey = 0;
		for (int32 ItemIndex = 0; ItemIndex < ItemCount; ++ItemIndex)
		{
			FAttributeKey DepIndexKey = GetIndexKey(ItemIndex);
			FAttributeStorage::TAttributeHandle<ItemType> HandleItem = AttributePtr->GetAttributeHandle<ItemType>(DepIndexKey);
			if (!HandleItem.IsValid())
			{
				continue;
			}
			ItemType Item;
			HandleItem.Get(Item);
			//Remove at most one item
			if (DecrementKey == 0 && Item == ItemToDelete)
			{
				//Remove this entry
				AttributePtr->UnregisterAttribute(DepIndexKey);
				Handle.Set(ItemCount - 1);
				//We have to rename the key for all the next item
				DecrementKey++;
			}
			else if (DecrementKey > 0)
			{
				FAttributeKey NewDepIndexKey = GetIndexKey(ItemIndex - DecrementKey);
				EAttributeStorageResult UnregisterResult = AttributePtr->UnregisterAttribute(DepIndexKey);
				if (IsAttributeStorageResultSuccess(UnregisterResult))
				{
					EAttributeStorageResult RegisterResult = AttributePtr->RegisterAttribute<ItemType>(NewDepIndexKey, Item);
					if (!IsAttributeStorageResultSuccess(RegisterResult))
					{
						LogAttributeStorageErrors(RegisterResult, TEXT("TArrayAttributeHelper.RemoveItem"), NewDepIndexKey);
					}
				}
				else
				{
					LogAttributeStorageErrors(UnregisterResult, TEXT("TArrayAttributeHelper.RemoveItem"), DepIndexKey);
				}

				//Avoid doing more code in the for since the HandleItem is now invalid
				continue;
			}
		}
		return true;
	}

	bool RemoveAllItems()
	{
		//The Class must be initialize properly before we can use it
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributePtr = Attributes.Pin();
		if (!ensure(AttributePtr.IsValid()))
		{
			return false;
		}

		int32 ItemCount = 0;
		if (!AttributePtr->ContainAttribute(GetKeyCount()))
		{
			return false;
		}
		FAttributeStorage::TAttributeHandle<int32> HandleCount = AttributePtr->GetAttributeHandle<int32>(GetKeyCount());
		if (!HandleCount.IsValid())
		{
			return false;
		}
		HandleCount.Get(ItemCount);
		//Remove all attribute one by one
		for (int32 NameIndex = 0; NameIndex < ItemCount; ++NameIndex)
		{
			FAttributeKey DepIndexKey = GetIndexKey(NameIndex);
			AttributePtr->UnregisterAttribute(DepIndexKey);
		}
		//Make sure Count is zero
		ItemCount = 0;
		HandleCount.Set(ItemCount);
		return true;
	}

private:
	TWeakPtr<FAttributeStorage, ESPMode::ThreadSafe> Attributes = nullptr;
	FAttributeKey KeyCount;
	FAttributeKey GetKeyCount() const
	{
		ensure(!KeyCount.Key.IsEmpty()); return KeyCount;
	}

	FAttributeKey GetIndexKey(int32 Index) const
	{
		return UE::Interchange::BuildArrayKeyFromIndex(GetKeyCount().ToString(), Index);
	}
};

/**
 * Helper class to manage a TMap where the key and value type can be any type support by the FAttributeStorage
 */
template<typename KeyType, typename ValueType>
class TMapAttributeHelper
{
	template<typename T>
	using TAttributeHandle = FAttributeStorage::TAttributeHandle<T>;

public:
	void Initialize(const TSharedRef<FAttributeStorage, ESPMode::ThreadSafe>& InAttributes, const FString& BaseKeyName)
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		Attributes = InAttributes;
		FString BaseTryName = BaseKeyName;
		FAttributeKey KeyCountKey(MoveTemp(BaseTryName));
		if (!InAttributes->ContainAttribute(KeyCountKey))
		{
			EAttributeStorageResult Result = InAttributes->RegisterAttribute<int32>(KeyCountKey, 0);
			check(Result == EAttributeStorageResult::Operation_Success);
			KeyCountHandle = InAttributes->GetAttributeHandle<int32>(KeyCountKey);
		}
		else
		{
			KeyCountHandle = InAttributes->GetAttributeHandle<int32>(KeyCountKey);
			RebuildCache();
		}
	}

	TMapAttributeHelper() = default;
	TMapAttributeHelper(const TMapAttributeHelper&) = default;
	TMapAttributeHelper(TMapAttributeHelper&&) = default;
	TMapAttributeHelper& operator=(const TMapAttributeHelper&) = default;
	TMapAttributeHelper& operator=(TMapAttributeHelper&&) = default;

	~TMapAttributeHelper()
	{
		Attributes.Reset();
	}

	bool SetKeyValue(const KeyType& InKey, const ValueType& InValue)
	{
		const uint32 Hash = GetTypeHash(InKey);
		return SetKeyValueByHash(Hash, InKey, InValue);
	}

	bool GetValue(const KeyType& InKey, ValueType& OutValue) const
	{
		const uint32 Hash = GetTypeHash(InKey);
		return GetValueByHash(Hash, InKey, OutValue);
	}

	bool RemoveKey(const KeyType& InKey)
	{
		const uint32 Hash = GetTypeHash(InKey);
		return RemoveKeyByHash(Hash, InKey);
	}

	bool RemoveKeyAndGetValue(const KeyType& InKey, ValueType& OutValue)
	{
		const uint32 Hash = GetTypeHash(InKey);
		return RemoveKeyAndGetValueByHash(Hash, InKey, OutValue);
	}

	bool SetKeyValueByHash(uint32 Hash, const KeyType& InKey, const ValueType& InValue)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		if (TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>* Pair = CachedKeysAndValues.FindByHash(Hash, InKey))
		{
			Pair->Value.Set(InValue);
		}
		else
		{
			FAttributeKey IndexKey = GetKeyAttribute(CachedKeysAndValues.Num());
			FAttributeKey AttributeKey = GetValueAttribute(InKey);
			ensure(AttributesPtr->RegisterAttribute<KeyType>(IndexKey, InKey) == EAttributeStorageResult::Operation_Success);
			ensure(AttributesPtr->RegisterAttribute<ValueType>(AttributeKey, InValue) == EAttributeStorageResult::Operation_Success);
			CachedKeysAndValues.AddByHash(
				Hash
				, InKey
				, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>(
					AttributesPtr->GetAttributeHandle<KeyType>(IndexKey)
					, AttributesPtr->GetAttributeHandle<ValueType>(AttributeKey)
					));
			KeyCountHandle.Set(CachedKeysAndValues.Num());
		}

		return true;
	}


	bool GetValueByHash(uint32 Hash, const KeyType& InKey, ValueType& OutValue) const
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		if (const TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>* Pair = CachedKeysAndValues.FindByHash(Hash, InKey))
		{
			return Pair->Value.Get(OutValue) == EAttributeStorageResult::Operation_Success;
		}

		return false;
	}


	bool RemoveKeyByHash(uint32 Hash, const KeyType& InKey)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		if (TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>* Pair = CachedKeysAndValues.FindByHash(Hash, InKey))
		{
			return RemoveBySwap(Hash, InKey, *Pair);
		}

		return false;
	}

	bool RemoveKeyAndGetValueByHash(uint32 Hash, const KeyType& InKey, ValueType& OutValue)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		if (TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>* Pair = CachedKeysAndValues.FindByHash(Hash, InKey))
		{
			if (Pair->Value.Get(OutValue) == EAttributeStorageResult::Operation_Success)
			{
				return RemoveBySwap(Hash, InKey, *Pair);
			}
		}

		return false;
	}

	void Reserve(int32 Number)
	{
		CachedKeysAndValues.Reserve(Number);
	}

	void Empty(int32 NumOfExpectedElements = 0)
	{
		EmptyInternal(NumOfExpectedElements);
	}

	int32 GetCount() const
	{
		return CachedKeysAndValues.Num();
	}

	TMapAttributeHelper& operator=(const TMap<KeyType, ValueType>& InMap)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return *this;
		}

		// Empty
		EmptyInternal(InMap.Num());

		KeyCountHandle.Set(InMap.Num());

		for (const TPair<KeyType, ValueType>& Pair : InMap)
		{
			SetKeyValue(Pair.Key, Pair.Value);
		}

		return *this;
	}

	TMap<KeyType, ValueType> ToMap() const
	{
		TMap<KeyType, ValueType> Map;
		Map.Reserve(CachedKeysAndValues.Num());

		for (const TPair<KeyType, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>>& Pair : CachedKeysAndValues)
		{
			KeyType Key;
			Pair.Value.Key.Get(Key);
			ValueType Value;
			Pair.Value.Value.Get(Value);

			Map.Add(MoveTemp(Key), MoveTemp(Value));
		}

		return Map;
	}

	void RebuildCache()
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (AttributesPtr.IsValid() && KeyCountHandle.IsValid())
		{
			int32 KeyCount;
			if (KeyCountHandle.Get(KeyCount) == EAttributeStorageResult::Operation_Success)
			{
				CachedKeysAndValues.Empty(KeyCount);

				for (int32 Index = 0; Index < KeyCount; Index++)
				{
					TAttributeHandle<KeyType> KeyAttribute = AttributesPtr->GetAttributeHandle<KeyType>(GetKeyAttribute(Index));
					if (!ensure(KeyAttribute.IsValid()))
					{
						continue;
					}

					KeyType Key;
					KeyAttribute.Get(Key);

					TAttributeHandle<ValueType> ValueAttribute = AttributesPtr->GetAttributeHandle<ValueType>(GetValueAttribute(Key));
					if (!ensure(ValueAttribute.IsValid()))
					{
						continue;
					}

					CachedKeysAndValues.Add(Key, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>(KeyAttribute, ValueAttribute));
				}
			}
		}
	}

private:
	FAttributeKey GetKeyAttribute(int32 Index) const
	{
		static const FString KeyIndex = "_KeyIndex_";
		FString IndexedKey = KeyCountHandle.GetKey().ToString();
		IndexedKey.Reserve(KeyIndex.Len() + 16 /*Max size for a int32*/);
		IndexedKey.Append(KeyIndex);
		IndexedKey.AppendInt(Index);
		return FAttributeKey(MoveTemp(IndexedKey));
	}

	FAttributeKey GetValueAttribute(const KeyType& InKey) const
	{
		static const FString KeyChars = "_Key_";
		FString ValueAttribute = KeyCountHandle.GetKey().ToString();
		FString Key = TTypeToString<KeyType>::ToString(InKey);
		ValueAttribute.Reserve(KeyChars.Len() + Key.Len());
		ValueAttribute.Append(KeyChars);
		ValueAttribute.Append(Key);
		return FAttributeKey(MoveTemp(ValueAttribute));
	}

	TAttributeHandle<KeyType> GetLastKeyAttributeHandle() const
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return {};
		}
		return AttributesPtr->GetAttributeHandle<KeyType>(GetKeyAttribute(CachedKeysAndValues.Num() - 1));
	}

	bool RemoveBySwap(uint32 Hash, const KeyType& InKey, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>& CachedPair)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		TAttributeHandle<KeyType> LastKeyIndex = GetLastKeyAttributeHandle();
		KeyType LastKey;
		LastKeyIndex.Get(LastKey);
		CachedKeysAndValues[LastKey].Key = CachedPair.Key;
		CachedPair.Key.Set(MoveTemp(LastKey));

		AttributesPtr->UnregisterAttribute(LastKeyIndex.GetKey());
		AttributesPtr->UnregisterAttribute(CachedPair.Value.GetKey());
		CachedKeysAndValues.RemoveByHash(Hash, InKey);
		KeyCountHandle.Set(CachedKeysAndValues.Num());

		return true;
	}

	void EmptyInternal(int32 NumOfExpectedElements)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return;
		}

		for (const TPair<KeyType, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>>& Pair : CachedKeysAndValues)
		{
			const FAttributeKey& KeyAttribute = Pair.Value.Key.GetKey();
			AttributesPtr->UnregisterAttribute(KeyAttribute);

			const FAttributeKey& ValueAttribute = Pair.Value.Value.GetKey();
			AttributesPtr->UnregisterAttribute(ValueAttribute);
		}
		CachedKeysAndValues.Empty(NumOfExpectedElements);
		KeyCountHandle.Set(0);
	}

	TMap<KeyType, TPair<TAttributeHandle<KeyType>, TAttributeHandle<ValueType>>> CachedKeysAndValues;
	TAttributeHandle<int32> KeyCountHandle; //Assign in Initialize function, it will ensure if its the default value (NAME_None) when using the class
	TWeakPtr<FAttributeStorage, ESPMode::ThreadSafe> Attributes = nullptr;
};


/**
 * Helper class to manage a TSet where the value type can be any type support by the FAttributeStorage
 * This class can also load storage made for the TArrayAttributeHelper but will ignore duplicate values
 */
template<typename ValueType>
class TSetAttributeHelper
{
	template<typename T>
	using TAttributeHandle = FAttributeStorage::TAttributeHandle<T>;

public:
	void Initialize(const TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe>& InAttributes, const FString& BaseKeyName)
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		Attributes = InAttributes;
		KeyCount = BaseKeyName;

		if (!InAttributes->ContainAttribute(KeyCount))
		{
			EAttributeStorageResult Result = InAttributes->RegisterAttribute<int32>(KeyCount, 0);
			check(Result == EAttributeStorageResult::Operation_Success);
			KeyCountHandle = InAttributes->GetAttributeHandle<int32>(KeyCount);
		}
		else
		{
			KeyCountHandle = InAttributes->GetAttributeHandle<int32>(KeyCount);
			RebuildCache();
		}
	}

	TSetAttributeHelper() = default;
	TSetAttributeHelper(const TSetAttributeHelper&) = default;
	TSetAttributeHelper(TSetAttributeHelper&&) = default;
	TSetAttributeHelper& operator=(const TSetAttributeHelper&) = default;
	TSetAttributeHelper& operator=(TSetAttributeHelper&&) = default;

	~TSetAttributeHelper()
	{
		Attributes.Reset();
	}

	/*
	* Adds an item. Returns true if the item is added successfully.
	* Returns false if the item is already present or if Initialize was not properly called before adding an item.
	*/
	bool AddItem(const ValueType& InValue)
	{
		if (!CachedValues.Contains(InValue))
		{
			TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
			if (!ensure(AttributesPtr.IsValid()))
			{
				return false;
			}

			FAttributeKey AttributeKey = GetIndexKey(CachedValues.Num());
			ensure(AttributesPtr->RegisterAttribute<ValueType>(AttributeKey, InValue) == EAttributeStorageResult::Operation_Success);
			CachedValues.Add(InValue, AttributeKey);
			KeyCountHandle.Set(CachedValues.Num());

			return true;
		}

		return false;
	}

	bool HasItem(const ValueType& InValue) const
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		return CachedValues.Contains(InValue);
	}

	/*
	* Remove an item if it exists. Will realign attributes by indices in the storage if needed (0(N)).
	*/
	bool RemoveItem(const ValueType& InValue)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return false;
		}

		{
			FAttributeKey* KeyToRemove = CachedValues.Find(InValue);
			if (!KeyToRemove)
			{
				return false;
			}

			AttributesPtr->UnregisterAttribute(*KeyToRemove);
			CachedValues.Remove(InValue);
			KeyCountHandle.Set(CachedValues.Num());
		}

		//When removing an item, we need to re-align the keys by indices without having gaps so it can be loaded properly later.
		uint32 Index = 0;
		for (const TPair<ValueType, FAttributeKey>& Pair : CachedValues)
		{
			const FAttributeKey& AttributeKey = Pair.Value;
			const FAttributeKey NewAttributeKey = GetIndexKey(Index++);
			if (NewAttributeKey == AttributeKey)
			{
				//This item is already assigned the correct index.
				continue;
			}

			AttributesPtr->UnregisterAttribute(AttributeKey);
			ensure(AttributesPtr->RegisterAttribute<ValueType>(NewAttributeKey, Pair.Key) == EAttributeStorageResult::Operation_Success);

			CachedValues[Pair.Key] = NewAttributeKey;
		}

		return true;
	}

	void Reserve(int32 Number)
	{
		CachedValues.Reserve(Number);
	}

	void Empty(int32 NumOfExpectedElements = 0)
	{
		EmptyInternal(NumOfExpectedElements);
	}

	int32 GetCount() const
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		ensure(AttributesPtr.IsValid());
		return CachedValues.Num();
	}

	TSetAttributeHelper& operator=(const TSet<ValueType>& InSet)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return *this;
		}

		// Empty
		EmptyInternal(InSet.Num());

		KeyCountHandle.Set(InSet.Num());

		for (const ValueType& Value : InSet)
		{
			AddItem(Value);
		}

		return *this;
	}

	TSet<ValueType> ToSet() const
	{
		TSet<ValueType> Set;

		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return Set;
		}
		
		CachedValues.GetKeys(Set);

		return Set;
	}

	void RebuildCache()
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));

		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (AttributesPtr.IsValid() && KeyCountHandle.IsValid())
		{
			int32 ItemCount;
			if (KeyCountHandle.Get(ItemCount) == EAttributeStorageResult::Operation_Success)
			{
				CachedValues.Empty(ItemCount);

				for (int32 Index = 0; Index < ItemCount; Index++)
				{
					TAttributeHandle<ValueType> ValueAttribute = AttributesPtr->GetAttributeHandle<ValueType>(GetIndexKey(Index));
					if (!ValueAttribute.IsValid())
					{
						continue;
					}

					ValueType Value;
					if (ValueAttribute.Get(Value) == EAttributeStorageResult::Operation_Success)
					{
						CachedValues.Add(Value, ValueAttribute.GetKey());
					}
				}
			}
		}
	}

private:
	void EmptyInternal(int32 NumOfExpectedElements)
	{
		TSharedPtr<FAttributeStorage, ESPMode::ThreadSafe> AttributesPtr = Attributes.Pin();
		if (!ensure(AttributesPtr.IsValid()))
		{
			return;
		}

		for (const TPair<ValueType, FAttributeKey>& Pair : CachedValues)
		{
			const FAttributeKey& KeyAttribute = Pair.Value;
			AttributesPtr->UnregisterAttribute(KeyAttribute);
		}

		CachedValues.Empty(NumOfExpectedElements);
		KeyCountHandle.Set(0);
	}

	TMap<ValueType, FAttributeKey> CachedValues;
	TAttributeHandle<int32> KeyCountHandle;
	TWeakPtr<FAttributeStorage, ESPMode::ThreadSafe> Attributes = nullptr;

	FAttributeKey KeyCount;
	FAttributeKey GetKeyCount() const
	{
		ensure(!KeyCount.Key.IsEmpty()); return KeyCount;
	}

	FAttributeKey GetIndexKey(int32 Index) const
	{
		ensure(!KeyCount.Key.IsEmpty());
		return UE::Interchange::BuildArrayKeyFromIndex(KeyCount.Key, Index); //Must be identical to the TArrayAttributeHelper
	}
};

} //ns Interchange
} //ns UE
