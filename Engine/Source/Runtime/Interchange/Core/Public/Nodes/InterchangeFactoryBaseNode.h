// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "HAL/Platform.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/SoftObjectPath.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeFactoryBaseNode.generated.h"

class UObject;
struct FFrame;

#define IMPLEMENT_NODE_ATTRIBUTE_DELEGATE_BY_PROPERTYNAME(AttributeName, AttributeType, ObjectType, PropertyName)		\
	bool ApplyCustom##AttributeName##ToAsset(UObject* Asset) const														\
	{																													\
		return ApplyAttributeToObject<AttributeType>(Macro_Custom##AttributeName##Key.Key, Asset, PropertyName);		\
	}																													\
																														\
	bool FillCustom##AttributeName##FromAsset(UObject* Asset)															\
	{																													\
		return FillAttributeFromObject<AttributeType>(Macro_Custom##AttributeName##Key.Key, Asset, PropertyName);		\
	}

#define IMPLEMENT_NODE_ATTRIBUTE_SETTER(NodeClassName, AttributeName, AttributeType, AssetType)														\
	FString OperationName = GetTypeName() + TEXT(".Set" #AttributeName);																			\
	if(InterchangePrivateNodeBase::SetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, OperationName, AttributeValue))	\
	{																																				\
		if(bAddApplyDelegate)																														\
		{																																			\
			FName AttributeNameImpl = TEXT(""#AttributeName);																						\
			AttributesAppliedThroughDelegates.AddItem(Macro_Custom##AttributeName##Key.Key);														\
			AddApplyAndFillDelegates<AttributeType>(Macro_Custom##AttributeName##Key.Key, AssetType::StaticClass(), AttributeNameImpl);				\
		}																																			\
		return true;																																\
	}																																				\
	return false;

/*
 * Use this macro to implement a custom node attribute setter that will add a delegate to apply the node attribute value to any UObject derived from
 * the specified type class. Use this macro if the type class is different from the node's GetObjectCLass().
 *
 * The node should derive from UInterchangeFactoryBaseNode to apply a delegate.
 * The node needs to implement 2 functions for the attribute:
 *  - ApplyCustom##AttributeName##ToAsset(UObject*): see the UE::Interchange::FApplyAttributeToAsset delegate signature.
 *  - FillCustom##AttributeName##FromAsset(UObject*): see the UE::Interchange::FFillAttributeToAsset delegate signature.
 */
#define IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS(NodeClassName, AttributeName, AttributeType, AssetType)				\
	FString OperationName = GetTypeName() + TEXT(".Set" #AttributeName);																			\
	if(InterchangePrivateNodeBase::SetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, OperationName, AttributeValue))	\
	{																																				\
		if(bAddApplyDelegate)																														\
		{																																			\
			check(NodeClassName::StaticClass()->IsChildOf<UInterchangeFactoryBaseNode>());															\
			AttributesAppliedThroughDelegates.AddItem(Macro_Custom##AttributeName##Key.Key);														\
			UClass* AssetClass = AssetType::StaticClass();																							\
			TArray<UE::Interchange::FApplyAttributeToAsset>& Delegates = ApplyCustomAttributeDelegates.FindOrAdd(AssetClass);						\
			Delegates.Add(UE::Interchange::FApplyAttributeToAsset::CreateUObject(this, &NodeClassName::ApplyCustom##AttributeName##ToAsset));		\
			TArray<UE::Interchange::FFillAttributeToAsset>& FillDelegates = FillCustomAttributeDelegates.FindOrAdd(AssetClass);						\
			FillDelegates.Add(UE::Interchange::FFillAttributeToAsset::CreateUObject(this, &NodeClassName::FillCustom##AttributeName##FromAsset));	\
		}																																			\
		return true;																																\
	}																																				\
	return false;

/*
 * This macro uses the factory node GetObjectClass() to register the delegate instead of receiving a TypeClass.
 * See the documentation of IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE_WITH_CUSTOM_CLASS.
 */
#define IMPLEMENT_NODE_ATTRIBUTE_SETTER_WITH_CUSTOM_DELEGATE(NodeClassName, AttributeName, AttributeType)											\
	FString OperationName = GetTypeName() + TEXT(".Set" #AttributeName);																			\
	if(InterchangePrivateNodeBase::SetCustomAttribute<AttributeType>(*Attributes, Macro_Custom##AttributeName##Key, OperationName, AttributeValue))	\
	{																																				\
		if(bAddApplyDelegate)																														\
		{																																			\
			check(NodeClassName::StaticClass()->IsChildOf<UInterchangeFactoryBaseNode>());															\
			AttributesAppliedThroughDelegates.AddItem(Macro_Custom##AttributeName##Key.Key);														\
			TArray<UE::Interchange::FApplyAttributeToAsset>& Delegates = ApplyCustomAttributeDelegates.FindOrAdd(GetObjectClass());					\
			Delegates.Add(UE::Interchange::FApplyAttributeToAsset::CreateUObject(this, &NodeClassName::ApplyCustom##AttributeName##ToAsset));		\
			TArray<UE::Interchange::FFillAttributeToAsset>& FillDelegates = FillCustomAttributeDelegates.FindOrAdd(GetObjectClass());				\
			FillDelegates.Add(UE::Interchange::FFillAttributeToAsset::CreateUObject(this, &NodeClassName::FillCustom##AttributeName##FromAsset));	\
		}																																			\
		return true;																																\
	}																																				\
	return false;


/*
 * This macro forces the addition of the Fill and Apply delegates on 'this'
 * if the designated attribute has been set on the source node.
 * Copying the arrays of delegates from the source node to 'this' is not sufficient
 * because the delegates of the source node have a reference on the source node not 'this'.
 * Consequently, a call to those delegates would always use the source node.
 */
#define COPY_NODE_DELEGATES(SourceNode, AttributeName, AttributeType, AssetType)															\
{																																			\
	const UE::Interchange::FAttributeKey& AttributeKey = Macro_Custom##AttributeName##Key;													\
	FString OperationName = GetTypeName() + TEXT(".Get" #AttributeName);																	\
	AttributeType AttributeValue;																											\
	if(InterchangePrivateNodeBase::GetCustomAttribute<AttributeType>(*SourceNode->Attributes, AttributeKey, OperationName, AttributeValue)) \
	{																																		\
		FName PropertyName = TEXT(""#AttributeName);																						\
		AddApplyAndFillDelegates<AttributeType>(AttributeKey.Key, AssetType::StaticClass(), PropertyName);									\
	}																																		\
}

#define COPY_NODE_DELEGATES_WITH_CUSTOM_DELEGATE(SourceNode, NodeClassName, AttributeName, AttributeType, AssetClass)							\
{																																				\
	const UE::Interchange::FAttributeKey& AttributeKey = Macro_Custom##AttributeName##Key;														\
	FString OperationName = GetTypeName() + TEXT(".Get" #AttributeName);																		\
	AttributeType AttributeValue;																												\
	if(InterchangePrivateNodeBase::GetCustomAttribute<AttributeType>(*SourceNode->Attributes, AttributeKey, OperationName, AttributeValue))		\
	{																																			\
		TArray<UE::Interchange::FApplyAttributeToAsset>& Delegates = ApplyCustomAttributeDelegates.FindOrAdd(AssetClass);						\
		Delegates.Add(UE::Interchange::FApplyAttributeToAsset::CreateUObject(this, &NodeClassName::ApplyCustom##AttributeName##ToAsset));		\
		TArray<UE::Interchange::FFillAttributeToAsset>& FillDelegates = FillCustomAttributeDelegates.FindOrAdd(AssetClass);						\
		FillDelegates.Add(UE::Interchange::FFillAttributeToAsset::CreateUObject(this, &NodeClassName::FillCustom##AttributeName##FromAsset));	\
	}																																			\
}

#define REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE(AttributeName, AttributeType)																	\
if(AttributesAppliedThroughDelegatesKeySet.Contains(Macro_Custom##AttributeName##Key.Key))														\
{																																				\
	AttributeType ValueData;																													\
	if(GetCustom##AttributeName(ValueData))																										\
	{																																			\
		constexpr const bool bAddApplyDelegate = true;																							\
		SetCustom##AttributeName(ValueData, bAddApplyDelegate);																					\
	}																																			\
}																																				\

#define REFILL_CUSTOM_ATTRIBUTE_APPLY_DELEGATE_EXPLICIT(AttributeKey, Getter, Setter, AttributeType)																	\
if(AttributesAppliedThroughDelegatesKeySet.Contains(AttributeKey))																				\
{																																				\
	AttributeType ValueData;																													\
	if(Getter(ValueData))																														\
	{																																			\
		constexpr const bool bAddApplyDelegate = true;																							\
		Setter(ValueData, bAddApplyDelegate);																									\
	}																																			\
}


namespace UE::Interchange
{

	DECLARE_DELEGATE_RetVal_OneParam(bool, FApplyAttributeToAsset, UObject*);
	DECLARE_DELEGATE_RetVal_OneParam(bool, FFillAttributeToAsset, UObject*);

	struct FFactoryBaseNodeStaticData : public FBaseNodeStaticData
	{
		static INTERCHANGECORE_API const FString& FactoryDependenciesBaseKey();
		static INTERCHANGECORE_API const FString& AttributesAppliedThroughDelegatesBaseKey();
		static INTERCHANGECORE_API const FAttributeKey& ReimportStrategyFlagsKey();
		static INTERCHANGECORE_API const FAttributeKey& SkipNodeImportKey();
		static INTERCHANGECORE_API const FAttributeKey& ForceNodeReimportKey();
	};

} // namespace UE::Interchange

UENUM()
enum class EReimportStrategyFlags : uint8
{
	//Do not apply any properties when reimporting. Simply change the source data
	ApplyNoProperties,
	//Always apply all properties specified by the pipeline.
	ApplyPipelineProperties,
	//Always apply all properties specified by the pipeline, but leave the properties that were modified in the editor since the last import.
	ApplyEditorChangedProperties UMETA(DisplayName = "Preserve Editor Changed Properties")
};

/**
 * This struct is used to store and retrieve key-value attributes. The attributes are stored in a generic FAttributeStorage that serializes the values in a TArray64<uint8>.
 * See UE::Interchange::EAttributeTypes to know the supported template types.
 * This is an abstract class. This is the base class of the Interchange node graph format; all classes in this format should derive from this class.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeFactoryBaseNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

public:
	INTERCHANGECORE_API UInterchangeFactoryBaseNode();

	virtual void Serialize(FArchive& Ar) override
	{
		Super::Serialize(Ar);

		if (Ar.IsLoading() && bIsInitialized)
		{
			FactoryDependencies.RebuildCache();
		}
	}

#if WITH_EDITOR
	INTERCHANGECORE_API virtual FString GetKeyDisplayName(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	INTERCHANGECORE_API virtual FString GetAttributeCategory(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
	INTERCHANGECORE_API virtual bool ShouldHideAttribute(const UE::Interchange::FAttributeKey& NodeAttributeKey) const override;
#endif

	/**
	 * Return the reimport strategy flags.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API EReimportStrategyFlags GetReimportStrategyFlags() const;

	/**
	 * Change the reimport strategy flags.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetReimportStrategyFlags(const EReimportStrategyFlags& ReimportStrategyFlags);

	/**
	 * Return true if this node should skip the factory import process, or false otherwise.
	 * Nodes can be in a situation where we have to skip the import process because we cannot import the associated asset for multiple reasons. For example:
	 * - An asset can already exist and represents a different type (UClass).
	 * - An asset can already exist and is being compiled.
	 * - An asset can already exist and is being imported by another concurrent import task (such as a user importing multiple files at the same time in the same content folder).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool ShouldSkipNodeImport() const;

	/**
	 * Add the skip node attribute. Use this function to cancel the creation of the Unreal asset. See ShouldSkipNodeImport for more documentation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetSkipNodeImport();

	/**
	 * Remove the skip node attribute. See ShouldSkipNodeImport for more documentation.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool UnsetSkipNodeImport();

	/**
	 * Return the UClass of the object we represent, so we can find the factory/writer.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API virtual class UClass* GetObjectClass() const;

	/**
	 * Return the custom sub-path under PackageBasePath where the assets will be created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetCustomSubPath(FString& AttributeValue) const;

	/**
	 * Set the custom sub-path under PackageBasePath where the assets will be created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetCustomSubPath(const FString& AttributeValue);

	/**
	 * If this node represent a scene asset (actor), return a specific level in which we will create this scene asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	INTERCHANGECORE_API bool GetCustomLevelUid(FString& AttributeValue) const;

	/**
	 * If this node represent a scene asset (actor), you can set a specific level in which we will create this scene asset.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | ActorFactory")
	INTERCHANGECORE_API bool SetCustomLevelUid(const FString& AttributeValue);

	/**
	 * Retrieve the number of factory dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API int32 GetFactoryDependenciesCount() const;

	/**
	 * Retrieve the dependencies for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API TSet<FString> GetFactoryDependencies() const;

	/**
	 * Retrieve one dependency for this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	UE_DEPRECATED(5.8, "Use GetFactoryDependencies or HasFactoryDependency instead.")
	INTERCHANGECORE_API void GetFactoryDependency(const int32 Index, FString& OutDependency) const;

	/**
	 * Returns true if this factory node has a dependency on the factory which Uid is DependencyUid.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool HasFactoryDependency(const FString& DependencyUid) const;

	/**
	 * Add one dependency to this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool AddFactoryDependencyUid(const FString& DependencyUid);

	/**
	 * Remove one dependency from this object.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool RemoveFactoryDependencyUid(const FString& DependencyUid);

	/**
	 * Return the custom ReferenceObject: the UObject this factory node has created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool GetCustomReferenceObject(FSoftObjectPath& AttributeValue) const;

	/**
	 * Set the custom ReferenceObject: the UObject this factory node has created.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetCustomReferenceObject(const FSoftObjectPath& AttributeValue);

	static INTERCHANGECORE_API FString BuildFactoryNodeUid(const FString& TranslatedNodeUid);
	static INTERCHANGECORE_API FString BuildTranslatedNodeUid(const FString& InFactoryNodeUid);

	/**
	 * Adds the delegates that will read and write the attribute value to a UObject.
	 * @param NodeAttributeKey	The key of the attribute for which we are adding the delegates.
	 * @param ObjectClass		The class of the object that we will apply and fill on.
	 * @param PropertyName		The name of the property of the ObjectClass that we will read and write to.
	 */
	template<typename AttributeType>
	inline void AddApplyAndFillDelegates(const FString& NodeAttributeKey, UClass* ObjectClass, const FName PropertyName);

	/**
	 * Writes an attribute value to a UObject.
	 * @param NodeAttributeKey	The key for the attribute to write.
	 * @param Object			The object to write to.
	 * @param PropertyName		The name of the property to write to on the Object.
	 * @result					True if the operation succeeded.
	 */
	template<typename AttributeType>
	inline bool ApplyAttributeToObject(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName) const;

	/**
	 * Reads an attribute value from a UObject.
	 * @param NodeAttributeKey	The key for the attribute to read.
	 * @param Object			The object to read from.
	 * @param PropertyName		The name of the property to read from on the Object.
	 * @result					True if the operation succeeded.
	 */
	template<typename AttributeType>
	inline bool FillAttributeFromObject(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName);

	/**
	 * Copies all the attributes from SourceNode to this node.
	 * @param SourceNode		The source factory node to copy from.
	 * @param Object			The object to build delegates on custom attributes from, if applicable.
	 */
	virtual void CopyWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object)
	{
		// Just copy the attributes storage. Child classes will use Object in their override
		UInterchangeBaseNode::CopyStorage(SourceNode, this);
	}

	/**
	 * Each Attribute that was set and has a delegate set for the specified UObject->UClass will
	 * have the delegate executed so it applies the attribute to the UObject property.
	 * See the macros IMPLEMENT_NODE_ATTRIBUTE_SETTER at the top of the file to know how delegates are setup for properties.
	 */
	INTERCHANGECORE_API void ApplyAllCustomAttributeToObject(UObject* Object) const;

	INTERCHANGECORE_API void FillAllCustomAttributeFromObject(UObject* Object) const;

	/**
	 * Adds apply and fill delegates for attributes again that were using custom delegates.
	 */
	INTERCHANGECORE_API void RestoreAllCustomAttributeDelegates();

	/**
	 * Removes all custom attribute delegates registered for the provided class.
	 * @param Class				The class used to pick which delegates to remove.
	 */
	INTERCHANGECORE_API void RemoveCustomAttributesForClass(UClass* Class);

	/**
	 * Removes all custom attribute delegates that are currently added.
	 */
	INTERCHANGECORE_API void RemoveAllCustomAttributeDelegates();

	/**
	 * Copies all the custom attributes from SourceNode to this node, and
	 * gets the appropriate values from Object.
	 * @param SourceNode		The source factory node to copy from.
	 * @param Object			The object to fill custom attributes' values from, if applicable.
	 */
	static INTERCHANGECORE_API UInterchangeFactoryBaseNode* DuplicateWithObject(const UInterchangeFactoryBaseNode* SourceNode, UObject* Object);

	/**
	 * Return whether or not an object should be created even if it has been deleted in the editor.
	 * Return false by default.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool ShouldForceNodeReimport() const;

	/**
	 * Allow the creation of the Unreal object even if it has been previously deleted in the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool SetForceNodeReimport();

	/**
	 * Disallow the creation of the Unreal object if it has been previously deleted in the editor.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	INTERCHANGECORE_API bool UnsetForceNodeReimport();

	/**
	 * Return if the import of the class is allowed at runtime.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node")
	virtual bool IsRuntimeImportAllowed() const
	{
		return true;
	}

protected:
	/**
	 * Adds apply and fill delegates for attributes again that were using custom delegates.
	 */
	INTERCHANGECORE_API virtual void OnRestoreAllCustomAttributeDelegates();

protected:
	/**
	 * Writes a value to an UObject's property.
	 * @param Value				The value to write.
	 * @param Object			The object to write to.
	 * @param PropertyName		The name of the property to write to on the Object.
	 * @result					True if the operation succeeded.
	 */
	template<typename PropertyType>
	inline bool ApplyPropertyValueToObject(const PropertyType& Value, UObject* Object, const FName PropertyName) const;

	/**
	 * Those dependencies are used by the Interchange parsing task to make sure the assets are created in the correct order.
	 * Examples: The Mesh factory node will have dependencies on the material factory node, and the Material factory node will have dependencies on the texture factory node.
	 */
	UE::Interchange::TSetAttributeHelper<FString> FactoryDependencies;

	/* This array holds the delegate to apply the attribute that has to be set on an UObject. */
	TMap<UClass*, TArray<UE::Interchange::FApplyAttributeToAsset>> ApplyCustomAttributeDelegates;

	TMap<UClass*, TArray<UE::Interchange::FFillAttributeToAsset>> FillCustomAttributeDelegates;

	UE::Interchange::TArrayAttributeHelper<FString> AttributesAppliedThroughDelegates;

	UPROPERTY(Transient)
	TSet<FString> AttributesAppliedThroughDelegatesKeySet;

private:

	const UE::Interchange::FAttributeKey Macro_CustomSubPathKey = UE::Interchange::FAttributeKey(TEXT("SubPath"));
	const UE::Interchange::FAttributeKey Macro_CustomLevelUidKey = UE::Interchange::FAttributeKey(TEXT("LevelUid"));
	const UE::Interchange::FAttributeKey Macro_CustomReferenceObjectKey = UE::Interchange::FAttributeKey(TEXT("ReferenceObject"));
};

template<typename AttributeType>
inline void UInterchangeFactoryBaseNode::AddApplyAndFillDelegates(const FString& NodeAttributeKey, UClass* ObjectClass, const FName PropertyName)
{
	TArray<UE::Interchange::FApplyAttributeToAsset>& ApplyDelegates = ApplyCustomAttributeDelegates.FindOrAdd(ObjectClass);
	ApplyDelegates.Add(
		UE::Interchange::FApplyAttributeToAsset::CreateLambda(
			[NodeAttributeKey, PropertyName, this](UObject* Asset)
			{
				return ApplyAttributeToObject<AttributeType>(NodeAttributeKey, Asset, PropertyName);
			})
	);

	TArray<UE::Interchange::FFillAttributeToAsset>& FillDelegates = FillCustomAttributeDelegates.FindOrAdd(ObjectClass);
	FillDelegates.Add(
		UE::Interchange::FApplyAttributeToAsset::CreateLambda(
			[NodeAttributeKey, PropertyName, this](UObject* Asset)
			{
				return FillAttributeFromObject<AttributeType>(NodeAttributeKey, Asset, PropertyName);
			})
	);
}

template<typename AttributeType>
inline bool UInterchangeFactoryBaseNode::ApplyAttributeToObject(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName) const
{
	if (!Object)
	{
		return false;
	}

	AttributeType ValueData;
	if (GetAttribute<AttributeType>(NodeAttributeKey, ValueData))
	{
		return ApplyPropertyValueToObject(ValueData, Object, PropertyName);
	}

	return false;
}

template<typename AttributeType>
inline bool UInterchangeFactoryBaseNode::FillAttributeFromObject(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName)
{
	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		AttributeType* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<AttributeType>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<AttributeType>(Container.Get<uint8*>());
		}

		// Support for bitfields
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			const bool bPropertyValue = BoolProperty->GetPropertyValue(PropertyValueAddress);
			return SetAttribute(NodeAttributeKey, bPropertyValue);
		}
		else
		{
			return SetAttribute(NodeAttributeKey, *PropertyValueAddress);
		}
	}

	return false;
}

/**
 * Specialized version of FillAttributeFromObject for strings.
 * If the target property is an FObjectPropertyBase, treat the string as an object path.
 */
template<>
inline bool UInterchangeFactoryBaseNode::FillAttributeFromObject<FString>(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName)
{
	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		FString* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<FString>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<FString>(Container.Get<uint8*>());
		}

		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			UObject* ObjectValue = ObjectProperty->GetObjectPropertyValue(PropertyValueAddress);
			return SetAttribute(NodeAttributeKey, ObjectValue->GetPathName());
		}
		else
		{
			return SetAttribute(NodeAttributeKey, *PropertyValueAddress);
		}
	}

	return false;
}

/**
 * Specialized version of FillAttributeFromObject for bools.
 * If the target property is an FBoolProperty, treat the property as a bitfield.
 */
template<>
inline bool UInterchangeFactoryBaseNode::FillAttributeFromObject<bool>(const FString& NodeAttributeKey, UObject* Object, const FName PropertyName)
{
	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		bool* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<bool>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<bool>(Container.Get<uint8*>());
		}

		// Support for bitfields
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			const bool bPropertyValue = BoolProperty->GetPropertyValue(PropertyValueAddress);
			return SetAttribute(NodeAttributeKey, bPropertyValue);
		}
		else
		{
			return SetAttribute(NodeAttributeKey, *PropertyValueAddress);
		}
	}

	return false;
}

template<typename PropertyType>
inline bool UInterchangeFactoryBaseNode::ApplyPropertyValueToObject(const PropertyType& Value, UObject* Object, const FName PropertyName) const
{
	if (!Object)
	{
		return false;
	}

	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		PropertyType* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<PropertyType>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<PropertyType>(Container.Get<uint8*>());
		}

		*PropertyValueAddress = Value;

		return true;
	}

	return false;
}

/**
 * Specialized version of ApplyPropertyValueToObject for strings.
 * If the target property is an FObjectPropertyBase, treat the string as an object path.
 */
template<>
inline bool UInterchangeFactoryBaseNode::ApplyPropertyValueToObject<FString>(const FString& Value, UObject* Object, const FName PropertyName) const
{
	if (!Object)
	{
		return false;
	}

	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		FString* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<FString>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<FString>(Container.Get<uint8*>());
		}

		if (FObjectPropertyBase* ObjectProperty = CastField<FObjectPropertyBase>(Property))
		{
			ObjectProperty->SetObjectPropertyValue(PropertyValueAddress, FSoftObjectPath(Value).TryLoad());
		}
		else
		{
			*PropertyValueAddress = Value;
		}

		return true;
	}
	return false;
}

/**
 * Specialized version of ApplyPropertyValueToObject for bools.
 * If the target property is an FBoolProperty, treat the property as a bitfield.
 */
template<>
inline bool UInterchangeFactoryBaseNode::ApplyPropertyValueToObject<bool>(const bool& Value, UObject* Object, const FName PropertyName) const
{
	if (!Object)
	{
		return false;
	}

	TVariant<UObject*, uint8*> Container;
	Container.Set<UObject*>(Object);
	if (FProperty* Property = InterchangePrivateNodeBase::FindPropertyByPathChecked(Container, Object->GetClass(), PropertyName.ToString()))
	{
		bool* PropertyValueAddress;
		if (Container.IsType<UObject*>())
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<bool>(Container.Get<UObject*>());
		}
		else
		{
			PropertyValueAddress = Property->ContainerPtrToValuePtr<bool>(Container.Get<uint8*>());
		}

		// Support for bitfields
		if (FBoolProperty* BoolProperty = CastField<FBoolProperty>(Property))
		{
			BoolProperty->SetPropertyValue(PropertyValueAddress, Value);
		}
		else
		{
			*PropertyValueAddress = Value;
		}

		return true;
	}
	return false;
}
