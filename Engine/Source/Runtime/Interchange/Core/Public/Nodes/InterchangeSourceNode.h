// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeUtilities.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeSourceNode.generated.h"

class UInterchangeBaseNodeContainer;
class UObject;
struct FFrame;

namespace UE::Interchange
{

	struct FSourceNodeExtraInfoStaticData
	{
		static INTERCHANGECORE_API const FString& GetApplicationVendorExtraInfoKey();
		static INTERCHANGECORE_API const FString& GetApplicationNameExtraInfoKey();
		static INTERCHANGECORE_API const FString& GetApplicationVersionExtraInfoKey();
	};
}

/**
 * This class allows a translator to add general source data that describes the whole source. Pipelines can use this information.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeSourceNode : public UInterchangeBaseNode
{
	GENERATED_BODY()

	UInterchangeSourceNode();

public:
	/**
	 * Initialize the base data of the node. Adds it to NodeContainer.
	 * @param UniqueID - The unique ID for this node.
	 * @param DisplayLabel - The name of the node.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | source")
	INTERCHANGECORE_API void InitializeSourceNode(const FString& UniqueID, const FString& DisplayLabel, UInterchangeBaseNodeContainer* NodeContainer);

	/**
	 * Return the node type name of the class. This is used when reporting errors.
	 */
	INTERCHANGECORE_API virtual FString GetTypeName() const override;

	/* Translators that want to modify the common data should ensure they create the unique common pipeline node. */
	static INTERCHANGECORE_API UInterchangeSourceNode* FindOrCreateUniqueInstance(UInterchangeBaseNodeContainer* NodeContainer);

	/* This function should be use by pipelines to avoid creating a node. If the unique instance doesn't exist, returns nullptr. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | source")
	static INTERCHANGECORE_API const UInterchangeSourceNode* GetUniqueInstance(const UInterchangeBaseNodeContainer* NodeContainer);

	/** Query the source frame rate numerator. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomSourceFrameRateNumerator(int32& AttributeValue) const;

	/** Set the source frame rate numerator. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomSourceFrameRateNumerator(const int32& AttributeValue);

	/** Query the source frame rate denominator. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomSourceFrameRateDenominator(int32& AttributeValue) const;

	/** Set the source frame rate denominator. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomSourceFrameRateDenominator(const int32& AttributeValue);

	/** Query the start of the source timeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::GetCustomSourceTimelineAnimationStartTime")
	INTERCHANGECORE_API bool GetCustomSourceTimelineStart(double& AttributeValue) const;

	/** Set the start of the source timeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::SetCustomSourceTimelineAnimationStartTime")
	INTERCHANGECORE_API bool SetCustomSourceTimelineStart(const double& AttributeValue);

	/** Query the end of the source timeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::GetCustomSourceTimelineAnimationStopTime")
	INTERCHANGECORE_API bool GetCustomSourceTimelineEnd(double& AttributeValue) const;

	/** Set the end of the source timeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::SetCustomSourceTimelineAnimationStopTime")
	INTERCHANGECORE_API bool SetCustomSourceTimelineEnd(const double& AttributeValue);

	/** Query the start of the source animated time. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::GetCustomAnimationStartTime")
	INTERCHANGECORE_API bool GetCustomAnimatedTimeStart(double& AttributeValue) const;

	/** Set the start of the source animated time. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::SetCustomAnimationStartTime")
	INTERCHANGECORE_API bool SetCustomAnimatedTimeStart(const double& AttributeValue);

	/** Query the end of the source animated time. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::GetCustomAnimationStopTime")
	INTERCHANGECORE_API bool GetCustomAnimatedTimeEnd(double& AttributeValue) const;

	/** Set the end of the source animated time. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	UE_DEPRECATED(5.6, "Please, use UInterchangeSkeletalAnimationTrackNode::SetCustomAnimationStopTime")
	INTERCHANGECORE_API bool SetCustomAnimatedTimeEnd(const double& AttributeValue);

	/** Query whether to import materials that aren't used. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomImportUnusedMaterial(bool& AttributeValue) const;

	/** Set whether to import materials that aren't used. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomImportUnusedMaterial(const bool& AttributeValue);


	/** Set Extra Information that we want to show in the Config Panel (such as File Information). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetExtraInformation(const FString& Name, const FString& Value);

	/** Remove Extra Information that we dont want to show in the Config Panel. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool RemoveExtraInformation(const FString& Name);

	/** Get Extra Information that we want to show in the Config Panel (such as File Information). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API void GetExtraInformation(TMap<FString, FString>& OutExtraInformation) const;


	/** Query Axis Conversion Inverse Transform (Primarily used for Socket transform calculations.). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomAxisConversionInverseTransform(FTransform& AxisConversionInverseTransform) const;

	/** Set the Axis Conversion Inverse Transform (Primarily used for Socket transform calculations.). */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomAxisConversionInverseTransform(const FTransform& AxisConversionInverseTransform);

	/**
	 * Gets a prefix that should be added to factory node SubPath custom attributes.
	 * For example this can contain the imported scene's name, so that we create an additional content folder named
	 * after it to contain the imported assets.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomSubPathPrefix(FString& Prefix) const;

	/** Sets the prefix that should be added to factory node SubPath custom attributes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomSubPathPrefix(const FString& Prefix);

	/**
	 * Gets whether factory nodes for this import should have a suffix named after their asset category added to their
	 * custom sub path attribute. For example, if this is set then imported StaticMesh assets will be placed inside of
	 * an additional content folder named "StaticMeshes".
	 *
	 * Note that this is done automatically for all factory nodes created by the generic assets pipeline, but must be
	 * handled manually by calling FillSubPathFromSourceNode() from InterchangePipelineHelper.h for any factory nodes
	 * that other pipelines may create.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomUseAssetTypeSubPathSuffix(bool& Suffix) const;

	/**
	 * Sets whether factory nodes for this import should have a suffix named after their asset category added to their
	 * custom sub path attribute.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomUseAssetTypeSubPathSuffix(const bool& Suffix);

	/** Get the reimport strategy based on EReimportStrategyFlags */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomReimportStrategyFlags(uint8& StrategyFlag) const;

	/** Set the reimport strategy based on EReimportStrategyFlags */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomReimportStrategyFlags(uint8 StrategyFlag);

	/** Get the value of the front axis to be used when importing skeletal meshes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomSkeletalMeshFrontAxis(uint8& AttributeValue) const;

	/** Set the value of the front axis to be used when importing skeletal meshes. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomSkeletalMeshFrontAxis(uint8 AttributeValue);

	/** Gets the minimum triangle count a mesh needs to have in order to get Nanite enabled for it when bBuildNanite is true on the static mesh pipeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomNaniteTriangleThreshold(int64& MinNumTriangles) const;

	/** Sets the minimum triangle count a mesh needs to have in order to get Nanite enabled for it when bBuildNanite is true on the static mesh pipeline. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomNaniteTriangleThreshold(const int64& MinNumTriangles);

	/** Gets, if the skeleton processing should accept a SceneRoot as a joint (root node), for when import is forced as Skeletal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomAllowSceneRootAsJoint(bool& bAllowSceneRootAsJoint) const;

	/** Sets, if the skeleton processing should accept a SceneRoot as a joint (root node), for when import is forced as Skeletal. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomAllowSceneRootAsJoint(const bool& bAllowSceneRootAsJoint);


	/** 
	* Gets, SkeletalMeshes requires the Scene Transforms to be aligned with the Skeleton. 
	*	If false then Skeleton and SkeletalMesh should be aligned on Asset level already.
	*	(prior to the Scene Composition)
	* For example:
	*		true:=FBX
	*		false:=glTF
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool GetCustomAlignSkeletalMeshInScene(bool& AlignSkeletalMeshInScene) const;

	/** Sets, if SkeletalMeshes requires the Scene Transforms to be aligned with the Skeleton. */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | Source")
	INTERCHANGECORE_API bool SetCustomAlignSkeletalMeshInScene(const bool& AlignSkeletalMeshInScene);

private:
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceFrameRateNumerator);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceFrameRateDenominator);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceTimelineStart);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SourceTimelineEnd);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AnimatedTimeStart);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AnimatedTimeEnd);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ImportUnusedMaterial);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AxisConversionInverseTransform);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SubPathPrefix);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(UseAssetTypeSubPathSuffix);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(ReimportStrategyFlags);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(SkeletalMeshFrontAxis);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(NaniteTriangleThreshold);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AllowSceneRootAsJoint);
	IMPLEMENT_NODE_ATTRIBUTE_KEY(AlignSkeletalMeshInScene);

	// Extra InformationTo show in the Config Panel.
	UE::Interchange::TMapAttributeHelper<FString, FString> ExtraInformation;
};

