// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "HAL/Platform.h"
#include "HAL/PlatformCrt.h"
#include "Misc/AssertionMacros.h"
#include "Misc/Optional.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Types/AttributeStorage.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"

#include "InterchangeUserDefinedAttribute.generated.h"

struct FFrame;
class UInterchangeFactoryBaseNode;

USTRUCT(BlueprintType)
struct FInterchangeUserDefinedAttributeInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Node | UserDefinedAttributeInfo")
	FString Name;

	UE::Interchange::EAttributeTypes Type;

	TOptional<FString> PayloadKey;

	bool RequiresDelegate;
};

/**
 * UInterchangeUserDefinedAttributesAPI is used to store and retrieve user-defined attributes such as DCC node attributes, so that pipelines have access to those attributes.
 * Every user-defined attribute has a name, a value, and an optional AnimationPayloadKey: an FRichCurve that is a float curve.
 * The value type must be supported by the UE::Interchange::EAttributeTypes enumeration.
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangeUserDefinedAttributesAPI : public UObject
{
	GENERATED_BODY()
 
public:

	/**
	 * Create a user-defined attribute with a value and a optional payload key.
	 * param UserDefinedAttributeName - The name of the user-defined attribute.
	 * param Value - The value of the user-defined attribute.
	 * param PayloadKey - The translator payload key to retrieve the FRichCurve animation for this user-defined attribute.
	 * Note - User-defined attributes are the user custom attributes from the DCC translated node (for example, extra attributes in Maya).
	 *        The payload key points to an FRichCurve payload.
	 */
	template<typename ValueType>
	static bool CreateUserDefinedAttribute(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const ValueType& Value, const TOptional<FString>& PayloadKey, bool RequiresDelegate = false);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool CreateUserDefinedAttribute_Boolean(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const bool& Value, const FString& PayloadKey, bool RequiresDelegate = false);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool CreateUserDefinedAttribute_Float(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const float& Value, const FString& PayloadKey, bool RequiresDelegate = false);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool CreateUserDefinedAttribute_Double(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const double& Value, const FString& PayloadKey, bool RequiresDelegate = false);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool CreateUserDefinedAttribute_Int32(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const int32& Value, const FString& PayloadKey, bool RequiresDelegate = false);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool CreateUserDefinedAttribute_FString(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const FString& Value, const FString& PayloadKey, bool RequiresDelegate = false);

	/**
	 * Remove the specified user-defined attribute.
	 * @param UserDefinedAttributeName - The name of the user-defined attribute to remove.
	 * @return - True if the attribute exists and was removed, or if the attribute doesn't exist. Returns false if the attribute exists but could not be removed.
	 * Note - User-defined attributes are the user custom attributes from the DCC translated node (for example, extra attributes in Maya).
	 *        The payload key points to an FRichCurve payload.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool RemoveUserDefinedAttribute(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName);

	/**
	 * Get the value of a user-defined attribute and an optional payload key.
	 * @param UserDefinedAttributeName - The name of the user-defined attribute.
	 * @param OutValue - The value of the user-defined attribute.
	 * @param OutPayloadKey - The translator payload key to retrieve the FRichCurve animation for this user-defined attribute.
	 * Note - User-defined attributes are the user custom attributes from the DCC translated node (for example, extra attributes in Maya).
	 *        The payload key points to an FRichCurve payload.
	 */
	template<typename ValueType>
	static bool GetUserDefinedAttribute(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, ValueType& OutValue, TOptional<FString>& OutPayloadKey);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool GetUserDefinedAttribute_Boolean(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, bool& OutValue, FString& OutPayloadKey);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool GetUserDefinedAttribute_Float(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, float& OutValue, FString& OutPayloadKey);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool GetUserDefinedAttribute_Double(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, double& OutValue, FString& OutPayloadKey);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool GetUserDefinedAttribute_Int32(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, int32& OutValue, FString& OutPayloadKey);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API bool GetUserDefinedAttribute_FString(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, FString& OutValue, FString& OutPayloadKey);

	static INTERCHANGECORE_API TArray<FInterchangeUserDefinedAttributeInfo> GetUserDefinedAttributeInfos(const UInterchangeBaseNode* InterchangeNode);

	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API void GetUserDefinedAttributeInfos(const UInterchangeBaseNode* InterchangeNode, TArray<FInterchangeUserDefinedAttributeInfo>& UserDefinedAttributeInfos);
 
	UFUNCTION(BlueprintCallable, Category = "Interchange | Node | UserDefinedAttribute")
	static INTERCHANGECORE_API void DuplicateAllUserDefinedAttribute(const UInterchangeBaseNode* InterchangeSourceNode, UInterchangeBaseNode* InterchangeDestinationNode, bool bAddSourceNodeName);

	static INTERCHANGECORE_API void AddApplyAndFillDelegatesToFactory(UInterchangeFactoryBaseNode* InterchangeFactoryNode, UClass* ParentClass);

	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyValueKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate);
	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyValueKey(const FString& UserDefinedAttributeName,bool RequiresDelegate);
	
	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyPayloadKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate);
	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyPayloadKey(const FString& UserDefinedAttributeName, bool RequiresDelegate);

private:
	static INTERCHANGECORE_API bool HasAttribute(const UInterchangeBaseNode* InterchangeSourceNode, const FString& InUserDefinedAttributeName, bool GeneratePayloadKey, bool& OutRequiresDelegate);
	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyKey(const FStringView UserDefinedAttributeName, bool RequiresDelegate, bool GeneratePayloadKey = false); 
	static INTERCHANGECORE_API UE::Interchange::FAttributeKey MakeUserDefinedPropertyKey(const FString& UserDefinedAttributeName,bool RequiresDelegate, bool GeneratePayloadKey = false);
	
private:
	static INTERCHANGECORE_API const FString UserDefinedAttributeBaseKey;
	static INTERCHANGECORE_API const FString UserDefinedAttributeValuePostKey;
	static INTERCHANGECORE_API const FString UserDefinedAttributePayLoadPostKey;
	static INTERCHANGECORE_API const FString UserDefinedAttributeDelegateKey;
};

template<typename ValueType>
bool UInterchangeUserDefinedAttributesAPI::CreateUserDefinedAttribute(UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, const ValueType& Value, const TOptional<FString>& PayloadKey, bool RequiresDelegate /*= true*/)
{
	check(InterchangeNode);
	
	UE::Interchange::FAttributeKey UserDefinedValueKey = MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate);
	if (InterchangeNode->HasAttribute(UserDefinedValueKey))
	{
		return false;
	}
	
	UE::Interchange::FAttributeKey UserDefinedPayloadKey = MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate, true);
	if (InterchangeNode->HasAttribute(UserDefinedPayloadKey))
	{
		return false;
	}
	//Add the user defined attribute to TMap
	InterchangeNode->RegisterAttribute<ValueType>(UserDefinedValueKey, Value);
	if (PayloadKey.IsSet())
	{
		InterchangeNode->RegisterAttribute<FString>(UserDefinedPayloadKey, PayloadKey.GetValue());
	}
	return true;
}

template<typename ValueType>
bool UInterchangeUserDefinedAttributesAPI::GetUserDefinedAttribute(const UInterchangeBaseNode* InterchangeNode, const FString& UserDefinedAttributeName, ValueType& OutValue, TOptional<FString>& OutPayloadKey)
{
	check(InterchangeNode);
	bool RequiresDelegate;
	if (!HasAttribute(InterchangeNode, UserDefinedAttributeName, false, RequiresDelegate))
	{
		return false;
	}

	UE::Interchange::FAttributeKey UserDefinedValueKey = MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate);
	if (!InterchangeNode->GetAttribute<ValueType>(UserDefinedValueKey.Key, OutValue))
	{
		return false;
	}

	//Payload is optional
	OutPayloadKey.Reset();
	UE::Interchange::FAttributeKey UserDefinedPayloadKey = MakeUserDefinedPropertyKey(UserDefinedAttributeName, RequiresDelegate, true);
	if (InterchangeNode->HasAttribute(UserDefinedPayloadKey))
	{
		FString PayloadKeyValue;
		if (InterchangeNode->GetAttribute<FString>(UserDefinedPayloadKey.Key, PayloadKeyValue))
		{
			//Set the optional
			OutPayloadKey = PayloadKeyValue;
		}
		else
		{
			return false;
		}

	}
	return true;
}
