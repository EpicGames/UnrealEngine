// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "Misc/PackageName.h"
#include "Misc/Paths.h"

//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		class FPackageUtils
		{
		public:
			static bool IsMapPackageAsset(const FString& ObjectPath)
			{
				FString MapFilePath;
				return FPackageUtils::IsMapPackageAsset(ObjectPath, MapFilePath);
			}

			static bool IsMapPackageAsset(const FString& ObjectPath, FString& MapFilePath)
			{
				const FString PackageName = FPackageUtils::ExtractPackageName(ObjectPath);
				if (PackageName.Len() > 0)
				{
					FString PackagePath;
					if (FPackageName::DoesPackageExist(PackageName, &PackagePath))
					{
						const FString FileExtension = FPaths::GetExtension(PackagePath, true);
						if (FileExtension == FPackageName::GetMapPackageExtension())
						{
							MapFilePath = PackagePath;
							return true;
						}
					}
				}

				return false;
			}

			static FString ExtractPackageName(const FString& ObjectPath)
			{
				// To find the package name in an object path we need to find the path left of the FIRST delimiter.
				// Assets like BSPs, lightmaps etc. can have multiple '.' delimiters.
				const int32 PackageDelimiterPos = ObjectPath.Find(TEXT("."), ESearchCase::CaseSensitive, ESearchDir::FromStart);
				if (PackageDelimiterPos != INDEX_NONE)
				{
					return ObjectPath.Left(PackageDelimiterPos);
				}

				return ObjectPath;
			}
		};
	} //ns Interchange
} //ns UE
