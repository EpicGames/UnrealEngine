// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/Array.h"
#include "Containers/ContainersFwd.h"
#include "Containers/EnumAsByte.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"
#include "HAL/CriticalSection.h"
#include "Internationalization/Text.h"
#include "Math/Box.h"
#include "Math/Box2D.h"
#include "Math/BoxSphereBounds.h"
#include "Math/Color.h"
#include "Math/Float16.h"
#include "Math/IntPoint.h"
#include "Math/IntRect.h"
#include "Math/IntVector.h"
#include "Math/MathFwd.h"
#include "Math/Matrix.h"
#include "Math/OrientedBox.h"
#include "Math/Plane.h"
#include "Math/Quat.h"
#include "Math/RandomStream.h"
#include "Math/Rotator.h"
#include "Math/Sphere.h"
#include "Math/Transform.h"
#include "Math/TwoVectors.h"
#include "Math/Vector.h"
#include "Math/Vector2D.h"
#include "Math/Vector2DHalf.h"
#include "Math/Vector4.h"
#include "Misc/DateTime.h"
#include "Misc/EnumClassFlags.h"
#include "Misc/FrameNumber.h"
#include "Misc/FrameRate.h"
#include "Misc/FrameTime.h"
#include "Misc/Guid.h"
#include "Misc/Timespan.h"
#include "Serialization/Archive.h"
#include "Templates/UnrealTemplate.h"
#include "UObject/NameTypes.h"
#include "UObject/SoftObjectPath.h"

#include "InterchangeLODMeshData.h"

#include "InterchangeCustomVersion.h"

//Interchange namespace
namespace UE
{
	namespace Interchange
	{
		namespace Private
		{
			template <typename T, typename Enable = void>
			struct UE_DEPRECATED(5.7, "No longer used.") TUnderlyingType {};

			template <typename T>
			struct UE_DEPRECATED(5.7, "No longer used.") TUnderlyingType<T, typename std::enable_if<std::is_enum<T>::value>::type>
			{
				using type = typename std::underlying_type<T>::type;
			};
		}

		struct FAttributeKey
		{
			FString Key;

			FAttributeKey() = default;
		
			FAttributeKey(const FAttributeKey& Other) = default;
			FAttributeKey(FAttributeKey&& Other) = default;
			FAttributeKey& operator=(const FAttributeKey&) = default;
			FAttributeKey& operator=(FAttributeKey&&) = default;

			explicit FAttributeKey(const FName& Other)
			{
				Key = Other.ToString();
			}

			explicit FAttributeKey(const FStringView& Other)
			{
				Key = Other;
			}

			explicit FAttributeKey(const FString& Other)
			{
				Key = Other;
			}

			explicit FAttributeKey(FString&& Other)
			{
				Key = MoveTemp(Other);
			}

			explicit FAttributeKey(const FText& Other)
			{
				Key = Other.ToString();
			}

			explicit FAttributeKey(const TCHAR* Other)
			{
				Key = Other;
			}

			inline FAttributeKey& operator=(const FName& Other)
			{
				Key = Other.ToString();
				return *this;
			}

			inline FAttributeKey& operator=(const FString& Other)
			{
				Key = Other;
				return *this;
			}

			inline FAttributeKey& operator=(FString&& Other)
			{
				Key = MoveTemp(Other);
				return *this;
			}

			inline FAttributeKey& operator=(const FText& Other)
			{
				Key = Other.ToString();
				return *this;
			}

			inline FAttributeKey& operator=(const TCHAR* Other)
			{
				Key = Other;
				return *this;
			}
		
			inline bool operator==(const FAttributeKey& Other) const
			{
				return Key.Equals(Other.Key);
			}

			inline bool operator!=(const FAttributeKey& Other) const
			{
				return !Key.Equals(Other.Key);
			}

			inline bool operator<(const FAttributeKey& Other) const
			{
				return Key.Compare(Other.Key) < 0;
			}

			inline bool operator<=(const FAttributeKey& Other) const
			{
				return Key.Compare(Other.Key) <= 0;
			}

			inline bool operator>(const FAttributeKey& Other) const
			{
				return Key.Compare(Other.Key) > 0;
			}

			inline bool operator>=(const FAttributeKey& Other) const
			{
				return Key.Compare(Other.Key) >= 0;
			}

			friend FArchive& operator<<(FArchive& Ar, FAttributeKey& AttributeKey)
			{
				Ar << AttributeKey.Key;
				return Ar;
			}

			inline const FString& ToString() const
			{
				return Key;
			}

			inline FName ToName() const
			{
				return FName(*Key);
			}
		};

		using ::GetTypeHash;
		inline uint32 GetTypeHash(const FAttributeKey& AttributeKey)
		{
			return GetTypeHash(AttributeKey.Key);
		}


		/**
		 * Enumerates the built-in types that can be stored in instances of FAttributeStorage.
		 * We cannot change the value of a type to make sure the serialization of old assets is always working.
		 */
		enum class EAttributeTypes : int32
		{
			None					= 0,

			Bool					= 1,
			Color					= 4,
			DateTime				= 5,
			Double					= 6,
			Enum					= 7,
			Float					= 8,
			Guid					= 9,
			Int8					= 10,
			Int16					= 11,
			Int32					= 12,
			Int64					= 13,
			IntRect					= 14,
			LinearColor				= 15,
			Name					= 16,
			RandomStream			= 17,
			String					= 18,
			Timespan				= 19,
			TwoVectors				= 20,
			UInt8					= 21,
			UInt16					= 22,
			UInt32					= 23,
			UInt64					= 24,
			Vector2d				= 25,
			IntPoint				= 26,
			IntVector				= 27,
			Vector2DHalf			= 28,
			Float16					= 29,
			OrientedBox				= 30,
			FrameNumber				= 31,
			FrameRate				= 32,
			FrameTime				= 33,
			SoftObjectPath			= 34,
			Matrix44f				= 35,
			Matrix44d				= 36,
			Plane4f					= 37,
			Plane4d					= 38,
			Quat4f					= 39,
			Quat4d					= 40,
			Rotator3f				= 41,
			Rotator3d				= 42,
			Transform3f				= 43,
			Transform3d				= 44,
			Vector3f				= 45,
			Vector3d				= 46,
			Vector2f				= 47,
			Vector4f				= 48,
			Vector4d				= 49,
			Box2f					= 50,
			Box2D					= 51,
			Box3f					= 52,
			Box3d					= 53,
			BoxSphereBounds3f		= 54,
			BoxSphereBounds3d		= 55,
			Sphere3f				= 56,
			Sphere3d				= 57,

			BoolArray				= 58,
			ColorArray				= 59,
			DateTimeArray			= 60,
			DoubleArray				= 61,
			EnumArray				= 62,
			FloatArray				= 63,
			GuidArray				= 64,
			Int8Array				= 65,
			Int16Array				= 66,
			Int32Array				= 67,
			Int64Array				= 68,
			IntRectArray			= 69,
			LinearColorArray		= 70,
			NameArray				= 71,
			RandomStreamArray		= 72,
			StringArray				= 73,
			TimespanArray			= 74,
			TwoVectorsArray			= 75,
			ByteArray				= 2,
			ByteArray64				= 3,
			UInt16Array				= 76,
			UInt32Array				= 77,
			UInt64Array				= 78,
			Vector2dArray			= 79,
			IntPointArray			= 80,
			IntVectorArray			= 81,
			Vector2DHalfArray		= 82,
			Float16Array			= 83,
			OrientedBoxArray		= 84,
			FrameNumberArray		= 85,
			FrameRateArray			= 86,
			FrameTimeArray			= 87,
			SoftObjectPathArray		= 88,
			Matrix44fArray			= 89,
			Matrix44dArray			= 90,
			Plane4fArray			= 91,
			Plane4dArray			= 92,
			Quat4fArray				= 93,
			Quat4dArray				= 94,
			Rotator3fArray			= 95,
			Rotator3dArray			= 96,
			Transform3fArray		= 97,
			Transform3dArray		= 98,
			Vector3fArray			= 99,
			Vector3dArray			= 100,
			Vector2fArray			= 101,
			Vector4fArray			= 102,
			Vector4dArray			= 103,
			Box2fArray				= 104,
			Box2DArray				= 105,
			Box3fArray				= 106,
			Box3dArray				= 107,
			BoxSphereBounds3fArray 	= 108,
			BoxSphereBounds3dArray 	= 109,
			Sphere3fArray			= 110,
			Sphere3dArray 			= 111,

			InterchangeLODMeshData			= 112,
			InterchangeLODMeshDataArray		= 113,

			//Max should always be updated if we add a new supported type
			Max                 	= 114
		};

		/**
		 * Return the FString for the specified AttributeType.
		 */
		INTERCHANGECORE_API FString AttributeTypeToString(EAttributeTypes AttributeType);

		/**
		 * Return the AttributeType for the specified FString, or return EAttributeTypes::None if the string does not match any
		 * supported attribute type.
		 */
		INTERCHANGECORE_API EAttributeTypes StringToAttributeType(const FString& AttributeTypeString);

		template<typename T, typename Enable = void>
		struct UE_DEPRECATED(5.7, "No longer used: The AttributeStorage handles all supported type traits internally now.") TAttributeTypeTraits
		{
			static constexpr EAttributeTypes GetType()
			{
				return EAttributeTypes::None;
			}

			template<typename ValueType>
			static FString ToString(const ValueType& Value)
			{
				return TEXT("Unknown type");
			}
		};

		template<typename ValueType>
		INTERCHANGECORE_API FString AttributeValueToString(const ValueType& Value);

		/**
		 * Enum to return complete status of a storage operation.
		 * It supports success with additional information.
		 * It supports multiple errors.
		 */
		enum class EAttributeStorageResult : uint64
		{
			None											= 0x0,
			//Success result.
			Operation_Success								= 0x1,
		

			//Operation error results from here.

			//The type of the value was not matching the existing type. We cannot change the type of an existing attribute.
			Operation_Error_WrongType						= (0x1 << 20),
			//The size of the value is different from the existing size. We cannot have a different size.
			Operation_Error_WrongSize						= (0x1 << 21),
			//The AttributeAllocationTable has an attribute whose offset is not valid in the storage.
			Operation_Error_AttributeAllocationCorrupted	= (0x1 << 22),
			//We cannot find the specified key.
			Operation_Error_CannotFoundKey					= (0x1 << 23),
			//There was an error when removing an attribute from the AttributeAllocationTable. The TArray remove has failed.
			Operation_Error_CannotRemoveAttribute			= (0x1 << 24),
			//We try to override an attribute but the specified options do not allow override.
			Operation_Error_CannotOverrideAttribute			= (0x1 << 25),
			//The storage is invalid (nullptr).
			Operation_Error_InvalidStorage					= (0x1 << 26),
			//Cannot get a valid value data pointer.
			Operation_Error_InvalidMultiSizeValueData		= (0x1 << 27),
		};

		ENUM_CLASS_FLAGS(EAttributeStorageResult)

		/**
		 * Helper function to interpret storage results.
		 * @return true if the result contains at least one of the RefResult flags.
		 */
		inline bool HasAttributeStorageResult(const EAttributeStorageResult Result, const EAttributeStorageResult RefResult)
		{
			return ((Result & RefResult) != EAttributeStorageResult::None);
		}
	
		/**
		 * Helper function to determine if the storage result is success.
		 * @return true if the result contains Operation_Success.
		 */
		inline bool IsAttributeStorageResultSuccess(const EAttributeStorageResult Result)
		{
			return HasAttributeStorageResult(Result, EAttributeStorageResult::Operation_Success);
		}

		/**
		 * Helper function to transform an operation result into a LOG.
		 * @param Result - The result we want to output a log for.
		 * @param OperationName - The operation name that ended up with the specified result.
		 * @param AttributeKey - The attribute we applied the operation to.
		 */
		INTERCHANGECORE_API void LogAttributeStorageErrors(const EAttributeStorageResult Result, const FString OperationName, const FAttributeKey AttributeKey );


		/**
		 * Enum to pass options when we add an attribute.
		 */
		enum class EAttributeStorageAddOptions : uint32
		{
			None					= 0x0,
			Option_Override			= 0x1,
			//allow the AddAttribute to override the value if it exists.
		};

		ENUM_CLASS_FLAGS(EAttributeStorageAddOptions)

		/**
		 * Helper function to interpret storage add options.
		 */
		inline bool HasAttributeStorageAddOption(const EAttributeStorageAddOptions Options, const EAttributeStorageAddOptions RefOptions)
		{
			return ((Options & RefOptions) != EAttributeStorageAddOptions::None);
		}

		/**
		 * Enumerates the attribute properties. Those properties affect how the attribute are stored or what they are used for.
		 */
		enum class EAttributeProperty : uint32
		{
			None						= 0x0,
			NoHash						= 0x1, /* No hash attribute will not be part of the hash result when calling GetStorageHash. */
		};

		ENUM_CLASS_FLAGS(EAttributeProperty)

		/**
		 * Helper function to interpret an attribute property.
		 */
		inline bool HasAttributeProperty(const EAttributeProperty PropertyA, const EAttributeProperty PropertyB)
		{
			return ((PropertyA & PropertyB) != EAttributeProperty::None);
		}

		template<typename T>
		struct UE_DEPRECATED(5.7, "No longer used.") TSpecializeType
		{
			typedef T Type;
		};

		template<typename T>
		struct TAttributeStorageTypeTraits
		{
			static constexpr bool bIsArrayOfStringType = false;
			static constexpr bool bIsArrayOfEnumType = false;
			static constexpr bool bIsArrayOfEnumAsByteType = false;
		};

		template<typename T>
		inline constexpr bool TIsStringType = std::is_same_v<T, FString> || std::is_same_v<T, FName> || std::is_same_v<T, FSoftObjectPath>;

		template<typename T>
		inline constexpr bool TIsArrayOfStringType = TAttributeStorageTypeTraits<T>::bIsArrayOfStringType;

		template<typename T>
		inline constexpr bool TIsArrayOfEnumType = TAttributeStorageTypeTraits<T>::bIsArrayOfEnumType;

		template<typename T>
		inline constexpr bool TIsArrayOfEnumAsByteType = TAttributeStorageTypeTraits<T>::bIsArrayOfEnumAsByteType;

		template<typename T>
		inline constexpr bool TIsNonEnumType = !TIsEnum<T>::Value && !TIsTEnumAsByte<T>::Value && !TIsArrayOfEnumType<T> && !TIsArrayOfEnumAsByteType<T>;

		template<typename T>
		struct TAttributeStorageTypeTraits<TArray<T>>
		{
			using ElementType = T;

			static constexpr bool bIsArrayOfStringType = TIsStringType<ElementType>;
			static constexpr bool bIsArrayOfEnumType = TIsEnum<ElementType>::Value;
			static constexpr bool bIsArrayOfEnumAsByteType = TIsTEnumAsByte<ElementType>::Value;
		};

		/**
		 * This class is a Key/Value storage inside a TArray64<uint8>.
		 * The keys are of type FAttributeKey, which is an FString. Each key is unique and has only one value.
		 * The value can be of any type contained in EAttributeTypes.
		 * 
		 * @note
		 * The storage is multi-thread safe. It uses a mutex to lock the storage for every read/write operation.
		 * The hash of the storage is deterministic because it sorts the attributes before calculating the hash.
		 */
		class FAttributeStorage
		{
		public:
			template<typename T, typename Enable = void>
			class TAttributeHandle;

			/**
			 * Class to get/set an attribute of the storage.
			 */
			template<typename T>
			class TAttributeHandle<T, std::enable_if_t<TIsNonEnumType<T>>>
			{
			public:
				INTERCHANGECORE_API TAttributeHandle();

				/**
				* Return true if the storage contains a valid attribute key, or false otherwise. 
				*/
				INTERCHANGECORE_API bool IsValid() const;
			
				INTERCHANGECORE_API EAttributeStorageResult Get(T& Value) const;

				INTERCHANGECORE_API EAttributeStorageResult Set(const T& Value);

				INTERCHANGECORE_API const FAttributeKey& GetKey() const;

			protected:
				class FAttributeStorage* AttributeStorage;
				FAttributeKey Key;
			
				INTERCHANGECORE_API TAttributeHandle(const FAttributeKey& InKey, const FAttributeStorage* InAttributeStorage);

				friend FAttributeStorage;
			};

			// Redirect enum instantiations to underlying type
			template<typename E>
			class TAttributeHandle<E, std::enable_if_t<TIsEnum<E>::Value>>
				: public TAttributeHandle<std::underlying_type_t<E>>
			{
			public:
				using UnderlyingType = std::underlying_type_t<E>;
				using Base = TAttributeHandle<UnderlyingType>;

				using Base::Base;

				explicit TAttributeHandle(const TAttributeHandle<UnderlyingType>& Other)
					: TAttributeHandle(Other.Key, Other.AttributeStorage)
				{
				}

				// Note: Since we derive TAttributeHandle<std::underlying_type_t<E>> we also get the
				// base class overloads for free, and can Get()/Set() with the underlying type directly!
				EAttributeStorageResult Get(E& Value) const
				{
					return Base::Get((UnderlyingType&)Value);
				}

				EAttributeStorageResult Set(const E& Value)
				{
					return Base::Set((const UnderlyingType&)Value);
				}

			protected:
				friend FAttributeStorage;
			};

			// Redirect TEnumAsByte<T> instantiations to uint8
			template<typename B>
			class TAttributeHandle<B, std::enable_if_t<TIsTEnumAsByte<B>::Value>>
				: public TAttributeHandle<uint8>
			{
			public:
				using UnderlyingType = uint8;
				using Base = TAttributeHandle<UnderlyingType>;

				using Base::Base;

				explicit TAttributeHandle(const TAttributeHandle<UnderlyingType>& Other)
					: TAttributeHandle(Other.Key, Other.AttributeStorage)
				{
				}

				EAttributeStorageResult Get(B& Value) const
				{
					return Base::Get((UnderlyingType&)Value);
				}

				EAttributeStorageResult Set(const B& Value)
				{
					return Base::Set((const UnderlyingType&)Value);
				}

			protected:
				friend FAttributeStorage;
			};

			// Redirect TArray<enum> instantiations to TArray<underlying type>
			template<typename T>
			class TAttributeHandle<T, std::enable_if_t<TIsArrayOfEnumType<T>>>
				: public TAttributeHandle<TArray<std::underlying_type_t<typename T::ElementType>>>
			{
			public:
				using UnderlyingType = std::underlying_type_t<typename T::ElementType>;
				using Base = TAttributeHandle<TArray<UnderlyingType>>;

				using Base::Base;

				explicit TAttributeHandle(const TAttributeHandle<TArray<UnderlyingType>>& Other)
					: TAttributeHandle(Other.Key, Other.AttributeStorage)
				{
				}

				EAttributeStorageResult Get(T& Value) const
				{
					return Base::Get((TArray<UnderlyingType>&)Value);
				}

				EAttributeStorageResult Set(const T& Value)
				{
					return Base::Set((const TArray<UnderlyingType>&)Value);
				}

			protected:
				friend FAttributeStorage;
			};

			// Redirect TArray<TEnumAsByte<T>> instantiations to TArray<uint8>
			template<typename T>
			class TAttributeHandle<T, std::enable_if_t<TIsArrayOfEnumAsByteType<T>>>
				: public TAttributeHandle<TArray<uint8>>
			{
			public:
				using UnderlyingType = uint8;
				using Base = TAttributeHandle<TArray<UnderlyingType>>;

				using Base::Base;

				explicit TAttributeHandle(const TAttributeHandle<TArray<UnderlyingType>>& Other)
					: TAttributeHandle(Other.Key, Other.AttributeStorage)
				{
				}

				EAttributeStorageResult Get(T& Value) const
				{
					return Base::Get((TArray<UnderlyingType>&)Value);
				}

				EAttributeStorageResult Set(const T& Value)
				{
					return Base::Set((const TArray<UnderlyingType>&)Value);
				}

			protected:
				friend FAttributeStorage;
			};

			FAttributeStorage() = default;

			INTERCHANGECORE_API FAttributeStorage(const FAttributeStorage& Other);

			INTERCHANGECORE_API FAttributeStorage& operator=(const FAttributeStorage& Other);

			/**
			 * Register an attribute in the storage. Return success if the attribute was properly added, or there is an existing
			 * attribute of the same type. Return an error otherwise.
			 *
			 * @Param ElementAttributeKey - the storage key (the path) of the attribute.
			 * @Param DefaultValue - the default value for the registered attribute.
			 *
			 * @note Possible errors:
			 * - Key exists with a different type.
			 * - Storage is corrupted.
			 */
			template<typename T, typename = std::enable_if_t<TIsNonEnumType<T>>>
			INTERCHANGECORE_API EAttributeStorageResult RegisterAttribute(const FAttributeKey& ElementAttributeKey, const T& DefaultValue, EAttributeProperty AttributeProperty = EAttributeProperty::None);

			// Redirect enum instantiations to underlying type
			template<typename E, typename = std::enable_if_t<TIsEnum<E>::Value>, int = 0>
			EAttributeStorageResult RegisterAttribute(const FAttributeKey& ElementAttributeKey, const E& DefaultValue, EAttributeProperty AttributeProperty = EAttributeProperty::None)
			{
				using UnderlyingType = std::underlying_type_t<E>;
				return RegisterAttribute<UnderlyingType>(ElementAttributeKey, (const UnderlyingType&)DefaultValue, AttributeProperty);
			}

			// Redirect TEnumAsByte instantiations to underlying type
			template<typename B, typename = std::enable_if_t<TIsTEnumAsByte<B>::Value>, int = 0, int = 0>
			EAttributeStorageResult RegisterAttribute(const FAttributeKey& ElementAttributeKey, const B& DefaultValue, EAttributeProperty AttributeProperty = EAttributeProperty::None)
			{
				using UnderlyingType = uint8;
				return RegisterAttribute<UnderlyingType>(ElementAttributeKey, (const UnderlyingType&)DefaultValue, AttributeProperty);
			}

			// Redirect TArray<enum> instantiations to TArray<underlying type>
			template<typename T, typename = std::enable_if_t<TIsArrayOfEnumType<T>>, int = 0, int = 0, int = 0>
			EAttributeStorageResult RegisterAttribute(const FAttributeKey& ElementAttributeKey, const T& DefaultValue, EAttributeProperty AttributeProperty = EAttributeProperty::None)
			{
				using UnderlyingType = std::underlying_type_t<typename T::ElementType>;
				return RegisterAttribute<TArray<UnderlyingType>>(ElementAttributeKey, (const TArray<UnderlyingType>&)DefaultValue, AttributeProperty);
			}

			// Redirect TArray<TEnumAsByte> instantiations to TArray<underlying type>
			template<typename T, typename = std::enable_if_t<TIsArrayOfEnumAsByteType<T>>, int = 0, int = 0, int = 0, int = 0>
			EAttributeStorageResult RegisterAttribute(const FAttributeKey& ElementAttributeKey, const T& DefaultValue, EAttributeProperty AttributeProperty = EAttributeProperty::None)
			{
				using UnderlyingType = uint8;
				return RegisterAttribute<TArray<uint8>>(ElementAttributeKey, (const TArray<uint8>&)DefaultValue, AttributeProperty);
			}

			/**
			 * Remove an attribute from the storage.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute to remove.
			 *
			 * @note Possible errors:
			 * - Key does not exist.
			 * - Internal storage structure removal error.
			 */
			INTERCHANGECORE_API EAttributeStorageResult UnregisterAttribute(const FAttributeKey& ElementAttributeKey);

			/**
			 * Retrieve a copy of an attribute value. Return success if the attribute is added.
			 *
			 * @param ElementAttributeKey is the storage key (the path) of the attribute.
			 * @param OutValue the reference value where we copy of the attribute value.
			 *
			 * @note Possible errors
			 * - Key do not exist
			 * - Key exist with a different type
			 * - Key exist with a wrong size
			 */
			template<typename T, typename = std::enable_if_t<TIsNonEnumType<T>>>
			INTERCHANGECORE_API EAttributeStorageResult GetAttribute(const FAttributeKey& ElementAttributeKey, T& OutValue) const;

			// Redirect enum instantiations to underlying type
			template<typename E, typename = std::enable_if_t<TIsEnum<E>::Value>, int = 0>
			EAttributeStorageResult GetAttribute(const FAttributeKey& ElementAttributeKey, E& OutValue) const
			{
				using UnderlyingType = std::underlying_type_t<E>;
				return GetAttribute<UnderlyingType>(ElementAttributeKey, OutValue);
			}

			// Redirect TEnumAsByte instantiations to underlying type
			template<typename B, typename = std::enable_if_t<TIsTEnumAsByte<B>::Value>, int = 0, int = 0>
			EAttributeStorageResult GetAttribute(const FAttributeKey& ElementAttributeKey, B& OutValue) const
			{
				using UnderlyingType = uint8;
				return GetAttribute<UnderlyingType>(ElementAttributeKey, OutValue);
			}

			// Redirect TArray<enum> instantiations to TArray<underlying type>
			template<typename ArrayOfE, typename = std::enable_if_t<TIsArrayOfEnumType<ArrayOfE>>, int = 0, int = 0, int = 0>
			EAttributeStorageResult GetAttribute(const FAttributeKey& ElementAttributeKey, ArrayOfE& OutValue) const
			{
				using UnderlyingType = std::underlying_type_t<typename ArrayOfE::ElementType>;
				return GetAttribute<TArray<UnderlyingType>>(ElementAttributeKey, (TArray<UnderlyingType>&)OutValue);
			}

			// Redirect TArray<TEnumAsByte> instantiations to TArray<uint8>
			template<typename ArrayOfB, typename = std::enable_if_t<TIsArrayOfEnumAsByteType<ArrayOfB>>, int = 0, int = 0, int = 0, int = 0>
			EAttributeStorageResult GetAttribute(const FAttributeKey& ElementAttributeKey, ArrayOfB& OutValue) const
			{
				using UnderlyingType = uint8;
				return GetAttribute<TArray<uint8>>(ElementAttributeKey, (TArray<uint8>&)OutValue);
			}

			/**
			 * Return an attribute handle for the specified attribute. This handle is a compile type check and is use to get and set the attribute value type.
			 * The function will assert if the key is missing or the type doesn't match the specified template type.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute.
			 */
			template<typename T, typename = std::enable_if_t<TIsNonEnumType<T>>>
			INTERCHANGECORE_API TAttributeHandle<T> GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const;

			// Redirect enum instantiations to underlying type
			template<typename E, typename = std::enable_if_t<TIsEnum<E>::Value>, int = 0>
			TAttributeHandle<E> GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const
			{
				using UnderlyingType = std::underlying_type_t<E>;
				return (TAttributeHandle<E>)GetAttributeHandle<UnderlyingType>(ElementAttributeKey);
			}

			// Redirect TEnumAsByte instantiations to underlying type
			template<typename B, typename = std::enable_if_t<TIsTEnumAsByte<B>::Value>, int = 0, int = 0>
			TAttributeHandle<B> GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const
			{
				using UnderlyingType = uint8;
				return (TAttributeHandle<B>)GetAttributeHandle<UnderlyingType>(ElementAttributeKey);
			}

			// Redirect TArray<enum> instantiations to TArray<underlying type>
			template<typename ArrayOfE, typename = std::enable_if_t<TIsArrayOfEnumType<ArrayOfE>>, int = 0, int = 0, int = 0>
			TAttributeHandle<ArrayOfE> GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const
			{
				using UnderlyingType = std::underlying_type_t<typename ArrayOfE::ElementType>;
				return (TAttributeHandle<ArrayOfE>)GetAttributeHandle<TArray<UnderlyingType>>(ElementAttributeKey);
			}

			// Redirect TArray<TEnumAsByte> instantiations to TArray<uint8>
			template<typename ArrayOfB, typename = std::enable_if_t<TIsArrayOfEnumAsByteType<ArrayOfB>>, int = 0, int = 0, int = 0, int = 0>
			TAttributeHandle<ArrayOfB> GetAttributeHandle(const FAttributeKey& ElementAttributeKey) const
			{
				using UnderlyingType = uint8;
				return (TAttributeHandle<ArrayOfB>)GetAttributeHandle<TArray<UnderlyingType>>(ElementAttributeKey);
			}

			/**
			 * Return the attribute type if the key exists, or None if the key is missing.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute.
			 *
			 */
			INTERCHANGECORE_API EAttributeTypes GetAttributeType(const FAttributeKey& ElementAttributeKey) const;

			/**
			 * Return true if the attribute key points to an existing attribute in the storage. Return false otherwise.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute.
			 *
			 */
			INTERCHANGECORE_API bool ContainAttribute(const FAttributeKey& ElementAttributeKey) const;

			/**
			 * Retrieve the array of keys that can be used to iterate and do reflection on the storage content.
			 *
			 */
			INTERCHANGECORE_API void GetAttributeKeys(TArray<FAttributeKey>& AttributeKeys) const;
	
			/**
			 * Return an FGuid built from the FSHA1 of the specified attribute data. If the attribute does not exist, return an empty FGUID.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute.
			 *
			 */
			INTERCHANGECORE_API FGuid GetAttributeHash(const FAttributeKey& ElementAttributeKey) const;
	
			/**
			 * This function fills the OutGuid with the hash of the specified attribute. Return true if the attribute exists and the OutGuid was assigned, or false otherwise without touching the OutGuid.
			 *
			 * @param ElementAttributeKey - the storage key (the path) of the attribute.
			 * @param OutGuid - where we put the attribute hash.
			 *
			 */
			INTERCHANGECORE_API bool GetAttributeHash(const FAttributeKey& ElementAttributeKey, FGuid& OutGuid) const;

			/**
			 * Return an FGuid built from the FSHA1 of all the attribute data contained in the node.
			 * The data includes the UniqueID and the DisplayLabel.
			 *
			 * @note the attributes are sorted by key when building the FSHA1 data. The hash will be deterministic for the same data whatever
			 * the order we add the attributes.
			 */
			INTERCHANGECORE_API FGuid GetStorageHash() const;

			/**
			 * Compare two storage objects to know which properties were modified/added/removed.
			 *
			 * @param BaseStorage - The reference storage.
			 * @param VersionStorage - The storage with the changes.
			 * @param RemovedAttribute - All attributes that are in base storage but not in version storage. Contains keys that are only valid for the base storage.
			 * @param AddedAttributes - All attributes that are in version storage but not in base storage. Contains keys that are only valid for the version storage.
			 * @param ModifiedAttributes - All attributes that are in both storage but have a different hash (different value).
			 *
			 */
			static INTERCHANGECORE_API void CompareStorage(const FAttributeStorage& BaseStorage, const FAttributeStorage& VersionStorage, TArray<FAttributeKey>& RemovedAttributes, TArray<FAttributeKey>& AddedAttributes, TArray<FAttributeKey>& ModifiedAttributes);

			/**
			 * Copy an array of attributes from the source storage to the destination storage. If the attribute already exists in the destination, the value will be updated.
			 * If a key does not exist in the source it will not be copied/created in the destination.
			 *
			 * @param SourceStorage - The storage source.
			 * @param DestinationStorage - The storage destination.
			 * @param AttributeKeys - All attributes that must be copied from the source to the destination.
			 *
			 */
			static INTERCHANGECORE_API void CopyStorageAttributes(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<FAttributeKey>& AttributeKeys);

			/**
			 * Copy an array of attributes from the source storage to the destination storage. If the attribute already exists in the destination, the value will be updated.
			 * If a key does not exist in the source it will not be copied/created in the destination.
			 *
			 * @param SourceStorage - The storage source.
			 * @param DestinationStorage - The storage destination.
			 * @param SrcDestAttributeKeys - All attributes that must be copied from the source to the destination, with a new key name.
			 *
			 */
			static INTERCHANGECORE_API void CopyStorageAttributes(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<TPair<FAttributeKey, FAttributeKey>>& SrcDestAttributeKeys);

			/**
			 * Return the defrag ratio. This ratio is used to know when we need to defrag the storage.
			 * @example - a ratio of 0.1f will defrag the storage if the memory lost is bigger then 10% of the storage allocation.
			 * Defrag is called when we remove an attribute or when we set the defrag ratio.
			 */
			inline float GetDefragRatio() const
			{
				return DefragRatio;
			}
	
			/** Set the defrag ratio. See GetDefragRatio() for the defrag documentation. */
			INTERCHANGECORE_API void SetDefragRatio(const float InDefragRatio);
	
			friend FArchive& operator<<(FArchive& Ar, FAttributeStorage& Storage)
			{
				Ar << Storage.FragmentedMemoryCost;
				Ar << Storage.DefragRatio;
				Ar << Storage.AttributeAllocationTable;
				Ar << Storage.AttributeStorage;

				if (Ar.IsLoading())
				{
					Storage.UpdateAllocationCount();
				}

				return Ar;
			}

			/**
			 * Reserve the allocation table and the storage data.
			 * 
			 * @param NewAttributeCount: The number of attributes we want to reserve. Passing a zero value does not reserve attribute count.
			 * @param NewStorageSize: The size of the storage all the new attributes will need. Passing a zero value do not reserve storage size.
			 */
			INTERCHANGECORE_API void Reserve(int64 NewAttributeCount, int64 NewStorageSize);

		protected:
			/** Structure used to hold the attribute information stored in the attribute allocation table. */
			struct FAttributeAllocationInfo
			{
				// clang fix for std::is_default_constructible_v 
				// returning false in inlined code of outer class
				FAttributeAllocationInfo() {}
				
        		//The offsets of our allocations in the storage
        		TArray<uint64> Offsets;
        		//The sizes of our allocations in the storage, in bytes
        		TArray<uint64> Sizes;
        		//The real type of the attribute
        		EAttributeTypes Type = EAttributeTypes::None;
        		//The attribute properties
        		EAttributeProperty Property = EAttributeProperty::None;
        		//128 bit Attribute hash
        		FGuid Hash = FGuid();

        		bool operator==(const FAttributeAllocationInfo& Other) const
        		{
        			//Offset is a unique key
        			return Offsets == Other.Offsets
        				&& Sizes == Other.Sizes
        				&& Type == Other.Type
        				&& Property == Other.Property
        				&& Hash == Other.Hash;
        		}
        	
        		//Serialization
        		friend FArchive& operator<<(FArchive& Ar, FAttributeAllocationInfo& AttributeAllocationInfo)
        		{
					Ar.UsingCustomVersion(FInterchangeCustomVersion::GUID);

					if (Ar.IsLoading()
						&& Ar.CustomVer(FInterchangeCustomVersion::GUID) < FInterchangeCustomVersion::MultipleAllocationsPerAttributeInStorageFixed)
					{
						uint64 OldOffset;
						Ar << OldOffset;
						AttributeAllocationInfo.Offsets = {OldOffset};

						uint64 OldSize;
						Ar << OldSize;
						AttributeAllocationInfo.Sizes = {OldSize};
					}
					else
					{
						Ar << AttributeAllocationInfo.Offsets;
						Ar << AttributeAllocationInfo.Sizes;
					}

					Ar << AttributeAllocationInfo.Type;
					Ar << AttributeAllocationInfo.Property;
					Ar << AttributeAllocationInfo.Hash;
        			return Ar;
        		}
			};

			/**
			 * Compare two allocation infos. Returns true if they are equal
			 *
			 * @param BaseKey - The reference key.
			 * @param BaseInfo - The reference allocation info.
			 * @param VersionStorage - The storage with the changes.
			 * @param VersionInfo - The allocation info to compare BaseInfo with.
			 *
			 */
			INTERCHANGECORE_API bool AreAllocationInfosEqual(const FAttributeKey& BaseKey, const FAttributeAllocationInfo& BaseInfo, const FAttributeStorage& VersionStorage, const FAttributeAllocationInfo& VersionInfo) const;

			/**
			 * Set an attribute value into the storage. Return success if the attribute was properly set.
			 *
			 * @param ElementAttributeKey is the storage key (the path) of the attribute
			 * @param Value is the value we want to add to the storage
			 *
			 * @note Possible errors
			 * - Key exist with a different type
			 * - Key exist with a wrong size
			 */
			template<typename T>
			EAttributeStorageResult SetAttribute(const FAttributeKey& ElementAttributeKey, const T& Value);

			template<typename T>
			EAttributeStorageResult SetAttribute(FAttributeAllocationInfo* AttributeAllocationInfo, const T& Value);

			/** Defrag the storage by using memmove on the attribute store after a hole in the storage. */
			void DefragInternal();

			template<typename T>
			UE_DEPRECATED(5.7, "No longer used.")
			static uint64 GetValueSize(const T& Value)
			{
				return 1;
			}

			const FStringView GetFStringViewAttributeFromStorage(const uint8* StorageData, const FAttributeAllocationInfo* AttributeAllocationInfo, int32 ElementIndex = 0) const;

			template<typename MultiSizeType>
			EAttributeStorageResult MultiSizeSetAttribute(FAttributeAllocationInfo* AttributeAllocationInfo, int32 TargetAllocationIndex, const MultiSizeType& Value, const void* SourceDataPtr, bool& bOutNeedsDefrag);

			template<typename ArrayType>
			EAttributeStorageResult GenericArrayGetAttribute(const FAttributeKey& ElementAttributeKey, ArrayType& OutValue) const;

			template<typename T>
			friend void CopyStorageAttributesInternal(const FAttributeStorage& SourceStorage, FAttributeStorage& DestinationStorage, const TArray<T>& AttributeKeys);

			void UpdateAllocationInfoHash(FAttributeAllocationInfo& AllocationInfo);

			void UpdateAllocationCount();

			/** The attribute allocation table is use to index the attributes into the storage. */
			TMap<FAttributeKey, FAttributeAllocationInfo> AttributeAllocationTable;

			/** The storage of the data point by the attribute allocation table */
			TArray64<uint8> AttributeStorage;

			/**
			 * The total size of the fragmented holes in the AttributeStorage (memory waste).
			 * A Hole is create each time we remove an attribute.
			 */
			uint64 FragmentedMemoryCost = 0;

			/**
			 * if FragmentedMemoryCost > AttributeStorage.Num*DefragRatio then defrag.
			 * This is use whenever we remove attribute or change DefragRatio value.
			 */
			float DefragRatio = 0.1f;

			/**
			 * Used to help with the debug visualizers (on Unreal.natvis), because now that we support multiple
			 * allocations per attribute it is no longer trivial to compute the total number of allocations
			 */
			uint32 AllocationCount = 0;

			/**
			 * Mutex use when accessing or modifying the storage
			 */
			mutable FCriticalSection StorageMutex;
		};
	} //ns interchange
}
