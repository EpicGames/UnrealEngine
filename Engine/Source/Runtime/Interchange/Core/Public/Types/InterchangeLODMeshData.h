// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "Math/MathFwd.h"

#include "InterchangeLODMeshData.generated.h"

/**
* This helper struct is placed here, so that we can implement support for it in AttributeStorage.h
*/
USTRUCT(BlueprintType)
struct FInterchangeLODMeshData
{
	GENERATED_BODY()

public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | LOD | MeshData")
	FString MeshUid;

	/*
	* Used for Rigid to RiggedMesh conversions to identify the bone targeting given Mesh.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | LOD | MeshData")
	FString SceneNodeName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | LOD | MeshData")
	FTransform Transform;

	/*
	* Specifically and only used for re-skinning meshes to T0.
	* GeometricTransform is already part of the Transform, however we need the separated out GeometricTransform for re-skinning to T0 calculations.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | LOD | MeshData")
	FTransform GeometricTransform;

	FInterchangeLODMeshData()
	{
	}

	FInterchangeLODMeshData(
		const FString& InMeshUid)
		: MeshUid(InMeshUid)
	{
	}

	FInterchangeLODMeshData(
		const FString& InMeshUid, 
		const FTransform& InTransform)
		: MeshUid(InMeshUid)
		, Transform(InTransform)
	{
	}

	FInterchangeLODMeshData(
		const FString& InMeshUid, const FString& InSceneNodeName,
		const FTransform& InTransform, const FTransform& InGeometricTransform)
		: MeshUid(InMeshUid)
		, SceneNodeName(InSceneNodeName)
		, Transform(InTransform)
		, GeometricTransform(InGeometricTransform)
	{
	}

	FString ToString() const
	{
		return FString::Printf(TEXT("NodeUid=%s SceneNodeName=%s Transform=%s GeometricTransform=%s"), *MeshUid, *SceneNodeName, *Transform.ToString(), *GeometricTransform.ToString());
	}

	//2 FString, 2 FTransform
	static constexpr int32 NumberOfAttributes() { return 4; };
};