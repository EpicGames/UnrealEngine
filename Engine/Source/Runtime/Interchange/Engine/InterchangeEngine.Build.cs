// Copyright Epic Games, Inc. All Rights Reserved.

using UnrealBuildTool;

public class InterchangeEngine : ModuleRules
{
	public InterchangeEngine(ReadOnlyTargetRules Target) : base(Target)
	{
		bTreatAsEngineModule = true;
		PublicDependencyModuleNames.AddRange(
			new string[] {
				"Core",
				"CoreUObject",
				"Engine",
				"InterchangeCore"
			}
		);

		PrivateDependencyModuleNames.AddRange(
			new string[] {
				"AssetRegistry",
				"SlateCore",
				"Slate",
				"DeveloperSettings",
				"Json",
				"JsonUtilities",
				"MeshDescription",
				"SkeletalMeshDescription",
			}
		);

		if (Target.bBuildEditor)
		{
			PrivateDependencyModuleNames.AddRange(
				new string[]
				{
					"SkeletalMeshUtilitiesCommon",
				}
			);
		}
	}
}
