// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangePipelineBase.h"
#include "Misc/ScopeLock.h"

FString UInterchangeAnalyticsHandlerBase::GetDefaultAnalyticsCategoryName(EInterchangeAnalyticsDefaultCategory DefaultCategory)
{
	static const FString CategoryNames[(size_t)EInterchangeAnalyticsDefaultCategory::InterchangeAnalyticsCategory_COUNT]
	{
	#define Category(Name) TEXT("."#Name)

		TEXT(""), // Not Specified
		Category(Models),
		Category(Scenes),
		Category(Materials),
		Category(Textures),
		Category(Sounds),
		Category(Meshes),
		Category(StaticMeshes),
		Category(SkeletalMeshesAndAnimations),
		Category(GeometryCaches),
		Category(Grooms),
		Category(Physics),

	#undef Category
	};

	if (DefaultCategory > EInterchangeAnalyticsDefaultCategory::NotSpecified
		&& DefaultCategory < EInterchangeAnalyticsDefaultCategory::InterchangeAnalyticsCategory_COUNT)
	{
		return CategoryNames[(size_t)DefaultCategory];
	}

	return CategoryNames[(size_t)EInterchangeAnalyticsDefaultCategory::NotSpecified];
}

FAnalyticsEventAttribute UInterchangeAnalyticsHandlerBase::CreateAnalyticsIDAttribute(FGuid AnalyticsID)
{
	ensure(AnalyticsID.IsValid());
	return FAnalyticsEventAttribute(TEXT("InterchangeAnalyticsGUID"), AnalyticsID.ToString());
}

FString UInterchangeAnalyticsHandlerBase::GetPipelineAnalyticsEventIdentifier()
{
	const static FString PipelineAnalyticsEventIdentifier = TEXT("Interchange.Usage.Import.Pipeline");
	return PipelineAnalyticsEventIdentifier;
}

FString UInterchangeAnalyticsHandlerBase::GetImportResultEventIdentifier()
{
	const static FString ImportResultEventIdentifier = TEXT("Interchange.Usage.ImportResult");
	return ImportResultEventIdentifier;
}

void UInterchangeAnalyticsHandlerBase::SetGUID(const FGuid& InGUID)
{
	if (AnalyticsID.IsValid())
	{
		return;
	}

	AnalyticsID = InGUID;
}

void UInterchangeAnalyticsHandlerBase::Append(const FString& Identifier, const TArray<FAnalyticsEventAttribute>& ToAdd)
{
	FScopeLock ScopeLock(&CriticalSection);
	Execute_Append(Identifier, ToAdd);
}

void UInterchangeAnalyticsHandlerBase::Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry)
{
	FScopeLock ScopeLock(&CriticalSection);
	Execute_Add(Identifier, Entry);
}

void UInterchangeAnalyticsHandlerBase::Clear(const FString& Identifier)
{
	FScopeLock ScopeLock(&CriticalSection);
	Execute_Clear(Identifier);
}

void UInterchangeAnalyticsHandlerBase::Flush()
{
	FScopeLock ScopeLock(&CriticalSection);
	Execute_Flush();
}

FAnalyticsEventAttribute UInterchangeAnalyticsHandlerBase::GetAnalyticsIDAttribute() const
{
	ensure(AnalyticsID.IsValid());
	return CreateAnalyticsIDAttribute(AnalyticsID);
}