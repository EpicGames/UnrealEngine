// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeAssetImportData.h"
#include "InterchangeBlueprintPipelineBase.h"
#include "InterchangeCustomVersion.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeManager.h"
#include "InterchangePipelineBase.h"
#include "InterchangePythonPipelineBase.h"
#include "InterchangeProjectSettings.h"
#include "JsonObjectConverter.h"
#include "Serialization/LargeMemoryReader.h"
#include "Serialization/LargeMemoryWriter.h"
#include "UObject/CoreRedirects.h"
#include "UObject/UObjectIterator.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeAssetImportData)

TMap<FString, UClass*> UInterchangeAssetImportData::GetPipelineClassPerName()
{
	TMap<FString, UClass*> ClassPerName;

	TArray<UClass*> PipelineCandidates;
	UInterchangeManager::GetInterchangeManager().FindPipelineCandidate(PipelineCandidates);

	ClassPerName.Reserve(PipelineCandidates.NumBytes());

	for (UClass* PipelineClass : PipelineCandidates)
	{
		ClassPerName.Add(PipelineClass->GetFullName(), PipelineClass);
	}

	return ClassPerName;
}

void UInterchangeAssetImportData::PostLoad()
{
	Super::PostLoad();

	if (NodeContainer_DEPRECATED)
	{
		bool HasNulls = false;
		NodeContainer_DEPRECATED->IterateNodes([&HasNulls](const FString&UID, UInterchangeBaseNode*Node)
		{
			HasNulls |= !Node;
		});

		if (!HasNulls)
		{
			SetNodeContainer(NodeContainer_DEPRECATED.Get());

			NodeContainer_DEPRECATED = nullptr;
		}
		else
		{
			UE_LOGF(LogInterchangeEngine, Display, "Missing Interchange reimport data for %ls", *this->GetOuter()->GetFullName());
		}
	}

	if (Pipelines_DEPRECATED.Num() > 0)
	{
		SetPipelines(Pipelines_DEPRECATED);
		Pipelines_DEPRECATED.Empty();
	}
}

UObject* DeSerializeTranslatorSettings(const FString& TranslatorSettingsStr, UClass* TranslatorSettingsClass)
{
	UInterchangeTranslatorSettings* GeneratedTranslatorSettings = NewObject<UInterchangeTranslatorSettings>(GetTransientPackage(), TranslatorSettingsClass);
	GeneratedTranslatorSettings->ClearInternalFlags(EInternalObjectFlags::Async);

	TSharedPtr<FJsonObject> RootObject;
	TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(TranslatorSettingsStr);
	if (FJsonSerializer::Deserialize(JsonReader, RootObject))
	{
		const TSharedPtr<FJsonObject> JsonTranslatorSettingsProperties = RootObject->GetObjectField(TEXT("GeneratedTranslatorSettings"));
		FJsonObjectConverter::JsonObjectToUStruct(JsonTranslatorSettingsProperties.ToSharedRef(), TranslatorSettingsClass, GeneratedTranslatorSettings, 0, 0);
	}

	return GeneratedTranslatorSettings;
}

FString SerializeTranslatorSettings(UObject* TranslatorSettings)
{
	if (UClass* TranslatorSettingsClass = TranslatorSettings->GetClass())
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		TSharedRef<FJsonObject> TranslatorSettingsPropertiesObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(TranslatorSettingsClass, TranslatorSettings, TranslatorSettingsPropertiesObject, 0, 0))
		{
			RootObject->SetField(TEXT("GeneratedTranslatorSettings"), MakeShareable(new FJsonValueObject(TranslatorSettingsPropertiesObject)));
		}
		//Write the json file
		FString Json;
		TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&Json, 0);
		if (FJsonSerializer::Serialize(RootObject, JsonWriter))
		{
			return Json;
		}
	}

	return TEXT("");
}


UObject* DeSerializePipeline(const FString& PipelineStr, UClass* PipelineClass)
{
	// Note: PipelineClass can be a child of either UInterchangePipelineBase or UInterchangePythonPipelineAsset
	UObject* GeneratedPipeline = NewObject<UObject>(GetTransientPackage(), PipelineClass);

	TSharedPtr<FJsonObject> RootObject;
	TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(PipelineStr);
	if (FJsonSerializer::Deserialize(JsonReader, RootObject))
	{
		const TSharedPtr<FJsonObject> JsonPipelineProperties = RootObject->GetObjectField(TEXT("GeneratedPipeline"));
		FJsonObjectConverter::JsonObjectToUStruct(JsonPipelineProperties.ToSharedRef(), PipelineClass, GeneratedPipeline, 0, 0);
	}

	if (UInterchangePipelineBase* InterchangePipelineBase = Cast<UInterchangePipelineBase>(GeneratedPipeline))
	{
		InterchangePipelineBase->UpdateWeakObjectPtrs();
	}
	else if (UInterchangePythonPipelineAsset* PythonPipelineAsset = Cast<UInterchangePythonPipelineAsset>(GeneratedPipeline))
	{
		PythonPipelineAsset->GeneratePipeline();
	}

	return GeneratedPipeline;
}

FString SerializePipeline(UObject* Pipeline)
{
	if (UClass* PipelineClass = Pipeline->GetClass())
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		TSharedRef<FJsonObject> PipelinePropertiesObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(PipelineClass, Pipeline, PipelinePropertiesObject, 0, 0))
		{
			RootObject->SetField(TEXT("GeneratedPipeline"), MakeShareable(new FJsonValueObject(PipelinePropertiesObject)));
		}
		//Write the json file
		FString Json;
		TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&Json, 0);
		if (FJsonSerializer::Serialize(RootObject, JsonWriter))
		{
			return Json;
		}
	}
	
	return TEXT("");
}

void UInterchangeAssetImportData::Serialize(FArchive& Ar)
{
	Super::Serialize(Ar);

	Ar.UsingCustomVersion(FInterchangeCustomVersion::GUID);
	int32 CustomVersion = Ar.CustomVer(FInterchangeCustomVersion::GUID);
	if (Ar.IsLoading())
	{
		CachedCustomVersions = Ar.GetCustomVersions();
	}

	if (CustomVersion >= FInterchangeCustomVersion::SerializedInterchangeObjectStoring)
	{
		if (Ar.IsSaving())
		{
			// Summary of custom version handling: When we deserialize UInterchangeAssetImportData from an archive, we'll store
			// the custom versions that were used in that archive within CachedCustomVersions, at the top of this function. The only
			// goal here is that when we later want to deserialize the CachedNodeContainer/Pipelines buffers, we can reuse those custom
			// versions on the memory reader too, so that the attributes/nodes/etc. can be deserialized with the correct code.
			// Those byte buffers originally came from the same deserialize call that produced our CachedCustomVersions, so they will
			// match that version.
			//
			// The custom versions are ultimately only stored in the package however, so for *saving* the only thing that matters is that
			// we're calling UsingCustomVersion(FInterchangeCustomVersion::GUID) right here at the top of this function, to make sure it
			// ends up in the package where we're saving these byte buffers into. We don't have to call it when serializing the
			// nodes/attributes/pipelines into those byte buffers themselves: While saving into the memory buffers we'll always just be
			// saving with the latest custom versions anyway.

			const static FString DebugContext = "UInterchangeAssetImportData::Serialize";
			const TArray<FCustomVersionDifference> VersionDifferences = FCurrentCustomVersions::Compare(
				CachedCustomVersions.GetAllVersions(),
				*DebugContext
			);

			const bool bNeedsReserialization = VersionDifferences.Num() > 0;
			if (bNeedsReserialization)
			{
				// Our current custom version differs from the custom version we last used to serialize our buffers. This means
				// we must make sure we reserialize these buffers now using the latest serialization code, so that they match the
				// custom version number that is going to be saved alongside them

				// Deserialize both
				ProcessContainerCache();
				ProcessPipelinesCache();

				// Serialize NodeContainer into CachedNodeContainer if we have one
				if (TransientNodeContainer)
				{
					FLargeMemoryWriter NodeContainerAr;
					NodeContainerAr.UsingCustomVersion(FInterchangeCustomVersion::GUID);
					TransientNodeContainer->SerializeNodeContainerData(NodeContainerAr);
					CachedNodeContainer = TArray64<uint8>(NodeContainerAr.GetData(), NodeContainerAr.TotalSize());
				}

				// Serialize pipelines into CachedPipelines
				TArray<TObjectPtr<UObject>> CopiedPipelines = TransientPipelines;
				SetPipelines(CopiedPipelines);
			}
		}

		Ar << CachedNodeContainer;
		Ar << CachedPipelines;
	}
}

#if WITH_EDITOR
bool UInterchangeAssetImportData::ConvertAssetImportDataToNewOwner(UObject* Owner)
{
	if (!Owner)
	{
		return false;
	}

	ProcessPipelinesCache();
	if (TransientPipelines.Num() == 0)
	{
		return false;
	}
	//Find the default asset stack for the first file of this "asset import data"
	//Then we will generate all the pipeline with the correct context
	//The final goal is to transfer the UInterchangePiplineBase::PropertiesStates to make sure property category are not hidden if user re-import the asset owning this AssetImportData
	constexpr bool bImportSceneFalse = false;
	const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bImportSceneFalse);
	const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;
	UE::Interchange::FScopedSourceData InterchangeSourceData(GetFirstFilename());
	UE::Interchange::FScopedTranslator ScopedTranslator(InterchangeSourceData.GetSourceData());
	FName DefaultStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(bImportSceneFalse, *InterchangeSourceData.GetSourceData());
	if (DefaultPipelineStacks.Contains(DefaultStackName))
	{
		TArray<UInterchangePipelineBase*> GeneratedPipelines;
		const FInterchangePipelineStack& PipelineStack = DefaultPipelineStacks.FindChecked(DefaultStackName);
		const TArray<FSoftObjectPath>* SoftPathPipelines = &PipelineStack.Pipelines;
		// If applicable, check to see if a specific pipeline stack is associated with this translator
		for (const FInterchangeTranslatorPipelines& TranslatorPipelines : PipelineStack.PerTranslatorPipelines)
		{
			const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
			if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
			{
				SoftPathPipelines = &TranslatorPipelines.Pipelines;
				break;
			}
		}

		for (int32 PipelineIndex = 0; PipelineIndex < SoftPathPipelines->Num(); ++PipelineIndex)
		{
			if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance((*SoftPathPipelines)[PipelineIndex]))
			{
				FInterchangePipelineContextParams ContextParams;
				ContextParams.ContextType = EInterchangePipelineContext::AssetImport;
				ContextParams.ReimportAsset = Owner;
				ContextParams.BaseNodeContainer = GetNodeContainer();
				GeneratedPipeline->AdjustSettingsForContext(ContextParams);
				GeneratedPipelines.Add(GeneratedPipeline);
			}
		}

		//We are now properly setup with Some generated pipelines stack that has the correct context value for the PropertiesStates
		for (TObjectPtr<UObject>& PipelinePtr : TransientPipelines)
		{
			if (UInterchangePipelineBase* Pipeline = Cast<UInterchangePipelineBase>(PipelinePtr.Get()))
			{
				UClass* PipelineClass = Pipeline->GetClass();
				for (UInterchangePipelineBase* GeneratedPipeline : GeneratedPipelines)
				{
					if (GeneratedPipeline->GetClass()->IsChildOf(PipelineClass))
					{
						//Push the properties states to the pipeline
						Pipeline->TransferAdjustSettings(GeneratedPipeline);
						Pipeline->AdjustSettingsFromCache();
						break;
					}
				}
			}
		}
	}
	return true;
}
#endif

UInterchangeBaseNodeContainer* UInterchangeAssetImportData::GetNodeContainer() const
{
	ProcessContainerCache();

	return Cast<UInterchangeBaseNodeContainer>(TransientNodeContainer.Get());
}

void UInterchangeAssetImportData::SetNodeContainer(UInterchangeBaseNodeContainer* InNodeContainer) const
{
	TransientNodeContainer = InNodeContainer;

	//Serialize cache
	if (TransientNodeContainer)
	{
		FLargeMemoryWriter NodeContainerAr;

		// This is likely not needed, but done in case any custom serialization code, for whatever reason,
		// wants to check a custom version from the archive during save, which is kind of unlikely.
		//
		// See the comments within UInterchangeAssetImportData::Serialize(), and also the implementation
		// (and comments) inside FArchiveState::GetCustomVersions()
		NodeContainerAr.SetCustomVersions(FCurrentCustomVersions::GetAll());

		TransientNodeContainer->SerializeNodeContainerData(NodeContainerAr);
		CachedNodeContainer = TArray64<uint8>(NodeContainerAr.GetData(), NodeContainerAr.TotalSize());
	}
	else
	{
		CachedNodeContainer.Reset();
	}
}

const UInterchangeTranslatorSettings* UInterchangeAssetImportData::GetTranslatorSettings() const
{
	ProcessTranslatorCache();
	return TransientTranslatorSettings;
}

void UInterchangeAssetImportData::SetTranslatorSettings(UInterchangeTranslatorSettings* TranslatorSettings) const
{
	TransientTranslatorSettings = TranslatorSettings;
	TransientTranslatorSettings->SetFlags(RF_Standalone);

	//Serialize cache
	CachedTranslatorSettings = {};
	if (TranslatorSettings)
	{
		FString TranslatorSettingsJSON = SerializeTranslatorSettings(TranslatorSettings);

		FString TranslatorSettingsClassFullName = TranslatorSettings->GetClass()->GetFullName();
		CachedTranslatorSettings = TPair<FString, FString>(TranslatorSettingsClassFullName, TranslatorSettingsJSON);
	}
}

void UInterchangeAssetImportData::SetPipelines(const TArray<UObject*>& InPipelines)
{
	TransientPipelines.Reset();

	for (UObject* Pipeline : InPipelines)
	{
		if (Pipeline)
		{
			TransientPipelines.Add(Pipeline);
		}
	}

	//Serialize cache
	CachedPipelines.Reset(TransientPipelines.Num());
	for (TObjectPtr<UObject>& PipelineObjectPtr : TransientPipelines)
	{
		UObject* PipelineObject = PipelineObjectPtr.Get();
		if (PipelineObject)
		{
			FString PipelineJSON = SerializePipeline(PipelineObject);

			FString PipelineFullName = PipelineObject->GetClass()->GetFullName();
			CachedPipelines.Emplace(PipelineFullName, PipelineJSON);
		}
	}
}

TArray<UObject*> UInterchangeAssetImportData::GetPipelines() const
{
	ProcessPipelinesCache();

	TArray<UObject*> OutPipelines;
	for (TObjectPtr<UObject>& Pipeline : TransientPipelines)
	{
		if (Pipeline.Get())
		{
			OutPipelines.Add(Pipeline.Get());
		}
	}
	return OutPipelines;
}

int32 UInterchangeAssetImportData::GetNumberOfPipelines() const
{
	ProcessPipelinesCache();
	return TransientPipelines.Num();
}


const UInterchangeBaseNode* UInterchangeAssetImportData::GetStoredNode(const FString& InNodeUniqueId) const
{
	ProcessContainerCache();
	UInterchangeBaseNodeContainer* NodeContainerResolved = TransientNodeContainer.Get();
	if (NodeContainerResolved)
	{
		return NodeContainerResolved->GetNode(InNodeUniqueId);
	}

	return nullptr;
}

UInterchangeFactoryBaseNode* UInterchangeAssetImportData::GetStoredFactoryNode(const FString& InNodeUniqueId) const
{
	ProcessContainerCache();
	UInterchangeBaseNodeContainer* NodeContainerResolved = TransientNodeContainer.Get();
	if (NodeContainerResolved)
	{
		return NodeContainerResolved->GetFactoryNode(InNodeUniqueId);
	}

	return nullptr;
}


void UInterchangeAssetImportData::ProcessContainerCache() const
{
	//de-serialize
	if (!TransientNodeContainer && CachedNodeContainer.Num() > 0)
	{
		FLargeMemoryReader NodeContainerAr(CachedNodeContainer.GetData(), CachedNodeContainer.Num());
		NodeContainerAr.SetCustomVersions(CachedCustomVersions);

		TransientNodeContainer = NewObject<UInterchangeBaseNodeContainer>();
		TransientNodeContainer->SerializeNodeContainerData(NodeContainerAr);
	}

	if(!TransientNodeContainer)
	{
		ProcessDeprecatedData();
	}
}

void UInterchangeAssetImportData::ProcessTranslatorCache() const
{
	//Verify our transient object was not garbage collect
	if (TransientTranslatorSettings && (TransientTranslatorSettings->IsGarbageEliminationEnabled() || TransientTranslatorSettings->HasAnyFlags(RF_BeginDestroyed | RF_FinishDestroyed)) )
	{
		TransientTranslatorSettings = nullptr;
	}

	//de-serialize
	if (!TransientTranslatorSettings && !CachedTranslatorSettings.Key.IsEmpty())
	{
		TMap<FString, UClass*> ClassPerName;
		for (FThreadSafeObjectIterator It(UClass::StaticClass()); It; ++It)
		{
			UClass* Class = Cast<UClass>(*It);
			if (Class->IsChildOf(UInterchangeTranslatorSettings::StaticClass()))
			{
				ClassPerName.Add(Class->GetFullName(), Class);
			}
		}

		FString ClassFullName = CachedTranslatorSettings.Key;
		FCoreRedirectObjectName RedirectedObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(ClassFullName));
		if (RedirectedObjectName.IsValid())
		{
			ClassFullName = RedirectedObjectName.ToString();
		}
		//This cannot fail to make sure we have a healty serialization
		if (!ensure(ClassPerName.Contains(ClassFullName)))
		{
			//We did not successfully serialize the content of the file into the node container
			return;
		}

		UClass* ToCreateClass = ClassPerName.FindChecked(ClassFullName);

		TransientTranslatorSettings = Cast<UInterchangeTranslatorSettings>(DeSerializeTranslatorSettings(CachedTranslatorSettings.Value, ToCreateClass));
		TransientTranslatorSettings->SetFlags(RF_Standalone);
	}
}

void UInterchangeAssetImportData::ProcessPipelinesCache() const
{
	if ((TransientPipelines.Num() == 0) && (CachedPipelines.Num() > 0))
	{
		TransientPipelines.Reset(CachedPipelines.Num());

		TMap<FString, UClass*> ClassPerName = GetPipelineClassPerName();

		for (const TPair<FString, FString>& CachedPipeline : CachedPipelines)
		{
			FString ClassFullName = CachedPipeline.Key;
			FCoreRedirectObjectName RedirectedObjectName = FCoreRedirects::GetRedirectedName(ECoreRedirectFlags::Type_Class, FCoreRedirectObjectName(ClassFullName));
			if (RedirectedObjectName.IsValid())
			{
				ClassFullName = RedirectedObjectName.ToString();
			}

			//This cannot fail to make sure we have a healthy serialization
			if (!ensure(ClassPerName.Contains(ClassFullName)))
			{
				//We did not successfully serialize the cached pipelines
				TransientPipelines.Empty();
				return;
			}

			UClass* ToCreateClass = ClassPerName.FindChecked(ClassFullName);
			
			UObject* Pipeline = DeSerializePipeline(CachedPipeline.Value, ToCreateClass);

			TransientPipelines.Add(Pipeline);
		}
	}
	else if (TransientPipelines.Num() == 0)
	{
		ProcessDeprecatedData();
	}
}

void UInterchangeAssetImportData::ProcessDeprecatedData() const
{
	if (!TransientNodeContainer && NodeContainer_DEPRECATED)
	{
		SetNodeContainer(NodeContainer_DEPRECATED.Get());
	}
	if (TransientPipelines.Num() == 0)
	{
		TransientPipelines.Empty();

		for (const TObjectPtr<UObject>& PipelineObject : Pipelines_DEPRECATED)
		{
			if (PipelineObject)
			{
				TransientPipelines.Add(PipelineObject.Get());
			}
		}
	}
}

void UInterchangeAssetImportData::BackupSourceData() const
{
#if WITH_EDITORONLY_DATA
	if (SourceDataBackup.SourceFiles.Num() == 0)
	{
		SourceDataBackup = SourceData;
	}
#endif
}

void UInterchangeAssetImportData::ClearBackupSourceData() const
{
#if WITH_EDITORONLY_DATA
	SourceDataBackup = FAssetImportInfo();
#endif
}

void UInterchangeAssetImportData::ReinstateBackupSourceData()
{
#if WITH_EDITORONLY_DATA
	if (SourceDataBackup.SourceFiles.Num() > 0)
	{
		SourceData = SourceDataBackup;
	}
#endif
}