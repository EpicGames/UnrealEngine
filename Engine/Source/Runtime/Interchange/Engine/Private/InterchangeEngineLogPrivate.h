// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangeEngine, Log, All);

#define INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED 0

#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED

class FScopeInterchangeTraceAsyncTask
{
public:
	FScopeInterchangeTraceAsyncTask(FString InTaskName)
		: TaskName(InTaskName)
	{
		StartTime = FPlatformTime::Seconds();
		double IntegerPart;
		double FractionalPart = FMath::Modf(StartTime, &IntegerPart);
		TaskID = FractionalPart * 1000000000;
		FString MessageStr = FString::FromInt(TaskID)
			+ TEXT(" ")
			+ TaskName
			+ TEXT(" Execute Start on ")
			+ (IsInGameThread() ? TraceAsyncGameThreadStr : TraceAsyncBackgroundThreadStr);

		UE_LOGF(LogInterchangeEngine, Warning, "%ls", *MessageStr);
	}

	~FScopeInterchangeTraceAsyncTask()
	{
		const int32 DeltaTime = (FPlatformTime::Seconds() - StartTime)*1000;
		FString MessageStr = FString::FromInt(TaskID)
			+ TEXT(" ")
			+ TaskName 
			+ TEXT(" Execute End on ")
			+ (IsInGameThread() ? TraceAsyncGameThreadStr : TraceAsyncBackgroundThreadStr)
			+ TEXT(" Time: ")
			+ FString::FromInt(DeltaTime)
			+ TEXT("ms");

		UE_LOGF(LogInterchangeEngine, Warning, "%ls", *MessageStr);
	}
private:
	double StartTime = 0;
	int32 TaskID = 0;
	FString TaskName;
	const FString TraceAsyncGameThreadStr = "Game Thread";
	const FString TraceAsyncBackgroundThreadStr = "Background Thread";
};

#define INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(TaskName) \
FScopeInterchangeTraceAsyncTask ScopeTraceAsynTask(TEXT(#TaskName));

#endif //INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED