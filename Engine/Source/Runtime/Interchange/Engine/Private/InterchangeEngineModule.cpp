// Copyright Epic Games, Inc. All Rights Reserved.
#include "CoreMinimal.h"
#include "InterchangeEngineLogPrivate.h"
#include "Modules/ModuleInterface.h"
#include "Modules/ModuleManager.h"

DEFINE_LOG_CATEGORY(LogInterchangeEngine);


/**
 * Implements the Interchange module.
 */
class FInterchangeEngineModule
	: public IModuleInterface
{
public:

	// IModuleInterface interface

	virtual void StartupModule() override { }
	virtual void ShutdownModule() override { }

	virtual bool SupportsDynamicReloading() override
	{
		return false;
	}
};


IMPLEMENT_MODULE(FInterchangeEngineModule, InterchangeEngine);
