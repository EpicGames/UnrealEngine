// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeImportCommon.h"

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangePipelineBase.h"
#include "InterchangePythonPipelineBase.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "InterchangeFactoryBase.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Types/AttributeStorage.h"

#include "UObject/Object.h"
#include "EditorFramework/AssetImportData.h"
#include "HAL/FileManager.h"

namespace UE::Interchange
{
	namespace Private::ImportCommon
	{
		UInterchangeAssetImportData* BeginSetupAssetData(FFactoryCommon::FUpdateImportAssetDataParameters& Parameters)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::Private::ImportCommon::BeginSetupAssetData)
			if (!ensure(IsInGameThread()))
			{
				return nullptr;
			}
			if (!ensure(Parameters.SourceData && Parameters.AssetImportDataOuter))
			{
				return nullptr;
			}

			UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(Parameters.AssetImportData);

			if (!AssetImportData)
			{
				AssetImportData = NewObject<UInterchangeAssetImportData>(Parameters.AssetImportDataOuter, NAME_None);
			}

			ensure(AssetImportData);

			return AssetImportData;
		}

		void RecursivelyDuplicateFactoryNodeDependencies(UInterchangeBaseNodeContainer* SourceContainer, UInterchangeBaseNodeContainer* DestinationContainer, TSet<FString>& FactoryDependencies)
		{
			if (!DestinationContainer)
			{
				return;
			}

			for (const FString& DependencyUid : FactoryDependencies)
			{
				if (UInterchangeFactoryBaseNode* FactoryNode = Cast<UInterchangeFactoryBaseNode>(SourceContainer->GetFactoryNode(DependencyUid)))
				{
					FObjectDuplicationParameters DupParam(FactoryNode, DestinationContainer);
					DestinationContainer->AddNode(CastChecked<UInterchangeBaseNode>(StaticDuplicateObjectEx(DupParam)));

					TSet<FString> ChildFactoryDependencies = FactoryNode->GetFactoryDependencies();
					RecursivelyDuplicateFactoryNodeDependencies(SourceContainer, DestinationContainer, ChildFactoryDependencies);
				}
			}
		}

		void EndSetupAssetData(FFactoryCommon::FUpdateImportAssetDataParameters& Parameters, UInterchangeAssetImportData* AssetImportData)
		{
			TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::Private::ImportCommon::EndSetupAssetData)
			UInterchangeBaseNodeContainer* FactoryNodeContainer = NewObject<UInterchangeBaseNodeContainer>(AssetImportData);
			//We copy only the factory node dependencies, we use this only 
			if (UInterchangeFactoryBaseNode* FactoryNode = Cast<UInterchangeFactoryBaseNode>(Parameters.NodeContainer->GetFactoryNode(Parameters.NodeUniqueID)))
			{
				//Set the interchange node graph data
				AssetImportData->NodeUniqueID = Parameters.NodeUniqueID;
				TSet<FString> FactoryDependencies;
				FactoryDependencies.Add(AssetImportData->NodeUniqueID);
				RecursivelyDuplicateFactoryNodeDependencies(Parameters.NodeContainer, FactoryNodeContainer, FactoryDependencies);
			}
			AssetImportData->SetNodeContainer(FactoryNodeContainer);

			if (UInterchangeTranslatorSettings* InterchangeTranslatorSettings = Parameters.Translator->GetSettings())
			{
				AssetImportData->SetTranslatorSettings(InterchangeTranslatorSettings);
			}

			TArray<UObject*> NewPipelines;
			for (const UObject* Pipeline : Parameters.Pipelines)
			{
				//Do not copy pipeline that cannot be use at reimport
				bool bPipelineSupportReimport = false;
				if (const UInterchangePythonPipelineAsset* PythonPipelineAsset = Cast<UInterchangePythonPipelineAsset>(Pipeline))
				{
					if (PythonPipelineAsset->GeneratedPipeline)
					{
						bPipelineSupportReimport = PythonPipelineAsset->GeneratedPipeline->SupportReimport();
					}
				}
				else if (const UInterchangePipelineBase* PipelineBase = Cast<UInterchangePipelineBase>(Pipeline))
				{
					bPipelineSupportReimport = PipelineBase->SupportReimport();
				}

				if(bPipelineSupportReimport)
				{
					UObject* DupPipeline = Cast<UObject>(StaticDuplicateObject(Pipeline, AssetImportData));
					if (DupPipeline)
					{
						// CAUTION: Python Pipeline adds this flag to the Package in PostLoad, 
						// and StaticDuplicateObject calls PostLoad on the duplicated asset. But here as the Outer is AssetImportData
						// The package is the imported asset. This will only show references in the reference viewer when the mode is set to
						// Editor Only References. Hence removing that particular flag from the package.
						DupPipeline->GetPackage()->ClearPackageFlags(PKG_EditorOnly);
						NewPipelines.Add(DupPipeline);
					}
				}
			}
			AssetImportData->SetPipelines(NewPipelines);
		}
	}

	FFactoryCommon::FUpdateImportAssetDataParameters::FUpdateImportAssetDataParameters(UObject* InAssetImportDataOuter
																						, UAssetImportData* InAssetImportData
																						, const UInterchangeSourceData* InSourceData
																						, FString InNodeUniqueID
																						, UInterchangeBaseNodeContainer* InNodeContainer
																						, const TArray<UObject*>& InPipelines
																						, const UInterchangeTranslatorBase* InTranslator)
		: AssetImportDataOuter(InAssetImportDataOuter)
		, AssetImportData(InAssetImportData)
		, SourceData(InSourceData)
		, NodeUniqueID(InNodeUniqueID)
		, NodeContainer(InNodeContainer)
		, Pipelines(InPipelines)
		, Translator(InTranslator)
	{
		ensure(AssetImportDataOuter);
		ensure(SourceData);
		ensure(!NodeUniqueID.IsEmpty());
		ensure(NodeContainer);
		ensure(Translator);
	}

	UAssetImportData* FFactoryCommon::UpdateImportAssetData(FUpdateImportAssetDataParameters& Parameters)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::UpdateImportAssetData)
		return UpdateImportAssetData(Parameters, [&Parameters](UInterchangeAssetImportData* AssetImportData)
			{
#if WITH_EDITORONLY_DATA
				//Set the asset import data file source to allow reimport. TODO: manage MD5 Hash properly
				TOptional<FMD5Hash> FileContentHash = Parameters.SourceData->GetFileContentHash();

				//Update the first filename, TODO for asset using multiple source file we have to update the correct index
				AssetImportData->Update(Parameters.SourceData->GetFilename(), FileContentHash.IsSet() ? &FileContentHash.GetValue() : nullptr);
#endif //WITH_EDITORONLY_DATA
			});

	}

	UAssetImportData* FFactoryCommon::UpdateImportAssetData(FUpdateImportAssetDataParameters& Parameters, TFunctionRef<void(UInterchangeAssetImportData*)> CustomFileSourceUpdate)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::UpdateImportAssetData2)
#if WITH_EDITORONLY_DATA
		if (!ensure(IsInGameThread()))
		{
			return nullptr;
		}
		if (!ensure(Parameters.SourceData && Parameters.AssetImportDataOuter))
		{
			return nullptr;
		}

		UInterchangeAssetImportData* AssetImportData = Private::ImportCommon::BeginSetupAssetData(Parameters);

		if (Parameters.AssetImportData) //-V1051
		{
			if (!Parameters.AssetImportData->IsA<UInterchangeAssetImportData>())
			{
				// Migrate the old source data
				TArray<FAssetImportInfo::FSourceFile> OldSourceFiles = Parameters.AssetImportData->SourceData.SourceFiles;
				AssetImportData->SetSourceFiles(MoveTemp(OldSourceFiles));
			}
		}

		CustomFileSourceUpdate(AssetImportData);


		Private::ImportCommon::EndSetupAssetData(Parameters, AssetImportData);

		// Return the asset import data so it can be set on the imported asset.
		return AssetImportData;
#else
		return nullptr;
#endif //#if WITH_EDITORONLY_DATA
	}

#if WITH_EDITORONLY_DATA
	FFactoryCommon::FSetImportAssetDataParameters::FSetImportAssetDataParameters(UObject* InAssetImportDataOuter
																				, UAssetImportData* InAssetImportData
																				, const UInterchangeSourceData* InSourceData
																				, FString InNodeUniqueID
																				, UInterchangeBaseNodeContainer* InNodeContainer
																				, const TArray<UObject*>& InPipelines
																				, const UInterchangeTranslatorBase* InTranslator)
		: FUpdateImportAssetDataParameters(InAssetImportDataOuter
			, InAssetImportData
			, InSourceData
			, InNodeUniqueID
			, InNodeContainer
			, InPipelines
			, InTranslator
			)
		, SourceFiles()
	{
	}

	UAssetImportData* FFactoryCommon::SetImportAssetData(FSetImportAssetDataParameters& Parameters)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::SetImportAssetData)
		UInterchangeAssetImportData* AssetImportData = Private::ImportCommon::BeginSetupAssetData(Parameters);

		// Update the source files
		{
			if (Parameters.SourceFiles.IsEmpty())
			{
				TOptional<FMD5Hash> FileContentHash = Parameters.SourceData->GetFileContentHash();

				// Todo add display label?
				Parameters.SourceFiles.Emplace(
					AssetImportData->SanitizeImportFilename(Parameters.SourceData->GetFilename()),
					IFileManager::Get().GetTimeStamp(*Parameters.SourceData->GetFilename()),
					FileContentHash.IsSet() ? FileContentHash.GetValue() : FMD5Hash()
					);
			}
			else
			{
				for (FAssetImportInfo::FSourceFile& Source : Parameters.SourceFiles)
				{
					// This is done here since this is not thread safe
					Source.RelativeFilename = AssetImportData->SanitizeImportFilename(Source.RelativeFilename);
				}
			}

			AssetImportData->SetSourceFiles(MoveTemp(Parameters.SourceFiles));
		}

		Private::ImportCommon::EndSetupAssetData(Parameters, AssetImportData);

		// Return the asset import data so it can be set on the imported asset.
		return AssetImportData;
	}

	void FFactoryCommon::BackupSourceData(const UAssetImportData* AssetImportData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::BackupSourceFilenames)
		if (const UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(AssetImportData))
		{
			InterchangeAssetImportData->BackupSourceData();
		}
	}

	void FFactoryCommon::ReinstateSourceData(UAssetImportData* AssetImportData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::ReinstateSourceData)
		if (UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(AssetImportData))
		{
			InterchangeAssetImportData->ReinstateBackupSourceData();
		}
	}

	void FFactoryCommon::ClearBackupSourceData(const UAssetImportData* AssetImportData)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::ClearBackupSourceData)
		if (const UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(AssetImportData))
		{
			InterchangeAssetImportData->ClearBackupSourceData();
		}
	}

	bool FFactoryCommon::GetSourceFilenames(const UAssetImportData* AssetImportData, TArray<FString>& OutSourceFilenames)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::GetSourceFilenames)
		if (Cast<UInterchangeAssetImportData>(AssetImportData) != nullptr)
		{
			AssetImportData->ExtractFilenames(OutSourceFilenames);
			return true;
		}
		return false;
	}

	bool FFactoryCommon::SetSourceFilename(UAssetImportData* AssetImportData, const FString& SourceFilename, int32 SourceIndex, const FString& SourceLabel)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::SetSourceFilename)
		if (AssetImportData)
		{
			const int32 SafeSourceIndex = SourceIndex == INDEX_NONE ? 0 : SourceIndex;				
			if (SafeSourceIndex < AssetImportData->GetSourceFileCount())
			{
				AssetImportData->UpdateFilenameOnly(SourceFilename, SafeSourceIndex);
			}
			else
			{
				//Create a source file entry, this case happen when user import a specific content for the first time
				AssetImportData->AddFileName(SourceFilename, SafeSourceIndex, SourceLabel);
			}
			return true;
		}

		return false;
	}
		
	bool FFactoryCommon::SetReimportSourceIndex(const UObject* Object, UAssetImportData* AssetImportData, int32 SourceIndex)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::SetReimportSourceIndex)
		UInterchangeAssetImportData* InterchangeAssetImportData = Cast<UInterchangeAssetImportData>(AssetImportData);
		if (!InterchangeAssetImportData)
		{
			return false;
		}

		TArray<UObject*> Pipelines = InterchangeAssetImportData->GetPipelines();
		for (UObject* PipelineObject : Pipelines)
		{
			if (UInterchangePythonPipelineAsset* PythonPipelineAsset = Cast<UInterchangePythonPipelineAsset>(PipelineObject))
			{
				if (PythonPipelineAsset->GeneratedPipeline)
				{
					PythonPipelineAsset->GeneratedPipeline->ScriptedSetReimportSourceIndex(Object->GetClass(), SourceIndex);
					PythonPipelineAsset->SetupFromPipeline(PythonPipelineAsset->GeneratedPipeline);
				}
			}
			else if (UInterchangePipelineBase* PipelineBase = Cast<UInterchangePipelineBase>(PipelineObject))
			{
				PipelineBase->ScriptedSetReimportSourceIndex(Object->GetClass(), SourceIndex);
			}
		}
		return true;
	}

#endif //#if WITH_EDITORONLY_DATA


	void FFactoryCommon::ApplyReimportStrategyToAsset(UObject* Asset
										, const UInterchangeFactoryBaseNode* PreviousAssetNode
										, const UInterchangeFactoryBaseNode* CurrentAssetNode
										, UInterchangeFactoryBaseNode* PipelineAssetNode)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FFactoryCommon::ApplyReimportStrategyToAsset)
		if (!ensure(PipelineAssetNode) || !ensure(CurrentAssetNode))
		{
			return;
		}
		EReimportStrategyFlags ReimportStrategyFlags = PipelineAssetNode->GetReimportStrategyFlags();
		switch (ReimportStrategyFlags)
		{
			case EReimportStrategyFlags::ApplyNoProperties:
			{
				//We want to have no effect
				break;
			}
					
			case EReimportStrategyFlags::ApplyPipelineProperties:
			{
				//Directly apply pipeline node attribute to the asset
				PipelineAssetNode->ApplyAllCustomAttributeToObject(Asset);
				break;
			}
				
			case EReimportStrategyFlags::ApplyEditorChangedProperties:
			{
				if (!PreviousAssetNode)
				{
					UE_LOGF(LogInterchangeEngine, Error, "Cannot apply the reimport strategy for asset [%ls], because there is no previous asset node in the import data.", *Asset->GetName());
					return;
				}
				TArray<FAttributeKey> RemovedAttributes;
				TArray<FAttributeKey> AddedAttributes;
				TArray<FAttributeKey> ModifiedAttributes;
				UInterchangeBaseNode::CompareNodeStorage(PreviousAssetNode, CurrentAssetNode, RemovedAttributes, AddedAttributes, ModifiedAttributes);

				//Cache all modified attributes from the pipeline node
				UE::Interchange::FAttributeStorage CachedAttributes;
				UInterchangeBaseNode::CopyStorageAttributes(PipelineAssetNode, CachedAttributes, ModifiedAttributes);

				//Set all ModifedAttributes from the CurrentAssetNode to PipelineAssetNode
				//This way, call to ApplyAllCustomAttributeToObject will preserve modified attributes from CurrentAssetNode
				UInterchangeBaseNode::CopyStorageAttributes(CurrentAssetNode, PipelineAssetNode, ModifiedAttributes);

				//Apply the pipeline node's attributes to the asset
				PipelineAssetNode->ApplyAllCustomAttributeToObject(Asset);

				//Restore all modified attributes back to the pipeline node
				UInterchangeBaseNode::CopyStorageAttributes(CachedAttributes, PipelineAssetNode, ModifiedAttributes);
				break;
			}
		}
	}

	UObject* FFactoryCommon::GetObjectToReimport(UInterchangeFactoryBase* Factory, UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
	{
		UObject* ReimportObjectCandidate = Factory->GetObjectToReimport(ReimportObject, FactoryNode, PackageName, AssetName, SubPathString);
		if (ReimportObjectCandidate != ReimportObject)
		{
			return ReimportObjectCandidate;
		}
#if WITH_EDITORONLY_DATA
		if (ReimportObject && !ReimportObject->GetClass()->IsChildOf(FactoryNode.GetObjectClass()))
		{
			if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(ReimportObject))
			{
				return SceneImportAsset->GetSceneObject(PackageName, AssetName, SubPathString);
			}
		}
#endif

		return ReimportObject;
	}

	const UInterchangeFactoryBaseNode* FFactoryCommon::GetFactoryNode(UObject* ObjectToReimport, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
	{
#if WITH_EDITORONLY_DATA
		if (UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(ObjectToReimport))
		{
			return SceneImportAsset->GetFactoryNodeFromPath(PackageName, AssetName, SubPathString);
		}
#endif

		return nullptr;
	}

	bool FFactoryCommon::CanProceedWithFactoryNode(UInterchangeFactoryBaseNode& FactoryNode, UObject* ReimportObject, UObject* ObjectToReimport, const FString& PackageName, const FString& AssetName, const FString& SubPathString)
	{
		// If ReimportObject is null, proceed with factory node
		if (!ReimportObject)
		{
			return true;
		}

		// Object associated with the factory node was not part of the previous import, proceed with factory node 
		if (!GetFactoryNode(ReimportObject, PackageName, AssetName, SubPathString))
		{
			return true;
		}

		// Associated object was not found (deleted or renamed) and factory node has been flagged as to be reimported. Proceed with factory node
		if (!ObjectToReimport && FactoryNode.ShouldForceNodeReimport())
		{
			return true;
		}

		// If ObjectToReimport is null but still tracked in the UInterchangeSceneImportAsset, it means that
		// the associated object was part of a previous import/reimport but deleted or renamed since.
		// The UInterchangeSceneImportAsset only tracks factory nodes with reference object.
		// Consequently keep track of it for future reimport.
		if (!ObjectToReimport)
		{
			const FTopLevelAssetPath AssetPath(*PackageName, *AssetName);
			FactoryNode.SetCustomReferenceObject(FSoftObjectPath(AssetPath, SubPathString));

			return false;
		}


		return true;
	}

	UObject* FFactoryCommon::AsyncFindObject(UInterchangeFactoryBaseNode* FactoryNode, const UClass* FactoryClass, UObject* Parent, const FString& AssetName)
	{
		if (!FactoryNode)
		{
			return nullptr;
		}

		const UClass* AssetClass = FactoryNode->GetObjectClass();
		if (!AssetClass || !AssetClass->IsChildOf(FactoryClass))
		{
			UE_LOGF(LogInterchangeEngine, Error, "The asset class %ls is not compatible with the factory class %ls. Cannot import asset %ls ."
				, AssetClass ? *AssetClass->GetName() : TEXT("unknown")
				, FactoryClass ? *FactoryClass->GetName() : TEXT("unknown")
				, *AssetName);
			return nullptr;
		}

		UObject* ExistingAsset = nullptr;
		FSoftObjectPath ReferenceObject;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObject))
		{
			ExistingAsset = ReferenceObject.TryLoad();
		}

		// create a new static mesh or overwrite existing asset, if possible
		if (!ExistingAsset)
		{
			UE_LOGF(LogInterchangeEngine, Error, "Cannot import the %ls asset [%ls] because it was not created on the game thread.", *AssetClass->GetName(), *AssetName);
			return nullptr;
		}

		if (!ExistingAsset->GetClass()->IsChildOf(AssetClass))
		{
			UE_LOGF(LogInterchangeEngine, Error, "Cannot import the %ls asset [%ls] because it will override an asset of a different class (%ls).", *AssetClass->GetName(), *AssetName, *ExistingAsset->GetClass()->GetName());
			return nullptr;
		}
		return ExistingAsset;
	}
}
