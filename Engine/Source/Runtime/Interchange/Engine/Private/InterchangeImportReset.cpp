// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeImportReset.h"

#include "HAL/IConsoleManager.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeManager.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSceneImportAsset.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

DEFINE_LOG_CATEGORY(LogInterchangeReset)
namespace UE::Interchange::InterchangeReset
{
	namespace Constants
	{
		const FString SceneImportAssetPathKey("InterchangeSceneImportAssetPath");
		const FString FactoryNodeUidPathKey("InterchangeFactoryNodeUid");
	}

	bool GInterchangeResetEnabled = false;
	FAutoConsoleVariableRef CCvarInterchangeResetEnabled(
		TEXT("Interchange.FeatureFlags.InterchangeReset"),
		GInterchangeResetEnabled,
		TEXT("Whether Interchange Reset is available"),
		ECVF_Default);

	bool GInterchangeResetFilteredNodes = true;
	FAutoConsoleVariableRef CCvarInterchangeResetFilteredNodes(
		TEXT("Interchange.FeatureFlags.Reset.UseFilteredNodes"),
		GInterchangeResetFilteredNodes,
		TEXT("Whether Reset should be limited to the filtered nodes if any."),
		ECVF_Default);

	namespace Private
	{
		static bool CanExecuteResetPhaseForNode(FInterchangeResetParameters& ResetParameters, const UInterchangeFactoryBaseNode* FactoryNode, const FOnFilterFactoryNodeDelegate& FilterDelegate)
		{
			if (!FactoryNode)
			{
				return false;
			}

			if (FilterDelegate.IsBound())
			{
				if (!FilterDelegate.Execute(ResetParameters, FactoryNode))
				{
					return false;
				}
			}
			else if (!ResetParameters.FilteredNodes.IsEmpty())
			{
				if (!ResetParameters.FilteredNodes.Contains(FactoryNode))
				{
					return false;
				}
			}

			return true;
		}

		/**
		 * Helper called before resetting all the imported object properties to the original values
		 */
		static void PreResetObjectProperties(FInterchangeResetParameters& ResetParameters, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (!CanExecuteResetPhaseForNode(ResetParameters, FactoryNode, ResetParameters.PreResetDelegates.OnNodeFilter))
			{
				return;
			}

			// This will setup the factory node if is it not already setup.
			ResetParameters.SetupFactoryNode(FactoryNode);

			if (TObjectPtr<UInterchangeFactoryBase> Factory = ResetParameters.GetFactoryForNode(FactoryNode); IsValid(Factory))
			{
				TArray<TObjectPtr<UObject>> ObjectInstances = ResetParameters.GetObjectInstancesForFactoryNode(FactoryNode);
				for (TObjectPtr<UObject>& ObjectInstance : ObjectInstances)
				{
					Factory->PreResetObjectProperties(ResetParameters.GetBaseNodeContainer(), FactoryNode, ObjectInstance);
				}
				ResetParameters.PreResetDelegates.OnNodeProcessed.ExecuteIfBound(ResetParameters, Factory, FactoryNode);
			}
		}

		/**
		 * Helper used to reset all the properties to the object
		 */
		static void ResetObjectProperties(FInterchangeResetParameters& ResetParameters, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (!CanExecuteResetPhaseForNode(ResetParameters, FactoryNode, ResetParameters.ResetDelegates.OnNodeFilter))
			{
				return;
			}

			if (TObjectPtr<UInterchangeFactoryBase> Factory = ResetParameters.GetFactoryForNode(FactoryNode); IsValid(Factory))
			{
				TArray<TObjectPtr<UObject>> ObjectInstances = ResetParameters.GetObjectInstancesForFactoryNode(FactoryNode);
				for (TObjectPtr<UObject>& ObjectInstance : ObjectInstances)
				{
					Factory->ResetObjectProperties(ResetParameters.GetBaseNodeContainer(), FactoryNode, ObjectInstance);
				}
				ResetParameters.ResetDelegates.OnNodeProcessed.ExecuteIfBound(ResetParameters, Factory, FactoryNode);
			}
		}

		/**
		 * Helper called after the object properties are reset
		 */
		static void PostResetObjectProperties(FInterchangeResetParameters& ResetParameters, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (!CanExecuteResetPhaseForNode(ResetParameters, FactoryNode, ResetParameters.PostResetDelegates.OnNodeFilter))
			{
				return;
			}

			if (TObjectPtr<UInterchangeFactoryBase> Factory = ResetParameters.GetFactoryForNode(FactoryNode); IsValid(Factory))
			{
				TArray<TObjectPtr<UObject>> ObjectInstances = ResetParameters.GetObjectInstancesForFactoryNode(FactoryNode);
				for(TObjectPtr<UObject>& ObjectInstance : ObjectInstances)
				{
					Factory->PostResetObjectProperties(ResetParameters.GetBaseNodeContainer(), FactoryNode, ObjectInstance);
				}
				ResetParameters.PostResetDelegates.OnNodeProcessed.ExecuteIfBound(ResetParameters, Factory, FactoryNode);
			}
		}
	}
}

FInterchangeResetParameters::FInterchangeResetParameters(const UInterchangeSceneImportAsset* InSceneImportAsset)
	:FInterchangeResetParameters(InSceneImportAsset, MakeUnique<FInterchangeResetContextData>())
{
}

FInterchangeResetParameters::FInterchangeResetParameters(const UInterchangeSceneImportAsset* InSceneImportAsset, TUniquePtr<FInterchangeResetContextData> InResetContextData)
	: ResetContextData(MoveTemp(InResetContextData))
	, SceneImportAsset(InSceneImportAsset)
	, ResultsContainer(NewObject<UInterchangeResultsContainer>(GetTransientPackage()))
{
}

FInterchangeResetParameters::~FInterchangeResetParameters()
{
	for (const auto& FactoryNodeDataPair : FactoryNodeDataCache)
	{
		if (FactoryNodeDataPair.Value.Factory)
		{
			TObjectPtr<UInterchangeFactoryBase> Factory = FactoryNodeDataPair.Value.Factory;
			Factory->RemoveFromRoot();
			Factory->ClearFlags(RF_Standalone);
		}
	}
	FactoryNodeDataCache.Empty();
}

TObjectPtr<UInterchangeFactoryBase> FInterchangeResetParameters::GetFactoryForNode(const UInterchangeFactoryBaseNode* FactoryNode) const
{
	if (FactoryNodeDataCache.Contains(FactoryNode))
	{
		const FInterchangeResetParameters::FFactoryNodeData& FactoryNodeData = FactoryNodeDataCache[FactoryNode];
		return FactoryNodeData.Factory;
	}

	return TObjectPtr<UInterchangeFactoryBase>(nullptr);
}

TArray<TObjectPtr<UObject>> FInterchangeResetParameters::GetObjectInstancesForFactoryNode(const UInterchangeFactoryBaseNode* FactoryNode)
{
	TArray<TObjectPtr<UObject>> OutObjectInstances;

	if (FactoryNodeDataCache.Contains(FactoryNode))
	{
		const FInterchangeResetParameters::FFactoryNodeData& FactoryNodeData = FactoryNodeDataCache[FactoryNode];
		if (!FactoryNodeData.ObjectsToReset.IsEmpty())
		{
			for (const TObjectPtr<UObject>& ObjectInstance : FactoryNodeData.ObjectsToReset)
			{
				if (ObjectInstance.Get() != nullptr)
				{
					OutObjectInstances.Add(ObjectInstance);
				}
			}
		}
		else if (UObject* ReferencedObject = Cast<UObject>(FactoryNodeData.ReferencedObjectPath.TryLoad()))
		{
			OutObjectInstances.Add(ReferencedObject);
		}
	}

	return OutObjectInstances;
}

void FInterchangeResetParameters::AddObjectInstanceToReset(const UInterchangeFactoryBaseNode* FactoryNode, UObject* ObjectToReset)
{
	FInterchangeResetParameters::FFactoryNodeData& FactoryNodeData = FactoryNodeDataCache.FindOrAdd(FactoryNode);
	if (!FilteredNodes.Contains(FactoryNode))
	{
		FilteredNodes.Add(FactoryNode);
	}
	FactoryNodeDataCache[FactoryNode].ObjectsToReset.Add(ObjectToReset);
}

void FInterchangeResetParameters::SetupFactoryNode(const UInterchangeFactoryBaseNode* FactoryNode)
{
	FInterchangeResetParameters::FFactoryNodeData& FactoryNodeData = FactoryNodeDataCache.FindOrAdd(FactoryNode);
	if (!FactoryNodeData.Factory)
	{
		UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();
		if (const UClass* ObjectClass = FactoryNode->GetObjectClass())
		{
			if (const UClass* FactoryClass = InterchangeManager.GetRegisteredFactoryClass(ObjectClass))
			{
				FactoryNodeData.Factory = NewObject<UInterchangeFactoryBase>(GetTransientPackage(), FactoryClass, NAME_None, RF_Standalone);
				FactoryNodeData.Factory->AddToRoot();
				FactoryNodeData.Factory->SetResultsContainer(ResultsContainer);
			}
		}

		FSoftObjectPath ReferenceObjectPath;
		if (FactoryNode->GetCustomReferenceObject(ReferenceObjectPath))
		{
			FactoryNodeData.ReferencedObjectPath = ReferenceObjectPath;
		}
	}
}

const UInterchangeBaseNodeContainer* FInterchangeResetParameters::GetBaseNodeContainer() const
{
#if WITH_EDITORONLY_DATA
	if (!SceneImportAsset)
	{
		return nullptr;
	}

	if (!SceneImportAsset->AssetImportData)
	{
		return nullptr;
	}

	return SceneImportAsset->AssetImportData->GetNodeContainer();
#else
	return nullptr;
#endif
}
const UInterchangeSceneImportAsset* FInterchangeResetParameters::GetSceneImportAsset() const
{
	return SceneImportAsset;
}

const UInterchangeResultsContainer* FInterchangeResetParameters::GetResultsContainer() const
{
	return ResultsContainer;
}

void FInterchangeReset::ExecuteReset(FInterchangeResetParameters& ResetParameters)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(FInterchangeReset::ExecuteReset)
	if (const UInterchangeBaseNodeContainer* BaseNodeContainer = ResetParameters.GetBaseNodeContainer())
	{
		// Pre Reset Phase
		BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([&](const FString& NodeKey, UInterchangeFactoryBaseNode* FactoryNode)
			{
				UE::Interchange::InterchangeReset::Private::PreResetObjectProperties(ResetParameters, FactoryNode);
			});
		ResetParameters.PreResetDelegates.OnCompleted.ExecuteIfBound(ResetParameters);

		// Reset Phase
		BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([&](const FString& NodeKey, UInterchangeFactoryBaseNode* FactoryNode)
			{
				UE::Interchange::InterchangeReset::Private::ResetObjectProperties(ResetParameters, FactoryNode);
			});
		ResetParameters.ResetDelegates.OnCompleted.ExecuteIfBound(ResetParameters);

		// Post Reset Phase
		BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([&](const FString& NodeKey, UInterchangeFactoryBaseNode* FactoryNode)
			{
				UE::Interchange::InterchangeReset::Private::PostResetObjectProperties(ResetParameters, FactoryNode);
			});
		ResetParameters.PostResetDelegates.OnCompleted.ExecuteIfBound(ResetParameters);
	}
}