// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeManager.h"

#include "AssetCompilingManager.h"
#include "AssetRegistry/AssetData.h"
#include "AssetRegistry/AssetDataTagMap.h"
#include "AssetRegistry/AssetRegistryHelpers.h"
#include "AssetRegistry/AssetRegistryModule.h"
#if WITH_EDITOR
#include "AssetToolsModule.h"
#endif
#include "Async/Async.h"
#include "CoreGlobals.h"
#include "CoreMinimal.h"
#include "Engine/Blueprint.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "Framework/Notifications/NotificationManager.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeBlueprintPipelineBase.h"
#include "InterchangeEditorUtilitiesBase.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeHelper.h"
#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeProjectSettings.h"
#include "InterchangePythonPipelineBase.h"
#include "InterchangeSceneImportAsset.h"
#include "InterchangeSourceData.h"
#include "InterchangeTaskSystem.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeWriterBase.h"
#include "Internationalization/Internationalization.h"
#include "Misc/App.h"
#include "Misc/AssertionMacros.h"
#include "Misc/AsyncTaskNotification.h"
#include "Misc/MessageDialog.h"
#include "Misc/NamePermissionList.h"
#include "Misc/ScopedSlowTask.h"
#include "Misc/DateTime.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeSourceNode.h"
#include "PackageUtils/PackageUtils.h"
#include "Tasks/InterchangeTaskParsing.h"
#include "Tasks/InterchangeTaskPipeline.h"
#include "Tasks/InterchangeTaskTranslator.h"
#include "UObject/Class.h"
#include "UObject/GarbageCollection.h"
#include "UObject/NameTypes.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ReflectedTypeAccessors.h"
#include "UObject/UObjectIterator.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Widgets/Notifications/SNotificationList.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeManager)

static bool GInterchangeImportEnable = false;
static FAutoConsoleVariableRef CCvarInterchangeImportEnable(
	TEXT("Interchange.FeatureFlags.Import.Enable"),
	GInterchangeImportEnable,
	TEXT("Whether Interchange import is enabled."),
	ECVF_Default);

bool UInterchangeManager::bIsCreatingSingleton = false;

namespace UE::Interchange::Private
{
	namespace StaticHelpers
	{
		static TAtomic<uint64> AsyncHelperCounter = 0;
	
		/**
			 * Interchange import task will show a dialog in case user try to override an existing asset and bReplaceExisting is false,
			 * if this optional is set, it will override or not all existing assets this Import stack tries to override.
			 * This setting resets whenever Importing finishes.
			 * -1:= Reset (not set)
			 *  0:= Set and False
			 *  1:= Set and True;
			 */
		static TAtomic<int8> bReplaceExistingAllDialogAnswer = -1;
	}
	

	const FLogCategoryBase* GetLogInterchangePtr()
	{
#if NO_LOGGING
		return nullptr;
#else
		return &LogInterchangeEngine;
#endif
	}

	class FInterchangeAnalyticsHelper
	{
	public:
		FInterchangeAnalyticsHelper(UInterchangeAnalyticsHandlerBase* InHandler)
			:Handler(InHandler)
		{
		}

		void Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry)
		{
			if (!Handler)
			{
				return;
			}

			Handler->Add(Identifier, Entry);
		}

		void Flush()
		{
			if (!Handler)
			{
				return;
			}

			Handler->Flush();
		}

		void Send(const TArray<UInterchangePipelineBase*>& Pipelines, const int32 AsyncHelperUniqueId)
		{
			if (!Handler)
			{
				return;
			}

			Handler->Send(Pipelines, AsyncHelperUniqueId);
		}

	private:
		UInterchangeAnalyticsHandlerBase* Handler;
	};
}

UE::Interchange::FScopedInterchangeImportEnableState::FScopedInterchangeImportEnableState(const bool bScopeValue)
{
	bOriginalInterchangeImportEnableState = CCvarInterchangeImportEnable->GetBool();
	CCvarInterchangeImportEnable->Set(bScopeValue);
}

UE::Interchange::FScopedInterchangeImportEnableState::~FScopedInterchangeImportEnableState()
{
	CCvarInterchangeImportEnable->Set(bOriginalInterchangeImportEnableState);
}

UE::Interchange::FScopedSourceData::FScopedSourceData(const FString& Filename)
{
	SourceDataPtr = TStrongObjectPtr<UInterchangeSourceData>(UInterchangeManager::GetInterchangeManager().CreateSourceData(Filename));
	ensure(SourceDataPtr.IsValid());
}

UE::Interchange::FScopedSourceData::~FScopedSourceData()
{
	SourceDataPtr.Reset();
}


UInterchangeSourceData* UE::Interchange::FScopedSourceData::GetSourceData() const
{
	return SourceDataPtr.Get();
}

UE::Interchange::FScopedTranslator::FScopedTranslator(const UInterchangeSourceData* SourceData)
{
	//Found the translator
	ScopedTranslatorPtr = TStrongObjectPtr<UInterchangeTranslatorBase>(UInterchangeManager::GetInterchangeManager().GetTranslatorForSourceData(SourceData));
}

UE::Interchange::FScopedTranslator::~FScopedTranslator()
{
	//Found the translator
	if (ScopedTranslatorPtr.IsValid())
	{
		ScopedTranslatorPtr->ReleaseSource();
	}
	ScopedTranslatorPtr.Reset();
}

UInterchangeTranslatorBase* UE::Interchange::FScopedTranslator::GetTranslator()
{
	return ScopedTranslatorPtr.Get();
}

UE::Interchange::FScopedBaseNodeContainer::FScopedBaseNodeContainer()
{
	ScopedBaseNodeContainerPtr = TStrongObjectPtr<UInterchangeBaseNodeContainer>(NewObject<UInterchangeBaseNodeContainer>(GetTransientPackage()));
}

UE::Interchange::FScopedBaseNodeContainer::~FScopedBaseNodeContainer()
{
	ScopedBaseNodeContainerPtr.Reset();
}

UInterchangeBaseNodeContainer* UE::Interchange::FScopedBaseNodeContainer::GetBaseNodeContainer()
{
	return ScopedBaseNodeContainerPtr.Get();
}

FInterchangeImportResultAnalyticsInfo UE::Interchange::FImportAsyncHelper::CreateImportResultAnalyticsInfo(FImportAsyncHelper& AsyncHelper)
{
	const UInterchangeBaseNodeContainer* PrimaryBaseNodeContainer = AsyncHelper.BaseNodeContainers[0].Get();
	const UInterchangeSourceNode* PrimarySourceNode = UInterchangeSourceNode::GetUniqueInstance(PrimaryBaseNodeContainer);
	const UInterchangeSourceData* PrimarySourceData = AsyncHelper.SourceDatas[0].Get();

	TOptional<const UInterchangeResultsContainer*> AssetImportResultsContainer;
	TOptional<const UInterchangeResultsContainer*> SceneImportResultsContainer;

	TMap<int32, TArray<TObjectPtr<UObject>>> AssetObjectsPerSourceIndex;
	TMap<int32, TArray<TObjectPtr<UObject>>> SceneObjectsPerSourceIndex;

	for (const TPair<int32, TArray<FImportedObjectInfo>>& ImportedAssetPerSourceIndex : AsyncHelper.ImportedAssetsPerSourceIndex)
	{
		TArray<TObjectPtr<UObject>>& ImportedObjects = AssetObjectsPerSourceIndex.FindOrAdd(ImportedAssetPerSourceIndex.Key);
		for (const FImportedObjectInfo& ImportObjectInfo : ImportedAssetPerSourceIndex.Value)
		{
			ImportedObjects.Add(ImportObjectInfo.GetImportedObject());
		}
	}
	if (const UInterchangeResultsContainer* AssetImportResults = AsyncHelper.AssetImportResult->GetResults())
	{
		AssetImportResultsContainer = AssetImportResults;
	}


	for (const TPair<int32, TArray<FImportedObjectInfo>>& ImportedSceneObjectPerSourceIndex : AsyncHelper.ImportedSceneObjectsPerSourceIndex)
	{
		TArray<TObjectPtr<UObject>>& ImportedObjects = SceneObjectsPerSourceIndex.FindOrAdd(ImportedSceneObjectPerSourceIndex.Key);
		for (const FImportedObjectInfo& ImportObjectInfo : ImportedSceneObjectPerSourceIndex.Value)
		{
			ImportedObjects.Add(ImportObjectInfo.GetImportedObject());
		}
	}
	if (const UInterchangeResultsContainer* SceneImportResults = AsyncHelper.SceneImportResult->GetResults())
	{
		SceneImportResultsContainer = SceneImportResults;
	}

	return FInterchangeImportResultAnalyticsInfo{
		.bCancel  = AsyncHelper.bCancel,
		.UniqueId = AsyncHelper.UniqueId,
		.PrimarySourceData = PrimarySourceData,
		.PrimarySourceNode = PrimarySourceNode,
		.AssetImportResultsContainer = AssetImportResultsContainer,
		.SceneImportResultsContainer = SceneImportResultsContainer,
		.ImportedAssetsPerSourceIndex = MoveTemp(AssetObjectsPerSourceIndex),
		.ImportedSceneObjectsPerSourceIndex = MoveTemp(SceneObjectsPerSourceIndex),
	};
}

UE::Interchange::FImportAsyncHelper::FImportAsyncHelper()
	: AssetImportResult(MakeShared<FImportResult>())
	, SceneImportResult(MakeShared<FImportResult>())
{
	bCancel = false;

	if (UE::Interchange::Private::StaticHelpers::AsyncHelperCounter == 0)
	{
		UInterchangeManager::GetInterchangeManager().OnImportStarted.Broadcast();
	}
	UE::Interchange::Private::StaticHelpers::AsyncHelperCounter++;
}

UE::Interchange::FImportAsyncHelper::~FImportAsyncHelper()
{
	TMap<int32, TArray<TObjectPtr<UObject>>> ExtractedAssetsPerSourceIndex = ExtractImportedObjectsPerSourceIndex(ImportedAssetsPerSourceIndex);
	TMap<int32, TArray<TObjectPtr<UObject>>> ExtractedSceneObjectsPerSourceIndex  = ExtractImportedObjectsPerSourceIndex(ImportedSceneObjectsPerSourceIndex);
	UInterchangeManager::GetInterchangeManager().OnImportedObjectsAvailable.Broadcast(ExtractedAssetsPerSourceIndex, ExtractedSceneObjectsPerSourceIndex);

	CleanUp();
	
	if (AnalyticsHandler.IsValid())
	{
		AnalyticsHandler.Reset();
	}

	UE::Interchange::Private::StaticHelpers::AsyncHelperCounter--;
	if (UE::Interchange::Private::StaticHelpers::AsyncHelperCounter == 0)
	{
		UInterchangeManager::GetInterchangeManager().OnImportFinished.Broadcast();
		UInterchangeManager::GetInterchangeManager().ResetReplaceExistingAlldialogAnswerSet();
	}
}

void UE::Interchange::FImportAsyncHelper::AddReferencedObjects(FReferenceCollector& Collector)
{
	Collector.AddReferencedObjects(SourceDatas);
	Collector.AddReferencedObjects(Translators);
	Collector.AddReferencedObjects(Pipelines);
	Collector.AddReferencedObjects(CreatedFactories);
}

bool UE::Interchange::FImportAsyncHelper::IsClassImportAllowed(UClass* Class)
{
#if WITH_EDITOR
	//Lock the classes
	FScopeLock Lock(&ClassPermissionLock);

	if (AllowedClasses.Contains(Class))
	{
		return true;
	}
	else if (DeniedClasses.Contains(Class))
	{
		return false;
	}

	IAssetTools& AssetTools = FAssetToolsModule::GetModule().Get();
	TSharedPtr<FPathPermissionList> AssetClassPermissionList = AssetTools.GetAssetClassPathPermissionList(EAssetClassAction::ImportAsset);
	if (AssetClassPermissionList && AssetClassPermissionList->HasFiltering())
	{
		if (!AssetClassPermissionList->PassesFilter(Class->GetPathName()))
		{
			UE_LOGF(LogInterchangeEngine, Display, "Creating assets of class '%ls' is not allowed in this project.", *Class->GetName());
			DeniedClasses.Add(Class);
			return false;
		}
	}
	AllowedClasses.Add(Class);
#endif //WITH_EDITOR
	return true;
}

/*
* // Created factories map, Key is factory node UID
			mutable FCriticalSection CreatedFactoriesLock;
			TMap<FString, UInterchangeFactoryBase*> CreatedFactories;
*/

//Create package map, Key is package name. We cannot create package asynchronously so we have to create a game thread task to do this
UPackage* UE::Interchange::FImportAsyncHelper::GetCreatedPackage(const FString& PackageName) const
{
	FScopeLock Lock(&CreatedPackagesLock);
	if (UPackage* const* PkgPtr = CreatedPackages.Find(PackageName))
	{
		return *PkgPtr;
	}
	return nullptr;
}

void UE::Interchange::FImportAsyncHelper::AddCreatedPackage(const FString& PackageName, UPackage* Package)
{
	FScopeLock Lock(&CreatedPackagesLock);
	if (ensure(!CreatedPackages.Contains(PackageName)))
	{
		CreatedPackages.Add(PackageName, Package);
	}
}

UInterchangeFactoryBase* UE::Interchange::FImportAsyncHelper::GetCreatedFactory(const FString& FactoryNodeUniqueId) const
{
	FScopeLock Lock(&CreatedFactoriesLock);
	if (TObjectPtr<UInterchangeFactoryBase>const* FactoryPtr = CreatedFactories.Find(FactoryNodeUniqueId))
	{
		return *FactoryPtr;
	}
	return nullptr;
}

void UE::Interchange::FImportAsyncHelper::AddCreatedFactory(const FString& FactoryNodeUniqueId, UInterchangeFactoryBase* Factory)
{
	FScopeLock Lock(&CreatedFactoriesLock);
	if (ensure(!CreatedFactories.Contains(FactoryNodeUniqueId)))
	{
		CreatedFactories.Add(FactoryNodeUniqueId, Factory);
	}
}

UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& UE::Interchange::FImportAsyncHelper::AddDefaultImportedAssetGetRef(int32 SourceIndex)
{
	FScopeLock Lock(&ImportedAssetsPerSourceIndexLock);
	return ImportedAssetsPerSourceIndex.FindOrAdd(SourceIndex).AddDefaulted_GetRef();
}

const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo* UE::Interchange::FImportAsyncHelper::FindImportedAssets(int32 SourceIndex, TFunction< bool(const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ImportedObject) > Predicate) const
{
	FScopeLock Lock(&ImportedAssetsPerSourceIndexLock);
	if (!ImportedAssetsPerSourceIndex.Contains(SourceIndex))
	{
		return nullptr;
	}
	const TArray<FImportedObjectInfo>& ImportedObjectInfos = ImportedAssetsPerSourceIndex.FindChecked(SourceIndex);
	for (const FImportedObjectInfo& ImportObjectInfo : ImportedObjectInfos)
	{
		if (Predicate(ImportObjectInfo))
		{
			return &ImportObjectInfo;
		}
	}
	return nullptr;
}

void UE::Interchange::FImportAsyncHelper::IterateImportedAssets(int32 SourceIndex, TFunction< void(const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedObjects) > Callback) const
{
	FScopeLock Lock(&ImportedAssetsPerSourceIndexLock);
	
	if (!ImportedAssetsPerSourceIndex.Contains(SourceIndex))
	{
		return;
	}

	Callback(ImportedAssetsPerSourceIndex.FindChecked(SourceIndex));
}

void UE::Interchange::FImportAsyncHelper::IterateImportedAssetsPerSourceIndex(TFunction< void(int32 SourceIndex, const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedObjects) > Callback) const
{
	FScopeLock Lock(&ImportedAssetsPerSourceIndexLock);
	for (const TPair<int32, TArray<FImportedObjectInfo>>& SourceIndexAndImportedAssets : ImportedAssetsPerSourceIndex)
	{
		Callback(SourceIndexAndImportedAssets.Key, SourceIndexAndImportedAssets.Value);
	}
}

UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& UE::Interchange::FImportAsyncHelper::AddDefaultImportedSceneObjectGetRef(int32 SourceIndex)
{
	FScopeLock Lock(&ImportedSceneObjectsPerSourceIndexLock);
	return ImportedSceneObjectsPerSourceIndex.FindOrAdd(SourceIndex).AddDefaulted_GetRef();
}

const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo* UE::Interchange::FImportAsyncHelper::FindImportedSceneObjects(int32 SourceIndex, TFunction< bool(const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ImportedObject) > Predicate) const
{
	FScopeLock Lock(&ImportedSceneObjectsPerSourceIndexLock);
	if (!ImportedSceneObjectsPerSourceIndex.Contains(SourceIndex))
	{
		return nullptr;
	}
	const TArray<FImportedObjectInfo>& ImportedObjectInfos = ImportedSceneObjectsPerSourceIndex.FindChecked(SourceIndex);
	for (const FImportedObjectInfo& ImportObjectInfo : ImportedObjectInfos)
	{
		if (Predicate(ImportObjectInfo))
		{
			return &ImportObjectInfo;
		}
	}
	return nullptr;
}

void UE::Interchange::FImportAsyncHelper::IterateImportedSceneObjects(int32 SourceIndex, TFunction< void(const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedObjects) > Callback) const
{
	FScopeLock Lock(&ImportedSceneObjectsPerSourceIndexLock);

	if (!ImportedSceneObjectsPerSourceIndex.Contains(SourceIndex))
	{
		return;
	}

	Callback(ImportedSceneObjectsPerSourceIndex.FindChecked(SourceIndex));
}

void UE::Interchange::FImportAsyncHelper::IterateImportedSceneObjectsPerSourceIndex(TFunction< void(int32 SourceIndex, const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedObjects) > Callback) const
{
	FScopeLock Lock(&ImportedSceneObjectsPerSourceIndexLock);
	for (const TPair<int32, TArray<FImportedObjectInfo>>& SourceIndexAndImportedSceneObjects : ImportedSceneObjectsPerSourceIndex)
	{
		Callback(SourceIndexAndImportedSceneObjects.Key, SourceIndexAndImportedSceneObjects.Value);
	}
}

bool UE::Interchange::FImportAsyncHelper::IsImportingObject(UObject* Object) const
{
	if (!Object)
	{
		return false;
	}

	bool bFoundAsset = false;
	auto IsImportingAsset = [&bFoundAsset, Object](int32 SourceIndex, const TArray<FImportedObjectInfo>& ImportedObjects)
	{
		if (bFoundAsset)
		{
			return;
		}
		for (const FImportedObjectInfo& ImportedObjectInfo : ImportedObjects)
		{
			if (ImportedObjectInfo.GetImportedObject() == Object)
			{
				bFoundAsset = true;
				break;
			}
		}
	};
	//Asset import
	IterateImportedAssetsPerSourceIndex(IsImportingAsset);
	if (!bFoundAsset)
	{
		IterateImportedSceneObjectsPerSourceIndex(IsImportingAsset);
	}

	return bFoundAsset;
}

void UE::Interchange::FImportAsyncHelper::ReleaseTranslatorsSource()
{
	//Make sure Executing translator task are done before releasing the resource
	const int32 TranslatorCount = Translators.Num();
	for (int32 TranslatorIndex = 0; TranslatorIndex < TranslatorCount; ++TranslatorIndex)
	{
		const uint64 TaskId = TranslatorTasks.IsValidIndex(TranslatorIndex) ? TranslatorTasks[TranslatorIndex] : INTERCHANGE_INVALID_TASK_ID;
		UInterchangeTranslatorBase* BaseTranslator = Translators[TranslatorIndex];

		if (BaseTranslator)
		{
			//Wait until the translator is done before releasing the resource. This should not happen since resource are free during completion
			const EInterchangeTaskStatus TaskStatus = UE::Interchange::FInterchangeTaskSystem::Get().GetTaskStatus(TaskId);
			if (TaskStatus == EInterchangeTaskStatus::Executing)
			{
				ensure(TaskStatus != EInterchangeTaskStatus::Executing);
				UE::Interchange::FInterchangeTaskSystem::Get().WaitUntilTasksComplete({ TaskId });
			}
			BaseTranslator->ReleaseSource();
		}
	}
}

TArray<uint64> UE::Interchange::FImportAsyncHelper::GetCompletionTaskGraphEvent()
{
	TArray<uint64> TasksToComplete;

	TasksToComplete.Append(TranslatorTasks);
	TasksToComplete.Append(PipelineTasks);
	
	if (ParsingTask != INTERCHANGE_INVALID_TASK_ID)
	{
		TasksToComplete.Add(ParsingTask);
	}

	//Parsing task must be done before the other tasks get added
	UE::Interchange::FInterchangeTaskSystem::Get().WaitUntilTasksComplete(TasksToComplete);
	TasksToComplete.Reset();

	TasksToComplete.Append(ImportObjectQueryPayloadsTasks);
	TasksToComplete.Append(BeginImportObjectTasks);
	TasksToComplete.Append(ImportObjectTasks);
	TasksToComplete.Append(FinalizeImportObjectTasks);
	TasksToComplete.Append(SceneTasks);
	TasksToComplete.Append(WaitAssetCompilationTasks);
	TasksToComplete.Append(PostImportTasks);

	if (PreCompletionTask != INTERCHANGE_INVALID_TASK_ID)
	{
		TasksToComplete.Add(PreCompletionTask);
	}
	
	if (CompletionTask != INTERCHANGE_INVALID_TASK_ID)
	{
		//Completion task will make sure any created asset before canceling will be mark for delete
		TasksToComplete.Add(CompletionTask);
	}

	return TasksToComplete;
}

void UE::Interchange::FImportAsyncHelper::InitCancel()
{
	bCancel = true;
}

void UE::Interchange::FImportAsyncHelper::CleanUp()
{
	//Release the graph
	for (TStrongObjectPtr<UInterchangeBaseNodeContainer>& Container : BaseNodeContainers)
	{
		Container->IterateNodes([](const FString&, UInterchangeBaseNode* Node)
			{
				Node->ClearInternalFlags(EInternalObjectFlags::Async);
			}
		);
		Container->ClearInternalFlags(EInternalObjectFlags::Async);
		Container->ClearFlags(RF_Standalone);
	}
	BaseNodeContainers.Empty();

	for (UInterchangeSourceData* SourceData : SourceDatas)
	{
		if (SourceData)
		{
			SourceData->RemoveFromRoot();
			SourceData->ClearFlags(RF_Standalone);
		}
	}
	SourceDatas.Empty();

	for (UInterchangeTranslatorBase* Translator : Translators)
	{
		if(Translator)
		{
			Translator->ImportFinish();
			Translator->RemoveFromRoot();
			Translator->ClearFlags(RF_Standalone);
			Translator->ClearInternalFlags(EInternalObjectFlags::Async);
		}
	}
	Translators.Empty();

	for (UInterchangePipelineBase* Pipeline : Pipelines)
	{
		if(Pipeline)
		{
			Pipeline->RemoveFromRoot();
			Pipeline->ClearFlags(RF_Standalone);
			Pipeline->ClearInternalFlags(EInternalObjectFlags::Async);
		}
	}
	Pipelines.Empty();

	for (const auto& FactoryKeyAndValue : CreatedFactories)
	{
		if (FactoryKeyAndValue.Value)
		{
			FactoryKeyAndValue.Value->RemoveFromRoot();
			FactoryKeyAndValue.Value->ClearFlags(RF_Standalone);
			FactoryKeyAndValue.Value->ClearInternalFlags(EInternalObjectFlags::Async);
		}
	}
	CreatedFactories.Empty();
}

TMap<int32, TArray<TObjectPtr<UObject>>> UE::Interchange::FImportAsyncHelper::ExtractImportedObjectsPerSourceIndex(TMap<int32, TArray<FImportedObjectInfo>>& InObjectInfoPerSourceIndex)
{
	TMap<int32, TArray<TObjectPtr<UObject>>> ExtractedObjectsPerSourceIndex;
	for (const TPair<int32, TArray<FImportedObjectInfo>>& ImportedAssetPerSourceIndex : InObjectInfoPerSourceIndex)
	{
		TArray<TObjectPtr<UObject>>& ImportedObjects = ExtractedObjectsPerSourceIndex.FindOrAdd(ImportedAssetPerSourceIndex.Key);
		for (const FImportedObjectInfo& ImportObjectInfo : ImportedAssetPerSourceIndex.Value)
		{
			ImportedObjects.Add(ImportObjectInfo.GetImportedObject());
		}
	}

	return ExtractedObjectsPerSourceIndex;
}

UE::Interchange::FImportResult::FImportResult()
	: ImportStatus(EStatus::Invalid)
{
	Results = NewObject<UInterchangeResultsContainer>(GetTransientPackage());
}

UE::Interchange::FImportResult::EStatus UE::Interchange::FImportResult::GetStatus() const
{
	return ImportStatus;
}

bool UE::Interchange::FImportResult::IsValid() const
{
	return GetStatus() != EStatus::Invalid;
}

void UE::Interchange::FImportResult::SetInProgress()
{
	EStatus ExpectedStatus = EStatus::Invalid;
	ImportStatus.compare_exchange_strong(ExpectedStatus, EStatus::InProgress);
}

void UE::Interchange::FImportResult::SetDone()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FImportResult::SetDone);

	SetInProgress(); // Make sure we always pass through the InProgress state

	EStatus ExpectedStatus = EStatus::InProgress;
	if (ImportStatus.compare_exchange_strong(ExpectedStatus, EStatus::Done))
	{
		if (DoneCallback)
		{
			DoneCallback(*this);
		}

		TArray<UObject*> Objects = GetImportedObjects();

		if (IsInGameThread())
		{
			OnImportDoneNative.ExecuteIfBound(Objects);
			OnImportDone.ExecuteIfBound(Objects);
		}
		else
		{
			TArray<TWeakObjectPtr<UObject>> WeakObjects;
			WeakObjects.Reserve(Objects.Num());
			for (UObject* Object : Objects)
			{
				WeakObjects.Emplace(Object);
			}

			if(ensure(IsInGameThread()))
			{
				TArray<UObject*> ValidObjects;
				ValidObjects.Reserve(WeakObjects.Num());

				for (const TWeakObjectPtr<UObject>& WeakObject : WeakObjects)
				{
					if (UObject* ValidObject = WeakObject.Get())
					{
						ValidObjects.Add(ValidObject);
					}
				}

				OnImportDoneNative.ExecuteIfBound(ValidObjects);
				OnImportDone.ExecuteIfBound(ValidObjects);
			}
		}
	}
}

void UE::Interchange::FImportResult::WaitUntilDone(bool bSynchronous /*= false*/)
{
	if (ImportStatus == EStatus::InProgress)
	{
		//Pin the weak ptr, do not hold the shared ptr until the end of the import, simply get the completion task id
		TArray<uint64> TasksIds;
		{
			TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> PinAsyncHelper = AsyncHelper.Pin();
			if (PinAsyncHelper.IsValid())
			{
				TasksIds = PinAsyncHelper->GetCompletionTaskGraphEvent();
			}
		}
		//Wait until the whole import is done
		UE::Interchange::FInterchangeTaskSystem::Get().WaitUntilTasksComplete(TasksIds);
	}
}

const TArray< UObject* >& UE::Interchange::FImportResult::GetImportedObjects() const
{
	FReadScopeLock ReadScopeLock(ImportedObjectsRWLock);
	return ObjectPtrDecay(ImportedObjects);
}

UObject* UE::Interchange::FImportResult::GetFirstAssetOfClass(UClass* InClass) const
{
	UObject* Asset = nullptr;

	FReadScopeLock ReadScopeLock(ImportedObjectsRWLock);
	for (UObject* ImportedAsset : ImportedObjects)
	{
		if (ImportedAsset->IsA(InClass))
		{
			Asset = ImportedAsset;
			break;
		}
	}

	return Asset;
}

void UE::Interchange::FImportResult::AddImportedObject(UObject* ImportedObject)
{
	{
		FWriteScopeLock WriteScopeLock(ImportedObjectsRWLock);
		ImportedObjects.Add(ImportedObject);
	}

	if (IsInGameThread())
	{
		OnObjectDoneNative.ExecuteIfBound(ImportedObject);
		OnObjectDone.ExecuteIfBound(ImportedObject);
	}
	else
	{
		// call the callbacks on the game thread
		Async(EAsyncExecution::TaskGraphMainThread, [WeakImportedObject = TWeakObjectPtr<UObject>(ImportedObject), ObjectDoneNative = OnObjectDoneNative, ObjectDone = OnObjectDone] ()
			{
				if (UObject* ImportedObjectInGameThread = WeakImportedObject.Get())
				{
					ObjectDoneNative.ExecuteIfBound(ImportedObjectInGameThread);
					ObjectDone.ExecuteIfBound(ImportedObjectInGameThread);
				}
			});
	}
}

void UE::Interchange::FImportResult::OnDone(TFunction< void(FImportResult&) > Callback)
{
	DoneCallback = Callback;
	//In case the import is already done (because it was synchronous) execute the new OnDone callback
	if (ImportStatus == EStatus::Done)
	{
		if (DoneCallback)
		{
			DoneCallback(*this);
		}
	}
}

void UE::Interchange::FImportResult::AddReferencedObjects(FReferenceCollector& Collector)
{
	FReadScopeLock ReadScopeLock(ImportedObjectsRWLock);
	Collector.AddReferencedObjects(ImportedObjects);
	Collector.AddReferencedObject(Results);
}

void UE::Interchange::FImportResult::SetAsyncHelper(TWeakPtr<FImportAsyncHelper> InAsyncHelper)
{
	AsyncHelper = InAsyncHelper;
}

UInterchangePipelineBase* UE::Interchange::GeneratePipelineInstance(const FSoftObjectPath& PipelineInstance)
{
	UPackage* PipelineInstancePackage = GetTransientPackage();
	if (!ensure(PipelineInstancePackage))
	{
		return nullptr;
	}
	UObject* ReferenceInstance = PipelineInstance.TryLoad();
	if (!ReferenceInstance)
	{
		return nullptr;
	}
	UInterchangePipelineBase* GeneratedPipeline = nullptr;
	if (const UInterchangeBlueprintPipelineBase* BlueprintPipeline = Cast<UInterchangeBlueprintPipelineBase>(ReferenceInstance))
	{
		if (BlueprintPipeline->GeneratedClass.Get())
		{
			GeneratedPipeline = NewObject<UInterchangePipelineBase>(PipelineInstancePackage, BlueprintPipeline->GeneratedClass);
#if WITH_EDITORONLY_DATA
			GeneratedPipeline->OriginalPipelinePath = FSoftObjectPath(BlueprintPipeline);
#endif
		}
		else
		{
			//Log an error because we cannot load the python class, maybe the python script was not loaded
			UE_LOGF(LogInterchangeEngine, Error, "Cannot generate a pipeline instance because the Blueprint %ls does not have a valid generated class.", *PipelineInstance.GetWithoutSubPath().ToString());
		}
	}
	else if (const UInterchangePythonPipelineAsset* PythonPipeline = Cast<UInterchangePythonPipelineAsset>(ReferenceInstance))
	{
		if (PythonPipeline->GeneratedPipeline)
		{
			GeneratedPipeline = DuplicateObject<UInterchangePipelineBase>(PythonPipeline->GeneratedPipeline.Get(), PipelineInstancePackage);
#if WITH_EDITORONLY_DATA
			GeneratedPipeline->OriginalPipelinePath = FSoftObjectPath(PythonPipeline);
#endif
		}
		else
		{
			//Log an error because we cannot load the python class, maybe the python script was not loaded
			UE_LOGF(LogInterchangeEngine, Error, "Cannot generate a pipeline instance because the Python pipeline asset %ls does not have a valid generated pipeline instance.", *PipelineInstance.GetWithoutSubPath().ToString());
		}
	}
	else if (const UInterchangePipelineBase* DefaultPipeline = Cast<UInterchangePipelineBase>(ReferenceInstance))
	{
		GeneratedPipeline = DuplicateObject<UInterchangePipelineBase>(DefaultPipeline, PipelineInstancePackage);
#if WITH_EDITORONLY_DATA
		if (DefaultPipeline->OriginalPipelinePath.IsValid())
		{
			GeneratedPipeline->OriginalPipelinePath = DefaultPipeline->OriginalPipelinePath;
		}
		else
		{
			GeneratedPipeline->OriginalPipelinePath = FSoftObjectPath(DefaultPipeline);
		}
#endif
	}
	else
	{
		UE_LOGF(LogInterchangeEngine, Error, "Cannot generate a pipeline instance because the pipeline asset %ls type is unknown.", *PipelineInstance.GetWithoutSubPath().ToString());
	}

	if(GeneratedPipeline)
	{
		// Make sure that the instance does not carry over standalone and public flags as they are not actual assets to be persisted
		GeneratedPipeline->ClearFlags(EObjectFlags::RF_Standalone|EObjectFlags::RF_Public|EObjectFlags::RF_Transactional);
	}

	return GeneratedPipeline;
}

void UInterchangePipelineStackOverride::AddPythonPipeline(UInterchangePythonPipelineBase* PipelineBase)
{
	OverridePipelines.Add(PipelineBase);
}

void UInterchangePipelineStackOverride::AddBlueprintPipeline(UInterchangeBlueprintPipelineBase* PipelineBase)
{
	OverridePipelines.Add(PipelineBase);
}

void UInterchangePipelineStackOverride::AddPipeline(UInterchangePipelineBase* PipelineBase)
{
	OverridePipelines.Add(PipelineBase);
}

UInterchangeManager::UInterchangeManager(const FObjectInitializer& ObjectInitializer)
	: Super(ObjectInitializer)
{
	//Client must use the singleton API
	if (!bIsCreatingSingleton && !HasAnyFlags(RF_ClassDefaultObject | RF_ArchetypeObject))
	{
		UE_LOGF(LogInterchangeEngine, Error, "Interchange manager is a singleton you must call GetInterchangeManager() or GetInterchangeManagerScripted() to access it.");
	}
}

UInterchangeManager& UInterchangeManager::GetInterchangeManager()
{
	static TStrongObjectPtr<UInterchangeManager> InterchangeManager = nullptr;

	//This boolean will be true after we delete the singleton
	static bool InterchangeManagerScopeOfLifeEnded = false;

	if (!InterchangeManager.IsValid())
	{
		if ( InterchangeManagerScopeOfLifeEnded )
		{
			#if 1

			//Avoid hard crash if someone call the manager after we delete it, but send a callstack to the crash manager
			ensure(!InterchangeManagerScopeOfLifeEnded);

			#else

			// -> no, this is being called after "LogExit: Exiting."
			//	at that point you cannot check or ensure or log or anything
			//	do a low level break instead

			PLATFORM_BREAK(); //__debugbreak();

			// just return a nullptr ; you will crash after returning
			// this return is just so you can step out in the debugger to see who's calling this
			return *(InterchangeManager.Get());

			#endif
		}

		//We cannot create a TStrongObjectPtr outside of the main thread, we also need a valid Transient package
		check(IsInGameThread() && GetTransientPackage());

		bIsCreatingSingleton = true;

		InterchangeManager = TStrongObjectPtr<UInterchangeManager>(NewObject<UInterchangeManager>(GetTransientPackage(), NAME_None, EObjectFlags::RF_NoFlags));

		bIsCreatingSingleton = false;

		//Make sure we have a valid editor utilities
		InterchangeManager->SetEditorUtilities(UInterchangeEditorUtilitiesBase::StaticClass());

		FDelegateHandle OnTaskSystemTickDelegateHandle = UE::Interchange::FInterchangeTaskSystem::Get().OnTaskSystemTickDelegate().AddLambda([]()
			{
				//Tick task graph game thread tasks
				FTaskGraphInterface::Get().ProcessThreadUntilIdle(ENamedThreads::GameThread);
				//Tick the all compiling manager
				FAssetCompilingManager::Get().ProcessAsyncTasks(true);
			});
		//We cancel any running task when we pre exit the engine
		FCoreDelegates::OnEnginePreExit.AddLambda([OnTaskSystemTickDelegateHandle]()
		{
			//InterchangeManager should be valid at this point since this lambda is where the strong reference pointer get reset.
			if (!ensure(InterchangeManager.IsValid()))
			{
				InterchangeManagerScopeOfLifeEnded = true;
				return;
			}

			UE::Interchange::FInterchangeTaskSystem::Get().OnTaskSystemTickDelegate().Remove(OnTaskSystemTickDelegateHandle);

			if (IsInterchangeImportEnabled())
			{
				//If a user run a editor commandlet, we want to finish the import before the editor close.
				//In editor mode, the user cannot close the editor if an import task is running, so we should not endup here.
				const bool bCancel = !GIsEditor;
				//Synchronously wait all task to finish
				InterchangeManager->WaitUntilAllTasksDone(bCancel);
			}

			//Task should have been cancel in the Engine pre exit callback.
			ensure(InterchangeManager->GetImportTaskCount() == 0);
			InterchangeManager->OnPreDestroyInterchangeManager.Broadcast();

			if (InterchangeManager->QueuedPostImportTasksTickerHandle.IsValid())
			{
				FTSTicker::RemoveTicker(InterchangeManager->QueuedPostImportTasksTickerHandle);
				InterchangeManager->QueuedPostImportTasksTickerHandle.Reset();
			}

			//Release the editor utilities resources
			InterchangeManager->EditorUtilities.Reset();

			//Release the InterchangeManager object
			InterchangeManager.Reset();
			InterchangeManagerScopeOfLifeEnded = true;
		});
	}

	//When we get here we should be valid
	check(InterchangeManager.IsValid());

	return *(InterchangeManager.Get());
}

bool UInterchangeManager::RegisterTranslator(const UClass* TranslatorClass)
{
	if (!TranslatorClass)
	{
		return false;
	}

	RegisteredTranslatorsClass.Add(TranslatorClass);
	return true;
}

bool UInterchangeManager::RegisterFactory(const UClass* FactoryClass)
{
	if (!FactoryClass || !FactoryClass->IsChildOf<UInterchangeFactoryBase>())
	{
		return false;
	}

	UClass* ClassToMake = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>()->GetFactoryClass();
	if (ClassToMake)
	{
		if (!RegisteredFactoryClasses.Contains(ClassToMake))
		{
			RegisteredFactoryClasses.Add(ClassToMake, FactoryClass);
		}

		return true;
	}

	return false;
}

bool UInterchangeManager::RegisterWriter(const UClass* WriterClass)
{
#if WITH_EDITOR
	if (!WriterClass)
	{
		return false;
	}

	if (RegisteredWriters.Contains(WriterClass))
	{
		return true;
	}
	UInterchangeWriterBase* WriterToRegister = NewObject<UInterchangeWriterBase>(GetTransientPackage(), WriterClass, NAME_None);
	if (!WriterToRegister)
	{
		return false;
	}
	RegisteredWriters.Add(WriterClass, WriterToRegister);
#endif
	return true;
}

bool UInterchangeManager::RegisterImportDataConverter(const UClass* Converter)
{
#if WITH_EDITOR
	if (!Converter)
	{
		return false;
	}

	if (RegisteredConverters.Contains(Converter))
	{
		return true;
	}
	UInterchangeAssetImportDataConverterBase* ConverterToRegister = NewObject<UInterchangeAssetImportDataConverterBase>(GetTransientPackage(), Converter, NAME_None);
	if (!ConverterToRegister)
	{
		return false;
	}
	RegisteredConverters.Add(Converter, ConverterToRegister);
#endif
	return true;
}

bool UInterchangeManager::ConvertImportData(UObject* Object, const FString& Extension) const
{
	if (!Object)
	{
		return false;
	}

	for (TPair<TObjectPtr<const UClass>, TObjectPtr<UInterchangeAssetImportDataConverterBase>> RegisteredConverter : RegisteredConverters)
	{
		if (RegisteredConverter.Value->ConvertImportData(Object, Extension))
		{
			return true;
		}
	}
	return false;
}

bool UInterchangeManager::ConvertImportData(const UObject* SourceImportData, const UClass* DestinationClass, UObject** DestinationImportData) const
{
	if (!SourceImportData)
	{
		return false;
	}

	for (TPair<TObjectPtr<const UClass>, TObjectPtr<UInterchangeAssetImportDataConverterBase>> RegisteredConverter : RegisteredConverters)
	{
		if (RegisteredConverter.Value->CanConvertClass(SourceImportData->GetClass(), DestinationClass))
		{
			if (RegisteredConverter.Value->ConvertImportData(SourceImportData, DestinationClass, DestinationImportData))
			{
				break;
			}
		}
	}

	if (!(*DestinationImportData))
	{
		return false;
	}
	return true;
}

bool UInterchangeManager::ConvertImportData(const UObject* SourceImportData, FImportAssetParameters& ImportAssetParameters) const
{
	UObject* DestinationImportData = nullptr;
	if (!ConvertImportData(SourceImportData, UInterchangeAssetImportData::StaticClass(), &DestinationImportData))
	{
		return false;
	}

	if (!ensure(DestinationImportData))
	{
		return false;
	}

	if (UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(DestinationImportData))
	{
		//We can use the default pipeline stack, if it contain a pipeline that match the converted pipeline class
		bool bUseDefaultPipelineStack = false;
		TArray<UInterchangePipelineBase*> DuplicateDefaultPipelines;
		//Get the Interchange Default stack
		constexpr bool bIsSceneImport = false;
		const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bIsSceneImport);
		//Verify if we can use the default stack or not
		if (AssetImportData->GetNumberOfPipelines() == 1
			&& InterchangeImportSettings.PipelineStacks.Contains(InterchangeImportSettings.DefaultPipelineStack))
		{
			if (UInterchangePipelineBase* ConvertedPipeline = Cast<UInterchangePipelineBase>(AssetImportData->GetPipelines()[0]))
			{
				UClass* ConvertedPipelineClass = ConvertedPipeline->GetClass();
				const FInterchangePipelineStack& PipelineStack = InterchangeImportSettings.PipelineStacks.FindChecked(InterchangeImportSettings.DefaultPipelineStack);
				for (const FSoftObjectPath& PipelinePath : PipelineStack.Pipelines)
				{
					if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(PipelinePath))
					{
						FInterchangePipelineContextParams ContextParams;
						ContextParams.ContextType = EInterchangePipelineContext::AssetImport;
						GeneratedPipeline->AdjustSettingsForContext(ContextParams);
						if (GeneratedPipeline->IsA(ConvertedPipelineClass))
						{
							//We found a match, so we will use the default pipeline stacks
							bUseDefaultPipelineStack = true;
							DuplicateDefaultPipelines.Add(ConvertedPipeline);
						}
						else
						{
							DuplicateDefaultPipelines.Add(GeneratedPipeline);
						}
					}
				}
			}
		}

		if (bUseDefaultPipelineStack)
		{
			for (UInterchangePipelineBase* Pipeline : DuplicateDefaultPipelines)
			{
				ImportAssetParameters.OverridePipelines.Add(Pipeline);
			}
		}
		else
		{
			for (UObject* Pipeline : AssetImportData->GetPipelines())
			{
				ImportAssetParameters.OverridePipelines.Add(Pipeline);
			}
		}
		return true;
	}
	return false;
}

TArray<FString> UInterchangeManager::GetSupportedFormats(const EInterchangeTranslatorType ForTranslatorType) const
{
	TArray<FString> FileExtensions;
	if (!IsInterchangeImportEnabled())
	{
		return FileExtensions;
	}

	for (const UClass* TranslatorClass : RegisteredTranslatorsClass)
	{
		const UInterchangeTranslatorBase* TranslatorBaseCDO = TranslatorClass->GetDefaultObject<UInterchangeTranslatorBase>();

		if (EnumHasAllFlags(TranslatorBaseCDO->GetTranslatorType(), ForTranslatorType))
		{
			FileExtensions.Append(TranslatorBaseCDO->GetSupportedFormats());
		}
	}

	return FileExtensions;
}

TArray<FString> UInterchangeManager::GetSupportedAssetTypeFormats(const EInterchangeTranslatorAssetType ForTranslatorAssetType, const EInterchangeTranslatorType ForTranslatorType /*= EInterchangeTranslatorType::Invalid*/, bool bStrictMatchTranslatorType /*= false*/) const
{
	TArray<FString> FileExtensions;
	if (!IsInterchangeImportEnabled())
	{
		return FileExtensions;
	}

	for (const UClass* TranslatorClass : RegisteredTranslatorsClass)
	{
		const UInterchangeTranslatorBase* TranslatorBaseCDO = TranslatorClass->GetDefaultObject<UInterchangeTranslatorBase>();

		if (TranslatorBaseCDO->DoesSupportAssetType(ForTranslatorAssetType))
		{
			bool bAddTranslatorSupportedFormats = true;
			if (ForTranslatorType != EInterchangeTranslatorType::Invalid)
			{
				bAddTranslatorSupportedFormats = bStrictMatchTranslatorType
					? EnumHasAllFlags(TranslatorBaseCDO->GetTranslatorType(), ForTranslatorType)
					: EnumHasAnyFlags(TranslatorBaseCDO->GetTranslatorType(), ForTranslatorType);
			}

			if (bAddTranslatorSupportedFormats)
			{
				FileExtensions.Append(TranslatorBaseCDO->GetSupportedFormats());
			}
		}
	}

	return FileExtensions;
}

TArray<FString> UInterchangeManager::GetSupportedFormatsForObject(const UObject* Object, int32 SourceFileIndex) const
{
	TArray<FString> FileExtensions;
	if (!IsInterchangeImportEnabled())
	{
		return FileExtensions;
	}

	const UClass* RegisteredFactoryClass = GetRegisteredFactoryClass(Object->GetClass());
	if (!RegisteredFactoryClass)
	{
		return FileExtensions;
	}

	UInterchangeFactoryBase* Factory = RegisteredFactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
	TArray<FString> SourceFilenames;
	//GetSourceFilenames verify we have a valid UInterchangeAssetImportData for this Object
	//This ensure we do not allow re-import
	if (!Factory->GetSourceFilenames(Object, SourceFilenames))
	{
		return FileExtensions;
	}

	// SourceFileIndex is ensured to be a valid array index.
	if (SourceFileIndex < SourceFilenames.Num())
	{
		UE::Interchange::FScopedSourceData SourceData(SourceFilenames[SourceFileIndex]);
		{
			UE::Interchange::FScopedTranslator ScopedTranslator(SourceData.GetSourceData());
			if (UInterchangeTranslatorBase* Translator = ScopedTranslator.GetTranslator())
			{
				const EInterchangeTranslatorType TranslatorType = Translator->GetTranslatorType();
				constexpr bool bStrictMatchTranslatorType = false;

				switch (Factory->GetFactoryAssetType())
				{
				case EInterchangeFactoryAssetType::Animations:
					FileExtensions = GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Animations, TranslatorType, bStrictMatchTranslatorType);
					break;
				case EInterchangeFactoryAssetType::Materials:
					FileExtensions = GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Materials, TranslatorType, bStrictMatchTranslatorType);
					break;
				case EInterchangeFactoryAssetType::Meshes:
				case EInterchangeFactoryAssetType::Physics:
					FileExtensions = GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Meshes, TranslatorType, bStrictMatchTranslatorType);
					break;
				case EInterchangeFactoryAssetType::Textures:
					FileExtensions = GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Textures, TranslatorType, bStrictMatchTranslatorType);
					break;
				case EInterchangeFactoryAssetType::Grooms:
					FileExtensions = GetSupportedAssetTypeFormats(EInterchangeTranslatorAssetType::Grooms, TranslatorType, bStrictMatchTranslatorType);
					break;
				case EInterchangeFactoryAssetType::None: //Actor factories return None
					FileExtensions = GetSupportedFormats(EInterchangeTranslatorType::Actors);
					break;
				}
			}
		}
	}
	
	//Make sure we return lower case extensions
	for (FString& Extension : FileExtensions)
	{
		Extension.ToLowerInline();
	}

	return FileExtensions;
}

bool UInterchangeManager::CanTranslateSourceData(const UInterchangeSourceData* SourceData, bool bSceneImportOnly) const
{
	if (!IsInterchangeImportEnabled())
	{
		return false;
	}
#if WITH_EDITOR
	IAssetTools& AssetTools = FModuleManager::LoadModuleChecked<FAssetToolsModule>(TEXT("AssetTools")).Get();
	const FString Extension = FPaths::GetExtension(SourceData->GetFilename());
	if (!AssetTools.IsImportExtensionAllowed(Extension))
	{
		return false;
	}
#endif

	if (UInterchangeTranslatorBase* Translator = GetTranslatorForSourceData(SourceData))
	{
		Translator->ReleaseSource();
		return bSceneImportOnly ? Translator->GetTranslatorType() == EInterchangeTranslatorType::Scenes : true;
	}

	return false;
}

bool UInterchangeManager::CanReimport(const UObject* Object, TArray<FString>& OutFilenames) const
{
	if (!IsInterchangeImportEnabled())
	{
		return false;
	}

	const UClass* RegisteredFactoryClass = GetRegisteredFactoryClass(Object->GetClass());
	if (!RegisteredFactoryClass)
	{
		return false;
	}

	UInterchangeFactoryBase* Factory = RegisteredFactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
	if(!Factory->GetSourceFilenames(Object, OutFilenames))
	{
		return false;
	}

	for (const FString& Filename : OutFilenames)
	{
		UE::Interchange::FScopedSourceData ScopedSourceData(Filename);

		if (CanTranslateSourceData(ScopedSourceData.GetSourceData()))
		{
			return true;
		}
	}

	OutFilenames.Empty();
	return false;
}

void UInterchangeManager::StartQueuedTasks(bool bCancelAllTasks /*= false*/)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeManager::StartQueuedTasks)
		
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	ensure(IsInterchangeImportEnabled());
	if (!ensure(IsInGameThread()))
	{
		//Do not crash but we will not start any queued tasks if we are not in the game thread
		return;
	}

	//Garbage collect can stall and tick the task graph (if accessing a compiling asset locked UProperty )
	//We must avoid starting an import task in this case, import cannot be done when GC runs.
	//A delegate is implemented and call StartQueuedTasks when the GC is finish
	if(IsGarbageCollecting())
	{
		if (bCancelAllTasks)
		{
			bGCEndDelegateCancellAllTask = bCancelAllTasks;
		}
		return;
	}

	bGCEndDelegateCancellAllTask = false;

	uint64 LastNotificationFrame = 0;
	auto UpdateNotification = [this, &LastNotificationFrame]()
	{
		if (LastNotificationFrame == GFrameCounter)
		{
			return;
		}
		LastNotificationFrame = GFrameCounter;

		if (Notification.IsValid())
		{
			const int32 ImportTaskNumber = GetImportTaskCount() + QueueTaskCount;
			Notification->SetProgressText(FText::Format(
				NSLOCTEXT("Interchange", "Asynchronous_pending_import_tasks", "Importing ({0})"),
					ImportTaskNumber)
				);
		}
		else
		{
			bool bCanShowNotification = false;
			{
				int32 ImportTaskCount = GetImportTaskCount();
				for (int32 TaskIndex = 0; TaskIndex < ImportTaskCount; ++TaskIndex)
				{
					TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = GetImportTaskForIndex(TaskIndex);
					if (AsyncHelper.IsValid())
					{
						//Allow notification if at least one task is not automated
						if (!AsyncHelper->TaskData.bIsAutomated)
						{
							bCanShowNotification = true;
							break;
						}
					}
				}
			}
			if (bCanShowNotification)
			{
				FText TitleText = NSLOCTEXT("Interchange", "Asynchronous_import_start", "Importing");
				FAsyncTaskNotificationConfig NotificationConfig;
				NotificationConfig.bIsHeadless = false;
				NotificationConfig.TitleText = TitleText;
				NotificationConfig.LogCategory = UE::Interchange::Private::GetLogInterchangePtr();
				NotificationConfig.bCanCancel.Set(true);
				NotificationConfig.bKeepOpenOnSuccess.Set(false);
				NotificationConfig.bKeepOpenOnFailure.Set(false);

				Notification = MakeShared<FAsyncTaskNotification>(NotificationConfig);
				Notification->SetNotificationState(FAsyncNotificationStateData(TitleText, FText::GetEmpty(), EAsyncTaskNotificationState::Pending));
			}
		}
	};


	//We need to leave some free task in the pool to avoid deadlock.
	//Each import can use 2 tasks in same time if the build of the asset ddc use the same task pool (i.e. staticmesh, skeletalmesh, texture...)
	const int32 PoolWorkerThreadCount = FTaskGraphInterface::Get().GetNumWorkerThreads() / 2;
	const int32 MaxNumWorker = FMath::Max(PoolWorkerThreadCount, 1);

	for (TPair<UClass*, TArray<FQueuedTaskData>>& ClassAndTasks : NonParallelTranslatorQueueTasks)
	{
		if(ClassAndTasks.Value.IsEmpty())
		{
			continue;
		}
		if (bCancelAllTasks)
		{
			//Enqueue all the tasks they will be all cancel
			for (FQueuedTaskData QueuedTaskData : ClassAndTasks.Value)
			{
				QueuedTasks.Enqueue(QueuedTaskData);
			}
			ClassAndTasks.Value.Reset();
		}
		else
		{
			//Lock the translator and enqueue only the first task
			bool& TranslatorLock = NonParallelTranslatorLocks.FindChecked(ClassAndTasks.Key);
			if (!TranslatorLock)
			{
				FQueuedTaskData QueuedTaskData = ClassAndTasks.Value[0];
				QueuedTasks.Enqueue(QueuedTaskData);
				TranslatorLock = true;
				ClassAndTasks.Value.RemoveAt(0, EAllowShrinking::No);
				//No need to process an another the lock is set
				continue;
			}
		}
	}

	while (!QueuedTasks.IsEmpty() && (GetImportTaskCount() < MaxNumWorker || bCancelAllTasks))
	{
		FQueuedTaskData QueuedTaskData;
		if (QueuedTasks.Dequeue(QueuedTaskData))
		{
			QueueTaskCount = FMath::Clamp(QueueTaskCount-1, 0, MAX_int32);
			check(QueuedTaskData.AsyncHelper.IsValid());

			int32 AsyncHelperIndex = AddImportTask(QueuedTaskData.AsyncHelper);
			SetActiveMode(true);
			//Update the asynchronous notification
			UpdateNotification();

			TWeakPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper = QueuedTaskData.AsyncHelper;
			
			if (bCancelAllTasks)
			{
				QueuedTaskData.AsyncHelper->InitCancel();
			}

			//Create/Start import tasks
			TArray<uint64> PipelinePrerequistes;
			if(QueuedTaskData.AsyncHelper->TranslatorTasks.Num() == 0)
			{
				check(QueuedTaskData.AsyncHelper->Translators.Num() == QueuedTaskData.AsyncHelper->SourceDatas.Num());
				for (int32 SourceDataIndex = 0; SourceDataIndex < QueuedTaskData.AsyncHelper->SourceDatas.Num(); ++SourceDataIndex)
				{
					//Log the source we begin importing
					UE_LOGF(LogInterchangeEngine, Display, "Interchange start importing source [%ls]", *QueuedTaskData.AsyncHelper->SourceDatas[SourceDataIndex]->GetFilename());
					TSharedPtr<UE::Interchange::FTaskTranslator, ESPMode::ThreadSafe> TaskTranslator = MakeShared<UE::Interchange::FTaskTranslator, ESPMode::ThreadSafe>(SourceDataIndex, WeakAsyncHelper);
					int32 TranslatorTaskIndex = QueuedTaskData.AsyncHelper->TranslatorTasks.Add(UE::Interchange::FInterchangeTaskSystem::Get().AddTask(TaskTranslator));
					PipelinePrerequistes.Add(QueuedTaskData.AsyncHelper->TranslatorTasks[TranslatorTaskIndex]);
				}
			}

			TArray<uint64> GraphParsingPrerequistes;
			for (int32 GraphPipelineIndex = 0; GraphPipelineIndex < QueuedTaskData.AsyncHelper->Pipelines.Num(); ++GraphPipelineIndex)
			{
				UInterchangePipelineBase* GraphPipeline = QueuedTaskData.AsyncHelper->Pipelines[GraphPipelineIndex];
				TWeakObjectPtr<UInterchangePipelineBase> WeakPipelinePtr = GraphPipeline;

				TSharedPtr<UE::Interchange::FTaskPipeline, ESPMode::ThreadSafe> TaskPipeline = MakeShared<UE::Interchange::FTaskPipeline, ESPMode::ThreadSafe>(WeakPipelinePtr, WeakAsyncHelper);
				int32 GraphPipelineTaskIndex = QueuedTaskData.AsyncHelper->PipelineTasks.Add(UE::Interchange::FInterchangeTaskSystem::Get().AddTask(TaskPipeline, PipelinePrerequistes));
				//Ensure we run the pipeline in the same order we create the task, since pipeline modify the node container, its important that its not process in parallel, Adding the one we start to the prerequisites
				//is the way to go here
				PipelinePrerequistes.Add(QueuedTaskData.AsyncHelper->PipelineTasks[GraphPipelineTaskIndex]);

				//Add pipeline to the graph parsing prerequisites
				GraphParsingPrerequistes.Add(QueuedTaskData.AsyncHelper->PipelineTasks[GraphPipelineTaskIndex]);
			}

			if (GraphParsingPrerequistes.Num() == 0)
			{
				//Fallback on the translator pipeline prerequisites (translator must be done if there is no pipeline)
				GraphParsingPrerequistes.Append(PipelinePrerequistes);
			}

			TSharedPtr<UE::Interchange::FTaskParsing, ESPMode::ThreadSafe> TaskParsing = MakeShared<UE::Interchange::FTaskParsing, ESPMode::ThreadSafe>(this, WeakAsyncHelper);
			QueuedTaskData.AsyncHelper->ParsingTask = UE::Interchange::FInterchangeTaskSystem::Get().AddTask(TaskParsing, GraphParsingPrerequistes);

			//The graph parsing task will create the FCreateAssetTask that will run after them, the FAssetImportTask will call the appropriate Post asset import pipeline when the asset is completed
		}
	}

	if (!QueuedTasks.IsEmpty())
	{
		//Make sure any task we add is count in the task to do, even if we cannot start it
		UpdateNotification();
	}
}

bool UInterchangeManager::ImportAsset(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters, TArray<UObject*>& OutImportedObjects)
{
	ImportAssetParameters.bRunSynchronous = true;
	UE::Interchange::FAssetImportResultRef InterchangeResult = ImportInternal(ContentPath, SourceData, ImportAssetParameters, UE::Interchange::EImportType::ImportType_Asset).Get<0>();
	InterchangeResult->WaitUntilDone(ImportAssetParameters.bRunSynchronous);
	OutImportedObjects = InterchangeResult->GetImportedObjects();
	return InterchangeResult->IsValid();
}

bool UInterchangeManager::ImportAsset(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	TArray<UObject*> ImportedObjects;
	return ImportAsset(ContentPath, SourceData, ImportAssetParameters, ImportedObjects);
}

UE::Interchange::FAssetImportResultRef UInterchangeManager::ImportAssetWithResult(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	ImportAssetParameters.bRunSynchronous = true;
	UE::Interchange::FAssetImportResultRef InterchangeResult = ImportInternal(ContentPath, SourceData, ImportAssetParameters, UE::Interchange::EImportType::ImportType_Asset).Get<0>();
	InterchangeResult->WaitUntilDone(ImportAssetParameters.bRunSynchronous);
	return InterchangeResult;
}

UE::Interchange::FAssetImportResultRef UInterchangeManager::ImportAssetAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	ImportAssetParameters.bRunSynchronous = false;
	return ImportInternal(ContentPath, SourceData, ImportAssetParameters, UE::Interchange::EImportType::ImportType_Asset).Get<0>();
}

bool UInterchangeManager::ScriptedImportAssetAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	UE::Interchange::FAssetImportResultRef InterchangeResult = ImportAssetAsync(ContentPath, SourceData, ImportAssetParameters);
	return InterchangeResult->IsValid();
}

TOptional<FString> UInterchangeManager::ValidateReimportParameter(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters, bool bRunSynchronous)
{

	TArray<FString> Filenames;
	if (!ObjectToReimport || !CanReimport(ObjectToReimport, Filenames))
	{
		return TOptional<FString>();
	}
	int32 ReimportIndex = ImportAssetParameters.ReimportSourceIndex == INDEX_NONE ? 0 : ImportAssetParameters.ReimportSourceIndex;
	if (!Filenames.IsValidIndex(ReimportIndex))
	{
		return TOptional<FString>();
	}
	if (FImportAssetParameters* MutableImportAssetParameters = const_cast<FImportAssetParameters*>(&ImportAssetParameters))
	{
		MutableImportAssetParameters->bRunSynchronous = bRunSynchronous;
		MutableImportAssetParameters->ReimportAsset = ObjectToReimport;
	}
	return Filenames[ReimportIndex];
}

bool UInterchangeManager::ReimportAsset(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters, TArray<UObject*>& OutImportedObjects)
{
	constexpr bool bRunSynchronousTrue = true;
	TOptional<FString> Filename = ValidateReimportParameter(ObjectToReimport, ImportAssetParameters, bRunSynchronousTrue);
	if (!Filename.IsSet())
	{
		return false;
	}
	UE::Interchange::FScopedSourceData SourceData(Filename.GetValue());
	UE::Interchange::FAssetImportResultRef InterchangeResult = ImportInternal(FString(), SourceData.GetSourceData(), ImportAssetParameters, UE::Interchange::EImportType::ImportType_Asset).Get<0>();
	InterchangeResult->WaitUntilDone(ImportAssetParameters.bRunSynchronous);
	OutImportedObjects = InterchangeResult->GetImportedObjects();
	return InterchangeResult->IsValid();
}

UE::Interchange::FAssetImportResultRef UInterchangeManager::ReimportAssetAsync(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters)
{
	UE::Interchange::FAssetImportResultRef ErrorImportResult = UE::Interchange::FAssetImportResultRef { MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() };
	ErrorImportResult->SetDone();
	constexpr bool bRunSynchronousFalse = false;
	TOptional<FString> Filename = ValidateReimportParameter(ObjectToReimport, ImportAssetParameters, bRunSynchronousFalse);
	if (!Filename.IsSet())
	{
		return ErrorImportResult;
	}
	UE::Interchange::FScopedSourceData SourceData(Filename.GetValue());
	return ImportInternal(FString(), SourceData.GetSourceData(), ImportAssetParameters, UE::Interchange::EImportType::ImportType_Asset).Get<0>();
}

bool UInterchangeManager::ScriptedReimportAssetAsync(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters)
{
	UE::Interchange::FAssetImportResultRef InterchangeResult = ReimportAssetAsync(ObjectToReimport, ImportAssetParameters);
	return InterchangeResult->IsValid();
}


bool UInterchangeManager::ImportScene(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	ImportAssetParameters.bRunSynchronous = true;
	using namespace UE::Interchange;
	TTuple<FAssetImportResultRef, FSceneImportResultRef> ImportResults = ImportInternal(ContentPath, SourceData, ImportAssetParameters, UE::Interchange::EImportType::ImportType_Scene);
	ImportResults.Get<0>()->WaitUntilDone(ImportAssetParameters.bRunSynchronous);
	ImportResults.Get<1>()->WaitUntilDone(ImportAssetParameters.bRunSynchronous);
	return ImportResults.Get<0>()->IsValid() && ImportResults.Get<1>()->IsValid();
}

bool UInterchangeManager::ScriptedImportSceneAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	using namespace UE::Interchange;
	TTuple<FAssetImportResultRef, FSceneImportResultRef> ImportResults = ImportSceneAsync(ContentPath, SourceData, ImportAssetParameters);
	return ImportResults.Get<0>()->IsValid() && ImportResults.Get<1>()->IsValid();
}

TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>
UInterchangeManager::ImportSceneAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters)
{
	ImportAssetParameters.bRunSynchronous = false;
	return ImportInternal(ContentPath, SourceData, ImportAssetParameters, UE::Interchange::EImportType::ImportType_Scene);
}

TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>
UInterchangeManager::ImportInternal(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters, const UE::Interchange::EImportType ImportType)
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UInterchangeManager::ImportInternal)
	
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	if (!ensure(IsInGameThread()))
	{
		UE_LOGF(LogInterchangeEngine, Error, "Cannot import file, the import process can be started only in the game thread.");
		return TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>{ MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >(), MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() };
	}

	// Check on validity of destination content folder on import not reimport
	if (!ImportAssetParameters.ReimportAsset && !FPackageName::IsValidPath(ContentPath))
	{
		UE_LOGF(LogInterchangeEngine, Error, "Destination content path, %ls, is not valid.", *ContentPath);
		return TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>{ MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >(), MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() };
	}

	ensure(IsInterchangeImportEnabled());

	static int32 GeneratedUniqueID = 0;
	int32 UniqueId = ++GeneratedUniqueID;

	
	TSharedRef<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = MakeShared<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe>();
	AsyncHelper->UniqueId = UniqueId;
	AsyncHelper->bRunSynchronous = ImportAssetParameters.bRunSynchronous;

	// Do not collect Analytics while automation testing.
	if (!GIsAutomationTesting)
	{
		AsyncHelper->AnalyticsHandler = CreateAnalyticsHandler(SourceData ? SourceData->GetFilename() : FString());
	}

	UE::Interchange::Private::FInterchangeAnalyticsHelper AnalyticsHelper(AsyncHelper->AnalyticsHandler.Get());

	const FString UsageImportAnalayticsId = "Interchange.Usage.Import";
	auto PreReturn = [&AnalyticsHelper]()
	{
		AnalyticsHelper.Flush();	
	};

	auto EarlyExit = [&PreReturn]()
	{
		PreReturn();
		TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef> ImportResult = TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>{ MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >(), MakeShared< UE::Interchange::FImportResult, ESPMode::ThreadSafe >() };
		ImportResult.Key->SetDone();
		ImportResult.Value->SetDone();
		return ImportResult;
	};

	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("UniqueId"), UniqueId));
	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("ImportType"), ImportType == UE::Interchange::EImportType::ImportType_Asset ? TEXT("Asset") : TEXT("Scene")));
	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.IsAutomated"), ImportAssetParameters.bIsAutomated));
	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.IsReimport"), ImportAssetParameters.ReimportAsset != nullptr));
	if (ImportAssetParameters.ReimportAsset != nullptr)
	{
		AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.ReimportSourceIndex"), ImportAssetParameters.ReimportSourceIndex));
	}
	const bool bIsPipelineOverride = ImportAssetParameters.OverridePipelines.Num() > 0;
	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.IsPipelineOverrided"), bIsPipelineOverride));
	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.bReplaceExisting"), ImportAssetParameters.bReplaceExisting));
	if (!ImportAssetParameters.DestinationName.IsEmpty())
	{
		AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.DestinationName"), ImportAssetParameters.DestinationName));
	}
	

	if (this != &GetInterchangeManager())
	{
		UE_LOGF(LogInterchangeEngine, Error, "Cannot import file, the interchange manager use to import this file is not the singleton, use GetInterchangeManager() or GetInterchangeManagerScripted() to acces the interchange manager singleton.");
		return EarlyExit();
	}

	if (!SourceData)
	{
		UE_LOGF(LogInterchangeEngine, Error, "Cannot import file. The source data is invalid.");
		return EarlyExit();
	}
	
	{
		UInterchangeTranslatorBase* Translator = GetTranslatorForSourceData(SourceData);
		if(!Translator)
		{
			UE_LOGF(LogInterchangeEngine, Error, "Cannot import file. The source data is not supported. Try enabling the [%ls] extension for Interchange.", *FPaths::GetExtension(SourceData->GetFilename()));
			return EarlyExit();
		}
		Translator->ReleaseSource();
	}

	AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("SourceExtension"), FPaths::GetExtension(SourceData->GetFilename())));

	bool bImportScene = ImportType == UE::Interchange::EImportType::ImportType_Scene;
	const FInterchangeImportSettings& InterchangeImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bImportScene);
	
	if (InterchangeImportSettings.PipelineStacks.Num() == 0)
	{
		UE_LOGF(LogInterchangeEngine, Error, "Cannot import file. There is no pipeline stack defined for the %ls import type.", bImportScene ? TEXT("scene") : TEXT("content"));
		return EarlyExit();
	}

	const bool bIsUnattended = FApp::IsUnattended() || FApp::IsGame() || GIsAutomationTesting || ImportAssetParameters.bIsAutomated;
	//If we detect the import will do a re-import of an existing asset, the ReimportAsset parameter should be filled
	//And we will also convert the import data. With this code this import will be considered a re-import.
	if (!ImportAssetParameters.ReimportAsset)
	{
		auto GetAssetNameFromFilename = [&bImportScene, &SourceData]()
			{
				FString AssetpackageName = FPaths::GetBaseFilename(SourceData->GetFilename());
				if (bImportScene)
				{
					AssetpackageName = TEXT("SceneImport_") + AssetpackageName;
				}
				return AssetpackageName;
			};
		FString AssetpackageName = ImportAssetParameters.DestinationName.IsEmpty() ? GetAssetNameFromFilename() : ImportAssetParameters.DestinationName;
		
		FString PackageBasePath = ContentPath;
		SanitizeNameInline(AssetpackageName, ESanitizeNameTypeFlags::ObjectName | ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);
		SanitizeNameInline(PackageBasePath, ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);
		FString FullPackagePath = FPaths::Combine(*PackageBasePath, *AssetpackageName);
		const bool bCanConvertAsset = bImportScene || !UE::Interchange::FPackageUtils::IsMapPackageAsset(FullPackagePath);
		if (bCanConvertAsset)
		{
			UPackage* Pkg = FindPackage(nullptr, *FullPackagePath);
			if (!Pkg)
			{
				//Try to load the package from disk
				Pkg = LoadPackage(nullptr, *FullPackagePath, LOAD_NoWarn | LOAD_Quiet);
			}

			if (Pkg)
			{
				UObject* ExistingAsset = StaticFindObject(nullptr, Pkg, *AssetpackageName);
				//If we found an asset we can re-import, convert set the param to re-import instead of override
				if (ExistingAsset && GetRegisteredFactoryClass(ExistingAsset->GetClass()))
				{
					//Scene conversion only support UInterchangeSceneImportAsset
					//Asset conversion support all registered factory class
					UInterchangeSceneImportAsset* SceneImportAsset = Cast<UInterchangeSceneImportAsset>(ExistingAsset);
					if (!bImportScene || SceneImportAsset)
					{
						bool bConvertToReimport = ImportAssetParameters.bReplaceExisting;
						if (!bIsUnattended && !ImportAssetParameters.bReplaceExisting)
						{
							FText OverrideDialogMessage = FText::Format(NSLOCTEXT("InterchangeManager", "OverrideAssetMessage", "This import will override asset '{0}', Do you want to convert this import to a re-import?")
								, FText::FromString(ExistingAsset->GetFullName()));
							EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNoCancel, OverrideDialogMessage);
							switch (DialogResult)
							{
							case EAppReturnType::Yes:
							{
								bConvertToReimport = true;
								break;
							}
							case EAppReturnType::No:
							{
								bConvertToReimport = false;
								break;
							}
							case EAppReturnType::Cancel:
							{
								UE_LOGF(LogInterchangeEngine, Display, "User cancel import of file [%ls].", *(SourceData->GetFilename()));
								return EarlyExit();
							}
							}
						}

						if (bConvertToReimport)
						{
							//Make sure we have the correct flag
							ExistingAsset->SetFlags(RF_Standalone | RF_Public);
							if (FImportAssetParameters* MutableImportAssetParameters = const_cast<FImportAssetParameters*>(&ImportAssetParameters))
							{
								MutableImportAssetParameters->ReimportAsset = ExistingAsset;
								MutableImportAssetParameters->ReimportSourceIndex = INDEX_NONE;
								MutableImportAssetParameters->bReplaceExisting |= bImportScene;
							}
							UInterchangeAssetImportData* MutableOriginalAssetImportData = UInterchangeAssetImportData::GetFromObject(ImportAssetParameters.ReimportAsset);
							if (!MutableOriginalAssetImportData)
							{
								//Convert the object asset import data to interchange
								ConvertImportData(ImportAssetParameters.ReimportAsset, FPaths::GetExtension(SourceData->GetFilename()));
							}

							//Add an analytic so we can know if a not re-import was turn into a re-import
							AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Parameters.ConvertToReimport"), true));
						}
					}
				}
			}
		}
	}

	//Set a default pipeline stack if none is valid
	if (!InterchangeImportSettings.PipelineStacks.Contains(InterchangeImportSettings.DefaultPipelineStack))
	{
		FInterchangeImportSettings& MutableInterchangeImportSettings = FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(bImportScene);
		TArray<FName> Keys;
		MutableInterchangeImportSettings.PipelineStacks.GetKeys(Keys);
		check(Keys.Num() > 0);
		MutableInterchangeImportSettings.DefaultPipelineStack = Keys[0];
	}

	UInterchangeAssetImportData* OriginalAssetImportData = UInterchangeAssetImportData::GetFromObject(ImportAssetParameters.ReimportAsset);
	FString ContentBasePath = ContentPath;
	if (!ImportAssetParameters.ReimportAsset)
	{
		SanitizeNameInline(ContentBasePath, ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);
	}
	else
	{
		ContentBasePath = FPaths::GetPath(ImportAssetParameters.ReimportAsset->GetPathName());

		// Check whether asset has been imported with a sub-folder
		FString SubPath;
		if (UInterchangeAssetImportData* ImportData = UInterchangeAssetImportData::GetFromObject(ImportAssetParameters.ReimportAsset))
		{
			UInterchangeBaseNodeContainer* AssetContainer = ImportData->GetNodeContainer();
			ensure(AssetContainer);

			FSoftObjectPath AssetObjectPath(ImportAssetParameters.ReimportAsset);

			AssetContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([&SubPath, &AssetObjectPath](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
				{
					if (SubPath.IsEmpty())
					{
						FSoftObjectPath ObjectPath;
						if (FactoryNode->GetCustomReferenceObject(ObjectPath))
						{
							if (ObjectPath == AssetObjectPath)
							{
								FactoryNode->GetCustomSubPath(SubPath);
							}
						}
					}
				});
		}

		// Remove sub-path from ContentBasePath
		if (!SubPath.IsEmpty())
		{
			ContentBasePath.LeftChopInline(SubPath.EndsWith(TEXT("/")) ? SubPath.Len() : SubPath.Len() + 1, EAllowShrinking::No);
		}
	}

	const bool bIsReimport = OriginalAssetImportData && OriginalAssetImportData->GetPipelines().Num() > 0;

	bool bImportAborted = false; // True when we're unable to go through with the import process

	//Create a task for every source data
	UE::Interchange::FImportAsyncHelperData TaskData;
	TaskData.bIsAutomated = ImportAssetParameters.bIsAutomated;
	TaskData.bFollowRedirectors = ImportAssetParameters.bFollowRedirectors;
	TaskData.ImportType = ImportType;
	TaskData.ReimportObject = ImportAssetParameters.ReimportAsset;
	TaskData.ImportLevel = ImportAssetParameters.ImportLevel;
	TaskData.DestinationName = ImportAssetParameters.DestinationName;
	TaskData.bReplaceExisting = ImportAssetParameters.bReplaceExisting;
	
	InitAsyncHelper(AsyncHelper, TaskData, ImportAssetParameters);
	
	//We support only one source currently

	//Create a duplicate of the source data, we need to be multithread safe so we copy it to control the life cycle. The async helper will hold it and delete it when the import task will be completed.
	UInterchangeSourceData* DuplicateSourceData = Cast<UInterchangeSourceData>(StaticDuplicateObject(SourceData, GetTransientPackage()));
	//Array of source data to build one graph per source
	AsyncHelper->SourceDatas.Add(DuplicateSourceData);

	//Get the first source data translator, we currently do not support more then one source.
	constexpr int32 SourceIndex = 0;
	UInterchangeTranslatorBase* AsyncTranslator = nullptr;
	//Add all source data translator
	for (int32 SourceDataIndex = 0; SourceDataIndex < AsyncHelper->SourceDatas.Num(); ++SourceDataIndex)
	{
		UInterchangeTranslatorBase* SourceTranslator = GetTranslatorForSourceData(AsyncHelper->SourceDatas[SourceDataIndex]);
		if (!ensureMsgf(SourceTranslator, TEXT("Each interchange source data should have a valid translator")))
		{
			return EarlyExit();
		}

		SourceTranslator->AnalyticsHandler = AsyncHelper->AnalyticsHandler.Get();

		if (bIsReimport)
		{
			//Set translator settings if we are doing a re-import
			if (const UInterchangeTranslatorSettings* InterchangeTranslatorSettings = OriginalAssetImportData->GetTranslatorSettings())
			{
				SourceTranslator->SetSettings(InterchangeTranslatorSettings);
			}
		}
		//Get the expected source index translator
		if (SourceDataIndex == SourceIndex)
		{
			AsyncTranslator = SourceTranslator;
		}
		ensure(AsyncHelper->Translators.Add(SourceTranslator) == SourceDataIndex);
	}

	//Create the node graphs for each source data (StrongObjectPtr has to be created on the main thread)
	for (int32 SourceDataIndex = 0; SourceDataIndex < AsyncHelper->SourceDatas.Num(); ++SourceDataIndex)
	{
		AsyncHelper->BaseNodeContainers.Add(TStrongObjectPtr<UInterchangeBaseNodeContainer>(NewObject<UInterchangeBaseNodeContainer>(GetTransientPackage(), NAME_None)));
		if (!ensureMsgf(AsyncHelper->BaseNodeContainers[SourceDataIndex].IsValid(), TEXT("Interchange node conatainer is invalid.")))
		{
			return EarlyExit();
		}
	}

	if (!ensureMsgf(AsyncHelper->Translators.Num() == AsyncHelper->SourceDatas.Num(), TEXT("Each interchange source data should have a translator")))
	{
		return EarlyExit();
	}
	
	//We need at least one translator
	if(!ensure(AsyncHelper->Translators.IsValidIndex(SourceIndex)))
	{
		return EarlyExit();
	}
	
	//Make sure we point on the first translator
	AsyncTranslator = AsyncHelper->Translators[SourceIndex];

	UInterchangePipelineConfigurationBase* RegisteredPipelineConfiguration = nullptr;

	//In runtime we do not have any pipeline configurator
#if WITH_EDITORONLY_DATA
	TSoftClassPtr <UInterchangePipelineConfigurationBase> ImportDialogClass = InterchangeImportSettings.ImportDialogClass;

	if (ImportDialogClass.IsValid())
	{
		UClass* PipelineConfigurationClass = ImportDialogClass.LoadSynchronous();
		if (PipelineConfigurationClass)
		{
			RegisteredPipelineConfiguration = NewObject<UInterchangePipelineConfigurationBase>(GetTransientPackage(), PipelineConfigurationClass, NAME_None, RF_NoFlags);
		}
	}
#endif
	
	UInterchangeBaseNodeContainer* BaseNodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();

	auto AdjustPipelineSettingForContext = [bIsReimport, bImportScene, &TaskData, BaseNodeContainer, &ContentBasePath](UInterchangePipelineBase* Pipeline)
		{
			EInterchangePipelineContext Context;
			if (bIsReimport)
			{
				Context = bImportScene ? EInterchangePipelineContext::SceneReimport : EInterchangePipelineContext::AssetReimport;
			}
			else
			{
				Context = bImportScene ? EInterchangePipelineContext::SceneImport : EInterchangePipelineContext::AssetImport;
			}
			Pipeline->ContentImportPath = ContentBasePath;
			FInterchangePipelineContextParams ContextParams;
			ContextParams.ContextType = Context;
			ContextParams.ReimportAsset = TaskData.ReimportObject;
			ContextParams.BaseNodeContainer = BaseNodeContainer;
			Pipeline->AdjustSettingsForContext(ContextParams);
			Pipeline->DestinationName = TaskData.DestinationName;
		};

	const bool bSkipImportDialog = AsyncTranslator ? ImportAllWithSamePipelines.Contains(AsyncTranslator->GetClass()) : false;
	if (bSkipImportDialog)
	{
		TArray<UInterchangePipelineBase*> LastImportPipelines = ImportAllWithSamePipelines.FindChecked(AsyncTranslator->GetClass());
		for (const UInterchangePipelineBase* LastImportPipeline : LastImportPipelines)
		{
			if(UInterchangePipelineBase* Pipeline = DuplicateObject<UInterchangePipelineBase>(LastImportPipeline, GetTransientPackage()))
			{
				AsyncHelper->Pipelines.Add(Pipeline);
				AsyncHelper->OriginalPipelines.Add(Pipeline);
			}
		}

		AnalyticsHelper.Send(LastImportPipelines, UniqueId);
		
	}
	else
	{
#if WITH_EDITORONLY_DATA
		bool bShowPipelineStacksConfigurationDialog = !bIsUnattended
			&& (FInterchangeProjectSettingsUtils::ShouldShowPipelineStacksConfigurationDialog(bImportScene, bIsReimport, *SourceData) || ImportAssetParameters.bForceShowDialog)
			&& !bImportCanceled
			&& !IsRunningCommandlet();
#else
		bool bShowPipelineStacksConfigurationDialog = false;
#endif

		
		AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("ShowImportDialog"), bShowPipelineStacksConfigurationDialog));


		//If we need to display the dialog we want to translate the source file before showing it
		if (RegisteredPipelineConfiguration && bShowPipelineStacksConfigurationDialog)
		{
			bool bCanTranslate = true;
			//Make sure we dont translate with a non thread translator that is already lock
			bool* NonParallelTranslatorLock = nullptr;
			bool bTranslatorIsThreadSafe = AsyncTranslator->IsThreadSafe();
			if (!bTranslatorIsThreadSafe)
			{
				NonParallelTranslatorLock = NonParallelTranslatorLocks.Find(AsyncTranslator->GetClass());
				//Add a NonParallelTranslatorLocks for this translator class
				if (!NonParallelTranslatorLock)
				{
					//Create a boolean lock and initialize it to false
					NonParallelTranslatorLock = &NonParallelTranslatorLocks.FindOrAdd(AsyncTranslator->GetClass());
					*NonParallelTranslatorLock = false;
				}
				//We ensure if we cannot translate
				bCanTranslate = ensureMsgf(NonParallelTranslatorLock && !(*NonParallelTranslatorLock), TEXT("Interchange, non thread safe translator cannot have multiple instance at the same time."));
			}

			if (bCanTranslate)
			{
				FScopedSlowTask Progress(2.f, NSLOCTEXT("InterchangeManager", "TranslatingSourceFile...", "Translating source file..."));
				Progress.MakeDialog();
				Progress.EnterProgressFrame(1.f);
				//Translate the source
				for (int32 SourceDataIndex = 0; SourceDataIndex < AsyncHelper->SourceDatas.Num(); ++SourceDataIndex)
				{
					constexpr uint64 PipelinePrerequistes = INTERCHANGE_INVALID_TASK_ID;
					//Log the source we begin importing
					UE_LOGF(LogInterchangeEngine, Display, "Interchange start importing source [%ls]", *AsyncHelper->SourceDatas[SourceDataIndex]->GetFilename());
					UE::Interchange::FTaskTranslator TaskTranslator(SourceDataIndex, AsyncHelper);
					TaskTranslator.Execute();
					AsyncHelper->TranslatorTasks.Add(PipelinePrerequistes);
				}
				Progress.EnterProgressFrame(1.f);
			}
			else
			{
				//If we already have a running non thread safe translator, we cannot translate here
				UE_LOGF(LogInterchangeEngine, Error, "Interchange cannot translate simultanously multiple file with a non thread safe translator. Source file [%ls]", *AsyncHelper->SourceDatas[SourceIndex]->ToDisplayString());
				bShowPipelineStacksConfigurationDialog = false;
			}
		}

		const TMap<FName, FInterchangePipelineStack>& DefaultPipelineStacks = InterchangeImportSettings.PipelineStacks;

		const FName ReimportPipelineName = "ReimportPipeline";
		const FName OverridePipelineName = "OverridePipeline";
		TArray<FInterchangeStackInfo> PipelineStacks;
		TArray<UInterchangePipelineBase*> OutPipelines;

		if (ImportAssetParameters.OverridePipelines.IsEmpty())
		{
			//Fill the Stacks before showing the UI
			if (bIsReimport)
			{
				FInterchangeStackInfo& StackInfo = PipelineStacks.AddDefaulted_GetRef();
				StackInfo.StackName = ReimportPipelineName;

				TArray<UObject*> Pipelines = OriginalAssetImportData->GetPipelines();
				for (UObject* CurrentPipeline : Pipelines)
				{
					UInterchangePipelineBase* SourcePipeline = Cast<UInterchangePipelineBase>(CurrentPipeline);
					if (!SourcePipeline)
					{
						if (UInterchangePythonPipelineAsset* PythonPipelineAsset = Cast<UInterchangePythonPipelineAsset>(CurrentPipeline))
						{
							SourcePipeline = PythonPipelineAsset->GeneratedPipeline;
						}
					}
					if (SourcePipeline && SourcePipeline->SupportReimport()) //Its possible a pipeline doesn't exist anymore so it wont load into memory when we loading the outer asset
					{
						//Duplicate the pipeline saved in the asset import data
						UInterchangePipelineBase* GeneratedPipeline = Cast<UInterchangePipelineBase>(StaticDuplicateObject(SourcePipeline, GetTransientPackage()));
						// Make sure that the instance does not carry over standalone and public flags as they are not actual assets to be persisted
						GeneratedPipeline->ClearFlags(EObjectFlags::RF_Standalone | EObjectFlags::RF_Public);
						constexpr bool bFromReImportOrOverride = true;
						GeneratedPipeline->SetFromReimportOrOverride(bFromReImportOrOverride);
						AdjustPipelineSettingForContext(GeneratedPipeline);
						StackInfo.Pipelines.Add(GeneratedPipeline);
					}
					else if (!SourcePipeline)
					{
						//A pipeline was not loaded
						UE_LOGF(LogInterchangeEngine, Warning, "Interchange Reimport: Missing import pipeline from the reimporting asset. The reimport might fail.");
					}
				}
			}

			UE::Interchange::FScopedTranslator ScopedTranslator(SourceData);
			for (const TPair<FName, FInterchangePipelineStack>& PipelineStackInfo : DefaultPipelineStacks)
			{
				FName StackName = PipelineStackInfo.Key;
				FInterchangeStackInfo& StackInfo = PipelineStacks.AddDefaulted_GetRef();
				StackInfo.StackName = StackName;

				const FInterchangePipelineStack& PipelineStack = PipelineStackInfo.Value;
				const TArray<FSoftObjectPath>* Pipelines = &PipelineStack.Pipelines;

				// If applicable, check to see if a specific pipeline stack is associated with this translator
				for (const FInterchangeTranslatorPipelines& TranslatorPipelines : PipelineStack.PerTranslatorPipelines)
				{
					const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
					if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
					{
						Pipelines = &TranslatorPipelines.Pipelines;
						break;
					}
				}

				for (int32 PipelineIndex = 0; PipelineIndex < Pipelines->Num(); ++PipelineIndex)
				{
					if (UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance((*Pipelines)[PipelineIndex]))
					{
						AdjustPipelineSettingForContext(GeneratedPipeline);
						StackInfo.Pipelines.Add(GeneratedPipeline);
					}
				}
			}
		}
		else
		{
			//If we receive an override list of pipelines add them to the stack
			FInterchangeStackInfo& StackInfo = PipelineStacks.AddDefaulted_GetRef();
			StackInfo.StackName = OverridePipelineName;
			for (int32 GraphPipelineIndex = 0; GraphPipelineIndex < ImportAssetParameters.OverridePipelines.Num(); ++GraphPipelineIndex)
			{
				UInterchangePipelineBase* GeneratedPipeline = UE::Interchange::GeneratePipelineInstance(ImportAssetParameters.OverridePipelines[GraphPipelineIndex]);
				if (!GeneratedPipeline)
				{
					UE_LOGF(LogInterchangeEngine, Error, "Interchange Import: Overridden pipeline array contains a NULL pipeline. Fix your script or code to avoid this issue.");
					continue;
				}
				else
				{
					GeneratedPipeline->ClearFlags(EObjectFlags::RF_Standalone | EObjectFlags::RF_Public);
					constexpr bool bFromReImportOrOverride = true;
					GeneratedPipeline->SetFromReimportOrOverride(bFromReImportOrOverride);
					AdjustPipelineSettingForContext(GeneratedPipeline);
					StackInfo.Pipelines.Add(GeneratedPipeline);
				}
			}
		}

		auto SetImportAllWithSamePipelines = [this, &AsyncTranslator](TArray<UInterchangePipelineBase*>& ToDuplicatePipelines)
			{
				TArray<UInterchangePipelineBase*>& PipelineList = ImportAllWithSamePipelines.FindOrAdd(AsyncTranslator->GetClass());
				for (const UInterchangePipelineBase* Pipeline : ToDuplicatePipelines)
				{
					if (UInterchangePipelineBase* DupPipeline = DuplicateObject<UInterchangePipelineBase>(Pipeline, GetTransientPackage()))
					{
						DupPipeline->SetInternalFlags(EInternalObjectFlags::Async);
						PipelineList.Add(DupPipeline);
					}
				}
			};

		if (bIsReimport)
		{
			if (RegisteredPipelineConfiguration && bShowPipelineStacksConfigurationDialog && !bIsUnattended)
			{
				//Show the dialog, a plugin should have registered this dialog. We use a plugin to be able to use editor code when doing UI
				EInterchangePipelineConfigurationDialogResult DialogResult = RegisteredPipelineConfiguration->ScriptedShowReimportPipelineConfigurationDialog(PipelineStacks, OutPipelines, DuplicateSourceData, AsyncTranslator, BaseNodeContainer, ImportAssetParameters.ReimportAsset, bImportScene);
				if (DialogResult == EInterchangePipelineConfigurationDialogResult::Cancel)
				{
					bImportCanceled = true;
				}
				if (DialogResult == EInterchangePipelineConfigurationDialogResult::ImportAll)
				{
					SetImportAllWithSamePipelines(OutPipelines);
				}
			}
			else
			{
				FName ClosurePipelineStackName = ImportAssetParameters.OverridePipelines.IsEmpty() ? ReimportPipelineName : OverridePipelineName;
				//When we do not show the UI we use the original import stack or the provided override
				FInterchangeStackInfo* StackInfoPtr = PipelineStacks.FindByPredicate([ClosurePipelineStackName](const FInterchangeStackInfo& StackInfo)
					{
						return StackInfo.StackName == ClosurePipelineStackName;
					});

				check(StackInfoPtr);
				OutPipelines = StackInfoPtr->Pipelines;
			}
		}
		else
		{
			if (RegisteredPipelineConfiguration && bShowPipelineStacksConfigurationDialog)
			{
				//Show the dialog, a plugin should have register this dialog. We use a plugin to be able to use editor code when doing UI
				EInterchangePipelineConfigurationDialogResult DialogResult = bImportScene
					? RegisteredPipelineConfiguration->ScriptedShowScenePipelineConfigurationDialog(PipelineStacks, OutPipelines, DuplicateSourceData, AsyncTranslator, BaseNodeContainer)
					: RegisteredPipelineConfiguration->ScriptedShowPipelineConfigurationDialog(PipelineStacks, OutPipelines, DuplicateSourceData, AsyncTranslator, BaseNodeContainer);

				if (DialogResult == EInterchangePipelineConfigurationDialogResult::Cancel)
				{
					bImportCanceled = true;
				}
				if (DialogResult == EInterchangePipelineConfigurationDialogResult::ImportAll)
				{
					SetImportAllWithSamePipelines(OutPipelines);
				}
			}
			else
			{
				FName DefaultStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(bImportScene, *DuplicateSourceData);
				FInterchangeStackInfo* StackInfoPtr = PipelineStacks.FindByPredicate([DefaultStackName](const FInterchangeStackInfo& StackInfo)
					{
						return StackInfo.StackName == DefaultStackName;
					});
				if (StackInfoPtr)
				{
					//When we do not show the UI we use the original stack
					OutPipelines = StackInfoPtr->Pipelines;
				}
				else if (PipelineStacks.Num() > 0)
				{
					//Take the first valid stack
					for (FInterchangeStackInfo& StackInfo : PipelineStacks)
					{
						if (!StackInfo.Pipelines.IsEmpty())
						{
							OutPipelines = StackInfo.Pipelines;
							UE_LOGF(LogInterchangeEngine, Display, "Interchange import: Using stack [%ls] to import.", *StackInfo.StackName.ToString());
							break;
						}
					}
				}
				else
				{
					UE_LOGF(LogInterchangeEngine, Error, "Interchange Import: Cannot find any valid stack, canceling import.");
					bImportCanceled = true;
				}
			}
		}

		if (!bImportCanceled)
		{
			// Simply move the existing pipeline
			AsyncHelper->Pipelines = MoveTemp(OutPipelines);

			//Fill the original pipeline array that will be save in the asset import data
			for (UInterchangePipelineBase* Pipeline : AsyncHelper->Pipelines)
			{
				if (UInterchangePythonPipelineBase* PythonPipeline = Cast<UInterchangePythonPipelineBase>(Pipeline))
				{
					UInterchangePythonPipelineAsset* PythonPipelineAsset = NewObject<UInterchangePythonPipelineAsset>(GetTransientPackage());
					PythonPipelineAsset->PythonClass = PythonPipeline->GetClass();
					PythonPipelineAsset->SetupFromPipeline(PythonPipeline);
					AsyncHelper->OriginalPipelines.Add(PythonPipelineAsset);
				}
				else
				{
					AsyncHelper->OriginalPipelines.Add(Pipeline);
				}

			}

			AnalyticsHelper.Send(AsyncHelper->Pipelines, UniqueId);
		}
	}

	//Cancel the import do not queue task
	if (bImportCanceled || bImportAborted)
	{
		
		AnalyticsHelper.Add(UsageImportAnalayticsId, FAnalyticsEventAttribute(TEXT("Canceled"), bImportCanceled));
		AsyncHelper->InitCancel();

		//if it is a re-import, re-instate the backedup source data:
		if (UObject* ReimportCandidate = AsyncHelper->TaskData.ReimportObject)
		{
			if (const UClass* FactoryClass = GetRegisteredFactoryClass(ReimportCandidate->GetClass()))
			{
				UInterchangeFactoryBase* FactoryBase = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
				FactoryBase->ReinstateSourceData(ReimportCandidate);
			}
		}
	}
	else
	{
		//If it is a re-import, clear the backedup source data:
		if (UObject* ReimportCandidate = AsyncHelper->TaskData.ReimportObject)
		{
			if (const UClass* FactoryClass = GetRegisteredFactoryClass(ReimportCandidate->GetClass()))
			{
				UInterchangeFactoryBase* FactoryBase = FactoryClass->GetDefaultObject<UInterchangeFactoryBase>();
				FactoryBase->ClearBackupSourceData(ReimportCandidate);
			}
		}
	}

	AsyncHelper->ContentBasePath = ContentBasePath;
	//Queue the task cancel or not, we need to return a valid asset import result
	FQueuedTaskData QueuedTaskData;
	QueuedTaskData.AsyncHelper = AsyncHelper;
	QueuedTaskData.TranslatorClass = AsyncTranslator->GetClass();

	//If we cancel or abort the task we want to avoid putting it in the NonParallelTranslatorQueueTasks (the locks will not be release if the task doesn't start)
	bool bTranslatorIsThreadSafe = (bImportCanceled || bImportAborted) || AsyncTranslator->IsThreadSafe();
	if (bTranslatorIsThreadSafe)
	{
		QueuedTasks.Enqueue(QueuedTaskData);
	}
	else
	{
		//Add a NonParallelTranslatorLocks for this translator class
		if (!NonParallelTranslatorLocks.Contains(AsyncTranslator->GetClass()))
		{
			//Create a boolean lock and initialize it to false
			bool& TranslatorLock = NonParallelTranslatorLocks.FindOrAdd(AsyncTranslator->GetClass());
			TranslatorLock = false;
		}
		//Add an entry in NonParallelTranslatorQueueTasks
		TArray<FQueuedTaskData>& NonParallelQueuedTasks = NonParallelTranslatorQueueTasks.FindOrAdd(AsyncTranslator->GetClass());
		NonParallelQueuedTasks.Add(QueuedTaskData);
	}

	QueueTaskCount = FMath::Clamp(QueueTaskCount + 1, 0, MAX_int32);

	AsyncHelper->TaskExecuteStarted = FPlatformTime::Seconds();

	StartQueuedTasks();

	PreReturn();
	return TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>{ AsyncHelper->AssetImportResult, AsyncHelper->SceneImportResult };
}

void UInterchangeManager::SetAnalyticsHandlerClass(const TSoftClassPtr<UInterchangeAnalyticsHandlerBase>& InAnalyticsHandlerClass)
{
	AnalyticsHandlerClass = InAnalyticsHandlerClass;
}

TStrongObjectPtr<UInterchangeAnalyticsHandlerBase> UInterchangeManager::CreateAnalyticsHandler(const FString& ImportFilePath)
{
	TStrongObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler = nullptr;
	if (!AnalyticsHandlerClass.IsNull())
	{
		if (UClass* LoadedAnalyticsHandlerClass = AnalyticsHandlerClass.LoadSynchronous())
		{
			if (UInterchangeAnalyticsHandlerBase* AnalyticsHandlerBase = NewObject<UInterchangeAnalyticsHandlerBase>(GetTransientPackage(), LoadedAnalyticsHandlerClass, NAME_None, RF_NoFlags))
			{
				AnalyticsHandler = TStrongObjectPtr<UInterchangeAnalyticsHandlerBase>(AnalyticsHandlerBase);
				AnalyticsHandler->SetGUID(FGuid::NewDeterministicGuid(ImportFilePath, static_cast<uint64>(FDateTime::UtcNow().GetTicks())));
			}
		}
	}

	return AnalyticsHandler;
}

void UInterchangeManager::SetEditorUtilities(UClass* EditorUtilitiesClass)
{
	//Create the object
	if (UInterchangeEditorUtilitiesBase* NewEditorUtilities = NewObject<UInterchangeEditorUtilitiesBase>(GetTransientPackage(), EditorUtilitiesClass, NAME_None))
	{
		//If successful, assign the new editor utilities
		EditorUtilities.Reset();
		EditorUtilities = TStrongObjectPtr<UInterchangeEditorUtilitiesBase>(NewEditorUtilities);
	}
}

UInterchangeEditorUtilitiesBase* UInterchangeManager::GetEditorUtilities() const
{
	return EditorUtilities.Get();
}

bool UInterchangeManager::CanUseTranslator(UInterchangeTranslatorBase* Translator) const
{
	bool bCanTranslate = true;
	if (!Translator->IsThreadSafe())
	{
		const bool* NonParallelTranslatorLock = NonParallelTranslatorLocks.Find(Translator->GetClass());
		if (NonParallelTranslatorLock && *NonParallelTranslatorLock)
		{
			bCanTranslate = false;
		}
	}
	return bCanTranslate;
}

bool UInterchangeManager::IsObjectBeingImported(UObject* Object) const
{
	if (!ensure(IsInGameThread()))
	{
		return false;
	}
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (const TSharedPtr < UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe>& AsyncHelper : ImportTasks)
	{
		if (AsyncHelper->IsImportingObject(Object))
		{
			return true;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS

	return false;
}

bool UInterchangeManager::EnqueuePostImportTask(TSharedPtr<FInterchangePostImportTask> PostImportTask)
{
	//We can only enqueue on the game thread
	if (!ensure(IsInGameThread()))
	{
		return false;
	}

	QueuedPostImportTasks.Enqueue(PostImportTask);

	if (!QueuedPostImportTasksTickerHandle.IsValid())
	{
		QueuedPostImportTasksTickerHandle = FTSTicker::GetCoreTicker().AddTicker(FTickerDelegate::CreateLambda([this](float DeltaTime)
			{
				check(IsInGameThread());
				while (!QueuedPostImportTasks.IsEmpty())
				{
					//Wait next frame if we are importing assets or scenes
					if (!QueuedTasks.IsEmpty() || GetImportTaskCount() > 0)
					{
						break;
					}
					TSharedPtr<FInterchangePostImportTask> PostImportTask;
					if (QueuedPostImportTasks.Dequeue(PostImportTask))
					{
						if (PostImportTask)
						{
							PostImportTask->Execute();
						}
					}
				}

				if (QueuedPostImportTasks.IsEmpty())
				{
					QueuedPostImportTasksTickerHandle.Reset();
					return false;
				}
				return true;
			}));
	}
	return true;
}

bool UInterchangeManager::IsInterchangeImportEnabled()
{
	return CCvarInterchangeImportEnable->GetBool();
}

void UInterchangeManager::SetInterchangeImportEnabled(bool bEnabled)
{
	CCvarInterchangeImportEnable->Set(bEnabled);
}

bool UInterchangeManager::IsImporting()
{
	return UE::Interchange::Private::StaticHelpers::AsyncHelperCounter > 0;
}

void UInterchangeManager::SanitizeNameInline(FString& NameToSanitize, const ESanitizeNameTypeFlags NameType)
{
	//Default behavior don't use the delegates
	auto SanitizeCharacters = [&NameToSanitize](const TCHAR* InvalidCharacters)
		{
			while (*InvalidCharacters)
			{
				NameToSanitize.ReplaceCharInline(*InvalidCharacters, TCHAR('_'), ESearchCase::CaseSensitive);
				++InvalidCharacters;
			}
		};

	//Sanitize all Name type users ask for

	bool bBroadcastSanitizeName = false;
	if ((NameType & ESanitizeNameTypeFlags::Name) != ESanitizeNameTypeFlags::None)
	{
		SanitizeCharacters(INVALID_NAME_CHARACTERS);
		bBroadcastSanitizeName = true;
	}

	if ((NameType & ESanitizeNameTypeFlags::ObjectName) != ESanitizeNameTypeFlags::None)
	{
		SanitizeCharacters(INVALID_OBJECTNAME_CHARACTERS);
		bBroadcastSanitizeName = true;
	}

	if ((NameType & ESanitizeNameTypeFlags::ObjectPath) != ESanitizeNameTypeFlags::None)
	{
		SanitizeCharacters(INVALID_OBJECTPATH_CHARACTERS);
	}

	if ((NameType & ESanitizeNameTypeFlags::LongPackage) != ESanitizeNameTypeFlags::None)
	{
		SanitizeCharacters(INVALID_LONGPACKAGE_CHARACTERS);
	}

	if (bBroadcastSanitizeName)
	{
		//Call the delegates which will add extra sanitize rules, this is a thread safe delegate
		OnSanitizeName.Broadcast(NameToSanitize, NameType);
	}
}


bool UInterchangeManager::ExportAsset(const UObject* Asset, bool bIsAutomated)
{
	return false;
}

bool UInterchangeManager::ExportScene(const UObject* World, bool bIsAutomated)
{
	return false;
}

UInterchangeSourceData* UInterchangeManager::CreateSourceData(const FString& InFileName)
{
	UInterchangeSourceData* SourceDataAsset = NewObject<UInterchangeSourceData>(GetTransientPackage(), NAME_None);
	if(!InFileName.IsEmpty() && SourceDataAsset)
	{
		SourceDataAsset->SetFilename(InFileName);
	}
	return SourceDataAsset;
}

const UClass* UInterchangeManager::GetRegisteredFactoryClass(const UClass* ClassToMake) const
{
	const UClass* BestClassToMake = nullptr;
	const UClass* Result = nullptr;

	for (const auto& Kvp : RegisteredFactoryClasses)
	{
		if (ClassToMake->IsChildOf(Kvp.Key))
		{
			// Find the factory which handles the most derived registered type 
			if (BestClassToMake == nullptr || Kvp.Key->IsChildOf(BestClassToMake))
			{
				BestClassToMake = Kvp.Key.Get();
				Result = Kvp.Value.Get();
			}
		}
	}
	return Result;
}

void UInterchangeManager::InitAsyncHelper(TSharedRef<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe>& AsyncHelper, const UE::Interchange::FImportAsyncHelperData& Data, const FImportAssetParameters& ImportAssetParameters)
{
	//Copy the task data
	AsyncHelper->TaskData = Data;
	
	AsyncHelper->AssetImportResult->OnObjectDone = ImportAssetParameters.OnAssetDone;
	AsyncHelper->AssetImportResult->OnObjectDoneNative = ImportAssetParameters.OnAssetDoneNative;
	AsyncHelper->AssetImportResult->OnImportDone = ImportAssetParameters.OnAssetsImportDone;
	AsyncHelper->AssetImportResult->OnImportDoneNative = ImportAssetParameters.OnAssetsImportDoneNative;
	AsyncHelper->AssetImportResult->SetAsyncHelper(AsyncHelper);

	AsyncHelper->SceneImportResult->OnObjectDone = ImportAssetParameters.OnSceneObjectDone;
	AsyncHelper->SceneImportResult->OnObjectDoneNative = ImportAssetParameters.OnSceneObjectDoneNative;
	AsyncHelper->SceneImportResult->OnImportDone = ImportAssetParameters.OnSceneImportDone;
	AsyncHelper->SceneImportResult->OnImportDoneNative = ImportAssetParameters.OnSceneImportDoneNative;
	AsyncHelper->SceneImportResult->SetAsyncHelper(AsyncHelper);

	AsyncHelper->AssetImportResult->SetInProgress();

	AsyncHelper->bRuntimeOrPIE = GetEditorUtilities() ? GetEditorUtilities()->IsRuntimeOrPIE() : false;

	AsyncHelper->TaskExecuteStarted = FPlatformTime::Seconds();
}

void UInterchangeManager::ReleaseAsyncHelper(TWeakPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper)
{
	using namespace UE::Interchange;

	check(AsyncHelper.IsValid());
		
	// TODO: @vjavdekar : Find a way to run automated tests so that having this enabled
	// doesn't interrupt them and return early causing some to fail that expect the errors. 
	const bool bLogWarningsAndErrors = !GIsAutomationTesting;
	
	double ImportTimeDelta;
	int32 ResultCount = 0;
	bool bSucceeded = false;
	{
		TSharedPtr<FImportAsyncHelper> AsyncHelperPtr = AsyncHelper.Pin();

		ImportTimeDelta = FPlatformTime::Seconds() - AsyncHelperPtr->TaskExecuteStarted;

		//Free the lock to allow the next import to happen
		if (AsyncHelperPtr->Translators.IsValidIndex(0))
		{
			if (bool* bTranslatorLock = NonParallelTranslatorLocks.Find(AsyncHelperPtr->Translators[0]->GetClass()))
			{
				*bTranslatorLock = false;
			}
		}
		
		auto ForEachResult = [&bSucceeded, &ResultCount, bLogWarningsAndErrors](TArray<UInterchangeResult*>&& Results)
		{
			if (!bSucceeded || bLogWarningsAndErrors)
			{
				ResultCount += Results.Num();
				for (UInterchangeResult* Result : Results)
				{
					if (!Result)
					{
						continue;
					}

					if (Result->GetResultType() == EInterchangeResultType::Success)
					{
						bSucceeded = true;
					}
					else if (bLogWarningsAndErrors)
					{
						switch (Result->GetResultType())
						{
						case EInterchangeResultType::Warning:
							UE_LOGF(LogInterchangeEngine, Warning, "%ls", *Result->GetMessageLogText().ToString());
							break;
						case EInterchangeResultType::Error:
							UE_LOGF(LogInterchangeEngine, Error, "%ls", *Result->GetMessageLogText().ToString());
							break;
						}
					}
				}
			}
		};

		ForEachResult(AsyncHelperPtr->AssetImportResult->GetResults()->GetResults());
		ForEachResult(AsyncHelperPtr->SceneImportResult->GetResults()->GetResults());
	}

	RemoveImportTask(AsyncHelper.Pin());
	//Make sure the async helper is destroy, if not destroy its because we are canceling the import and we still have a shared ptr on it
	{
		TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelperSharedPtr = AsyncHelper.Pin();
		if (AsyncHelperSharedPtr.IsValid() && !AsyncHelperSharedPtr->bCancel)
		{
			UE_LOGF(LogInterchangeEngine, Warning, "Interchange AsyncHelper resource not released properly.");
		}
	}

	const int32 ImportTaskNumber = GetImportTaskCount() + QueueTaskCount;

	if (ImportTaskNumber == 0)
	{
		SetActiveMode(false);

		if (Notification.IsValid())
		{
			FText TitleText;
			FText ProgressText = FText::GetEmpty();
			if (bImportCanceled)
			{
				TitleText = NSLOCTEXT("Interchange", "Asynchronous_import_canceled", "Import Canceled");
				bSucceeded = true; // Mark the "cancelation" as a success so that the notification goes away
				Notification->SetProgressText(NSLOCTEXT("Interchange", "Asynchronous_operation_cancelled", "Operation was cancelled by user."));
			}
			else
			{
				if (ResultCount == 0)
				{
					UE_LOGF(LogInterchangeEngine, Warning, "Import Done but no data imported");
					ProgressText = NSLOCTEXT("Interchange", "Asynchronous_import_progress", "No Data Imported");
					TitleText = NSLOCTEXT("Interchange", "Asynchronous_import_empty_results", "Import Done");
					bSucceeded = true; // Mark the fact as nothing was imported as a success
				}
				else if (bSucceeded)
				{
					FString ImportDoneText = TEXT("Import Done in ") + FString::SanitizeFloat(ImportTimeDelta) + TEXT(" seconds.");
					UE_LOGF(LogInterchangeEngine, Display, "%ls", *ImportDoneText);
					
					TitleText = NSLOCTEXT("Interchange", "Asynchronous_import_end", "Import Done");
				}
				else
				{
					TitleText = NSLOCTEXT("Interchange", "Asynchronous_import_failed", "Import Failed");
				}

				Notification->SetProgressText(NSLOCTEXT("Interchange", "Asynchronous_operation_completed", "Operation Completed."));
			}
									
			Notification->SetComplete(TitleText, ProgressText, bSucceeded);
			Notification = nullptr; //This should delete the notification
		}

		//Release import all pipelines so they can be garbage collect
		for (TPair<UClass*, TArray<UInterchangePipelineBase*>>& ImportAllWithSamePipelinesPair : ImportAllWithSamePipelines)
		{
			for (UInterchangePipelineBase* Pipeline : ImportAllWithSamePipelinesPair.Value)
			{
				Pipeline->ClearInternalFlags(EInternalObjectFlags::Async);
			}
		}
		ImportAllWithSamePipelines.Empty();

		bImportCanceled = false;
	}
	else if(Notification.IsValid())
	{
		Notification->SetProgressText(FText::Format(NSLOCTEXT("Interchange", "Asynchronous_pending_import_tasks", "Importing ({0})"), ImportTaskNumber));
	}

	//Start some task if there is some waiting
	StartQueuedTasks();
}

TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> UInterchangeManager::GetAsyncHelper(int32 UniqueId)
{
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for (TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper : ImportTasks)
	{
		if (AsyncHelper.IsValid() && AsyncHelper->UniqueId == UniqueId)
		{
			return AsyncHelper;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return nullptr;
}

UInterchangeTranslatorBase* UInterchangeManager::GetTranslatorForSourceData(const UInterchangeSourceData* SourceData) const
{
	// Find the translator
	for (const UClass* TranslatorClass : RegisteredTranslatorsClass)
	{
		if (TranslatorClass->GetDefaultObject<UInterchangeTranslatorBase>()->CanImportSourceData(SourceData))
		{
			UInterchangeTranslatorBase* SourceDataTranslator = NewObject<UInterchangeTranslatorBase>(GetTransientPackage(), TranslatorClass, NAME_None);
			SourceDataTranslator->SourceData = SourceData;
			return SourceDataTranslator;
		}
	}
	return nullptr;
}

bool UInterchangeManager::IsInterchangeActive()
{
	return bIsActive;
}

bool UInterchangeManager::WarnIfInterchangeIsActive()
{
	if (!bIsActive)
	{
		return false;
	}
	//Tell the user they have to cancel the import before closing the editor
	FNotificationInfo Info(NSLOCTEXT("InterchangeManager", "WarnCannotProceed", "An import process is currently underway. Please cancel it to proceed."));
	Info.ExpireDuration = 5.0f;
	TSharedPtr<SNotificationItem> WarnNotification = FSlateNotificationManager::Get().AddNotification(Info);
	if (WarnNotification.IsValid())
	{
		WarnNotification->SetCompletionState(SNotificationItem::CS_Fail);
	}
	return true;
}

UInterchangeAssetImportData* UInterchangeManager::GetAssetImportData(UObject* Asset) const
{
	return UInterchangeAssetImportData::GetFromObject(Asset);
}

bool UInterchangeManager::CanTranslateSourceDataWithPayloadInterface(const UInterchangeSourceData* SourceData, const UClass* PayloadInterfaceClass) const
{
	if (GetTranslatorSupportingPayloadInterfaceForSourceData(SourceData, PayloadInterfaceClass))
	{
		return true;
	}
	return false;
}

UInterchangeTranslatorBase* UInterchangeManager::GetTranslatorSupportingPayloadInterfaceForSourceData(const UInterchangeSourceData* SourceData, const UClass* PayloadInterfaceClass) const
{
	// Find the translator
	for (const UClass* TranslatorClass : RegisteredTranslatorsClass)
	{
		if (TranslatorClass->ImplementsInterface(PayloadInterfaceClass) &&
			TranslatorClass->GetDefaultObject<UInterchangeTranslatorBase>()->CanImportSourceData(SourceData))
		{
			UInterchangeTranslatorBase* SourceDataTranslator = NewObject<UInterchangeTranslatorBase>(GetTransientPackage(), TranslatorClass, NAME_None);
			SourceDataTranslator->SourceData = SourceData;
			return SourceDataTranslator;
		}
	}
	return nullptr;
}

bool UInterchangeManager::IsAttended()
{
	if (FApp::IsGame())
	{
		return false;
	}
	if (FApp::IsUnattended())
	{
		return false;
	}
	return true;
}

// #todo_interchange: TODO: Can be costly on large project. Using a cache and using the AssetRegistry to monitor addition will be more efficient.
// This code could also be moved to a utility class.
void UInterchangeManager::FindPipelineCandidate(TArray<UClass*>& PipelineCandidates)
{
	//Find in memory pipeline class
	for (TObjectIterator< UClass > ClassIt; ClassIt; ++ClassIt)
	{
		UClass* Class = *ClassIt;
		// Ignore deprecated
		if (Class->HasAnyClassFlags(CLASS_Deprecated | CLASS_NewerVersionExists))
		{
			continue;
		}

		// Check this class is a subclass of Base and not the base itself
		if (Class == UInterchangePipelineBase::StaticClass() || !Class->IsChildOf(UInterchangePipelineBase::StaticClass()))
		{
			continue;
		}

		//We found a candidate
		PipelineCandidates.AddUnique(Class);
	}

	//Blueprint and python script discoverability is available only if we compile with the engine
	// Load the asset registry module
	FAssetRegistryModule& AssetRegistryModule = FModuleManager::LoadModuleChecked< FAssetRegistryModule >(FName("AssetRegistry"));
	IAssetRegistry& AssetRegistry = AssetRegistryModule.Get();

	TArray< FString > ContentPaths;
	ContentPaths.Add(TEXT("/Game"));
	//#todo_interchange: TODO do we have an other alternative, this call is synchronous and will wait unitl the registry database have finish the initial scan. If there is a lot of asset it can take multiple second the first time we call it.
	AssetRegistry.ScanPathsSynchronous(ContentPaths);


	// Use the asset registry to get the set of all class names deriving from Base
	TSet< FTopLevelAssetPath > DerivedNames;
	{
		FTopLevelAssetPath BaseClassName = UInterchangePipelineBase::StaticClass()->GetClassPathName();
		TSet< FTopLevelAssetPath > Excluded;

		AssetRegistry.GetDerivedClassNames({ BaseClassName }, Excluded, DerivedNames);
	}

	// Iterate over retrieved blueprint and python pipeline assets
	FARFilter Filter;
	Filter.ClassPaths.Add(UBlueprint::StaticClass()->GetClassPathName());
	Filter.ClassPaths.Add(UInterchangePythonPipelineAsset::StaticClass()->GetClassPathName());
	Filter.bRecursiveClasses = true;
	Filter.bRecursivePaths = true;

	TArray< FAssetData > AssetList;
	AssetRegistry.GetAssets(Filter, AssetList);

	// Iterate over retrieved blueprint assets
	for (const FAssetData& Asset : AssetList)
	{
		if (UAssetRegistryHelpers::IsAssetDataBlueprintOfClassSet(Asset, DerivedNames))
		{
			UBlueprint* Blueprint = Cast<UBlueprint>(Asset.GetAsset());

			if (ensure(Blueprint) && ensure(Blueprint->ParentClass->IsChildOf<UInterchangePipelineBase>()))
			{
				PipelineCandidates.AddUnique(Blueprint->GeneratedClass);
			}
		}
		else if (UClass* AssetClass = Asset.GetClass())
		{
			if (AssetClass->IsChildOf<UInterchangePythonPipelineAsset>())
			{
				UInterchangePythonPipelineAsset* PythonPipeline = Cast<UInterchangePythonPipelineAsset>(Asset.GetAsset());
				if (PythonPipeline && PythonPipeline->PythonClass.Get()->IsChildOf<UInterchangePipelineBase>())
				{
					PipelineCandidates.AddUnique(PythonPipeline->GetClass());
				}
			}
		}
	}
}

void UInterchangeManager::CancelAllTasks()
{
	check(IsInGameThread());

	//Cancel the queued tasks, we cannot simply not do them since, there is some promise objects
	//to setup in the completion task
	constexpr bool bCancelAllTasks = true;
	StartQueuedTasks(bCancelAllTasks);

	//Set the cancel state on all running tasks
	{
		FScopeLock Lock(&ImportTasksLock);
		int32 ImportTaskCount = GetImportTaskCount();
		for (int32 TaskIndex = 0; TaskIndex < ImportTaskCount; ++TaskIndex)
		{
			TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = GetImportTaskForIndex(TaskIndex);
			if (AsyncHelper.IsValid())
			{
				AsyncHelper->InitCancel();
			}
		}
	}
	for (TPair<UClass*, TArray<FQueuedTaskData>>& ClassAndTasks : NonParallelTranslatorQueueTasks)
	{
		//After calling StartQueuedTasks with bCancelAllTasks at true, we should not have any waiting task here
		if (ClassAndTasks.Value.IsEmpty())
		{
			continue;
		}
		//if we still have some task we need to Cancel them asap
		FQueuedTaskData QueuedTaskData = ClassAndTasks.Value[0];
		if (QueuedTaskData.AsyncHelper.IsValid())
		{
			QueuedTaskData.AsyncHelper->InitCancel();
		}
	}
	bImportCanceled = true;
	//Tasks should all finish quite fast now
};

void UInterchangeManager::WaitUntilAllTasksDone(bool bCancel)
{
	check(IsInGameThread());
	if (bCancel)
	{
		//Start the cancel process by cancelling all current task
		CancelAllTasks();
	}

	while (GetImportTaskCount())
	{
		TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = GetImportTaskForIndex(0);
		if (AsyncHelper.IsValid())
		{
			TWeakPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper = AsyncHelper;
			TArray<uint64> TasksToComplete = AsyncHelper->GetCompletionTaskGraphEvent();
			//Release the shared pointer before waiting to be sure the async helper can be destroy in the completion task
			AsyncHelper = nullptr;
			UE::Interchange::FInterchangeTaskSystem::Get().WaitUntilTasksComplete(TasksToComplete);
			//We verify that the weak pointer is invalid after the task completed
			ensure(!WeakAsyncHelper.IsValid());
		}
	}
}

void UInterchangeManager::SetActiveMode(bool IsActive)
{
	if (bIsActive == IsActive)
	{
		return;
	}

	bIsActive = IsActive;
	if (bIsActive)
	{
		ensure(!NotificationTickHandle.IsValid());
		NotificationTickHandle = FTSTicker::GetCoreTicker().AddTicker(TEXT("InterchangeManagerTickHandle"), 0.1f, [this](float)
		{
			if (Notification.IsValid() && Notification->GetPromptAction() == EAsyncTaskNotificationPromptAction::Cancel)
			{
				CancelAllTasks();
			}
			return true;
		});
	}
	else
	{
		FTSTicker::RemoveTicker(NotificationTickHandle);
		NotificationTickHandle.Reset();
	}
}

void UInterchangeManager::SetReplaceExistingAlldialogAnswer(bool bReplaceExistingAllDialogAnswer)
{
	UE::Interchange::Private::StaticHelpers::bReplaceExistingAllDialogAnswer = bReplaceExistingAllDialogAnswer ? 1 : 0;
}

void UInterchangeManager::ResetReplaceExistingAlldialogAnswerSet()
{
	UE::Interchange::Private::StaticHelpers::bReplaceExistingAllDialogAnswer = -1;
}

TOptional<bool> UInterchangeManager::GetReplaceExistingAlldialogAnswer()
{
	TOptional<bool> Result;
	if (UE::Interchange::Private::StaticHelpers::bReplaceExistingAllDialogAnswer != -1)
	{
		Result = (UE::Interchange::Private::StaticHelpers::bReplaceExistingAllDialogAnswer == 1);
	}
	return Result;
}

int32 UInterchangeManager::GetImportTaskCount() const
{
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ImportTasks.Num();
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> UInterchangeManager::GetImportTaskForIndex(int32 Index) const
{
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ImportTasks.IsValidIndex(Index))
	{
		return ImportTasks[Index];
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return nullptr;
}

TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> UInterchangeManager::GetImportTaskForAsyncHelperUniqueId(int32 AsyncHelperUniqueId) const
{
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper : ImportTasks)
	{
		if (AsyncHelper.IsValid() && AsyncHelper->UniqueId == AsyncHelperUniqueId)
		{
			return AsyncHelper;
		}
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
	return nullptr;
}

int32 UInterchangeManager::AddImportTask(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> ImportTask)
{
	if (!ImportTask.IsValid())
	{
		return INDEX_NONE;
	}
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	return ImportTasks.Add(ImportTask);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UInterchangeManager::RemoveImportTask(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> ImportTask)
{
	FScopeLock Lock(&ImportTasksLock);
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	if (ImportTask.IsValid())
	{
		ImportTasks.RemoveSingle(ImportTask);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}

void UInterchangeManager::RemoveImportTask(int32 AsyncHelperUniqueId)
{
	FScopeLock Lock(&ImportTasksLock);
	TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> ToRemoveAsyncHelper;
	PRAGMA_DISABLE_DEPRECATION_WARNINGS
	for(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper : ImportTasks)
	{
		if (AsyncHelper.IsValid() && AsyncHelper->UniqueId == AsyncHelperUniqueId)
		{
			ToRemoveAsyncHelper = AsyncHelper;
			break;
		}
	}
	if (ToRemoveAsyncHelper.IsValid())
	{
		ImportTasks.Remove(ToRemoveAsyncHelper);
	}
	PRAGMA_ENABLE_DEPRECATION_WARNINGS
}