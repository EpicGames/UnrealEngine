// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeProjectSettings.h"

#include "InterchangeManager.h"
#include "InterchangeSourceData.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeProjectSettings)

namespace UE::Interchange::Private
{
	TArray<UInterchangePipelineBase*> GetPipelineArrayFromSoftObjectArray(const TArray<FSoftObjectPath>& PipelineSoftObjectPaths)
	{
		TArray<UInterchangePipelineBase*> Pipelines;
		Pipelines.Reserve(PipelineSoftObjectPaths.Num());
		for (const FSoftObjectPath& PipelinePath : PipelineSoftObjectPaths)
		{
			if (UInterchangePipelineBase* PipelineBase = GeneratePipelineInstance(PipelinePath))
			{
				Pipelines.Add(PipelineBase);
			}
		}
		return Pipelines;
	}
}

#if WITH_EDITOR
void UInterchangeProjectSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FString PropertyName = PropertyThatChanged ? PropertyThatChanged->GetName() : TEXT("");

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UInterchangeProjectSettings, InterchangeGroups))
	{
		if (UInterchangeEditorSettings* InterchangeEditorSettings = GetMutableDefault<UInterchangeEditorSettings>())
		{
			InterchangeEditorSettings->UpdateUsedGroupName();
		}
	}
}
#endif

const FInterchangeImportSettings& FInterchangeProjectSettingsUtils::GetImportSettings(const UInterchangeProjectSettings& InterchangeProjectSettings, const bool bIsSceneImport)
{
	if (bIsSceneImport)
	{
		return InterchangeProjectSettings.SceneImportSettings;
	}
	else
	{
		return InterchangeProjectSettings.ContentImportSettings;
	}
}

FInterchangeImportSettings& FInterchangeProjectSettingsUtils::GetMutableImportSettings(UInterchangeProjectSettings& InterchangeProjectSettings, const bool bIsSceneImport)
{
	if (bIsSceneImport)
	{
		return InterchangeProjectSettings.SceneImportSettings;
	}
	else
	{
		return InterchangeProjectSettings.ContentImportSettings;
	}
}

const FInterchangeImportSettings& FInterchangeProjectSettingsUtils::GetDefaultImportSettings(const bool bIsSceneImport)
{
	return GetImportSettings(*GetDefault<UInterchangeProjectSettings>(), bIsSceneImport);
}

FInterchangeImportSettings& FInterchangeProjectSettingsUtils::GetMutableDefaultImportSettings(const bool bIsSceneImport)
{
	return GetMutableImportSettings(*GetMutableDefault<UInterchangeProjectSettings>(), bIsSceneImport);
}

FName FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(const bool bIsSceneImport, const UInterchangeSourceData& SourceData)
{
	const FInterchangeImportSettings& ImportSettings = GetDefaultImportSettings(bIsSceneImport);

	FInterchangeGroup::EUsedGroupStatus UsedGroupStatus;
	const FInterchangeGroup& UsedInterchangeGroup = FInterchangeProjectSettingsUtils::GetUsedGroup(UsedGroupStatus);
	bool bInterchangeGroupUsed = (UsedGroupStatus == FInterchangeGroup::EUsedGroupStatus::SetAndValid);

	FName DefaultPipelineStack = bInterchangeGroupUsed ? UsedInterchangeGroup.DefaultPipelineStack : ImportSettings.DefaultPipelineStack;

	if (!bIsSceneImport)
	{
		UE::Interchange::FScopedTranslator ScopedTranslator(&SourceData);

		if (UInterchangeTranslatorBase* Translator = ScopedTranslator.GetTranslator())
		{
			EInterchangeTranslatorAssetType SupportedAssetTypes = Translator->GetSupportedAssetTypes();

			const TMap<EInterchangeTranslatorAssetType, FName>& DefaultPipelineStackOverride = bInterchangeGroupUsed ? UsedInterchangeGroup.DefaultPipelineStackOverride : GetDefault<UInterchangeProjectSettings>()->ContentImportSettings.DefaultPipelineStackOverride;

			for (TMap<EInterchangeTranslatorAssetType, FName>::TConstIterator StackOverridesIt = DefaultPipelineStackOverride.CreateConstIterator(); StackOverridesIt; ++StackOverridesIt)
			{
				if ((SupportedAssetTypes ^ StackOverridesIt->Key) < StackOverridesIt->Key)
				{
					DefaultPipelineStack = StackOverridesIt->Value;
					break;
				}
			}
		}
	}

	return DefaultPipelineStack;
}

void FInterchangeProjectSettingsUtils::SetDefaultPipelineStackName(const bool bIsSceneImport, const UInterchangeSourceData& SourceData, const FName StackName)
{
	FInterchangeImportSettings& ImportSettings = GetMutableDefaultImportSettings(bIsSceneImport);

	if (!ImportSettings.PipelineStacks.Contains(StackName))
	{
		//The new stack name must be valid
		return;
	}

	FName DefaultPipelineStack = ImportSettings.DefaultPipelineStack;

	if (!bIsSceneImport)
	{
		UE::Interchange::FScopedTranslator ScopedTranslator(&SourceData);

		if (UInterchangeTranslatorBase* Translator = ScopedTranslator.GetTranslator())
		{
			EInterchangeTranslatorAssetType SupportedAssetTypes = Translator->GetSupportedAssetTypes();

			FInterchangeContentImportSettings& ContentImportSettings = GetMutableDefault<UInterchangeProjectSettings>()->ContentImportSettings;
			for (TMap<EInterchangeTranslatorAssetType, FName>::TIterator StackOverridesIt = ContentImportSettings.DefaultPipelineStackOverride.CreateIterator(); StackOverridesIt; ++StackOverridesIt)
			{
				if ((SupportedAssetTypes ^ StackOverridesIt->Key) < StackOverridesIt->Key)
				{
					//Update the override stack name and save the config
					StackOverridesIt->Value = StackName;
					GetMutableDefault<UInterchangeProjectSettings>()->SaveConfig();
					return;
				}
			}
		}
	}

	//We do not have any override default stack, simply change the DefaultPipelineStack and save the config
	ImportSettings.DefaultPipelineStack = StackName;
	GetMutableDefault<UInterchangeProjectSettings>()->SaveConfig();
}

bool FInterchangeProjectSettingsUtils::ShouldShowPipelineStacksConfigurationDialog(const bool bIsSceneImport, const bool bReImport, const UInterchangeSourceData& SourceData)
{
	FInterchangeGroup::EUsedGroupStatus UsedGroupStatus;
	const FInterchangeGroup& UsedInterchangeGroup = FInterchangeProjectSettingsUtils::GetUsedGroup(UsedGroupStatus);
	bool bInterchangeGroupUsed = (UsedGroupStatus == FInterchangeGroup::EUsedGroupStatus::SetAndValid);

	bool bShowDialog = false;
	if (bInterchangeGroupUsed)
	{
		bShowDialog = bReImport ? UsedInterchangeGroup.bShowReimportDialog
								: UsedInterchangeGroup.bShowImportDialog;
	}
	else
	{
		bShowDialog = bReImport ? GetDefaultImportSettings(bIsSceneImport).bShowReimportDialog
								: GetDefaultImportSettings(bIsSceneImport).bShowImportDialog;
	}

	if (bIsSceneImport)
	{
		// Find the per translator overrides to use
		const TArray<FInterchangePerTranslatorDialogOverride>* PerTranslatorOverrides = nullptr;
		if (bInterchangeGroupUsed)
		{
			// Use "None" to mean "scene" when authored on the groups, so we don't have to modify the group signature and
			// lose users' saved groups
			if (const FInterchangeDialogOverride* FoundNoneOverrides = UsedInterchangeGroup.ShowImportDialogOverride.Find(
					EInterchangeTranslatorAssetType::None
				))
			{
				PerTranslatorOverrides = &FoundNoneOverrides->PerTranslatorImportDialogOverride;
			}
		}
		else
		{
			PerTranslatorOverrides = &GetDefault<UInterchangeProjectSettings>()->SceneImportSettings.PerTranslatorDialogOverride;
		}

		if (PerTranslatorOverrides)
		{
			// Check if the translator for this source data has an override
			UE::Interchange::FScopedTranslator ScopedTranslator{&SourceData};
			if (UInterchangeTranslatorBase* Translator = ScopedTranslator.GetTranslator())
			{
				EInterchangeTranslatorAssetType SupportedAssetTypes = Translator->GetSupportedAssetTypes();
				const UClass* TranslatorClass = Translator->GetClass();

				const FInterchangePerTranslatorDialogOverride* PerTranslatorOverride = PerTranslatorOverrides->FindByPredicate(
					[&TranslatorClass](const FInterchangePerTranslatorDialogOverride& InterchangePerTranslatorDialogOverride)
					{
						return (InterchangePerTranslatorDialogOverride.Translator.Get() == TranslatorClass);
					}
				);

				if (PerTranslatorOverride)
				{
					bShowDialog = (bReImport ? PerTranslatorOverride->bShowReimportDialog : PerTranslatorOverride->bShowImportDialog);
				}
			}
		}
	}
	else
	{
		UE::Interchange::FScopedTranslator ScopedTranslator(&SourceData);

		if (UInterchangeTranslatorBase* Translator = ScopedTranslator.GetTranslator())
		{
			EInterchangeTranslatorAssetType SupportedAssetTypes = Translator->GetSupportedAssetTypes();
			const UClass* TranslatorClass = Translator->GetClass();

			//Iterate all override, if there is at least one override that show the imnport dialog we will show it.
			bool bFoundOverride = false;
			bool bShowFromOverrideStack = false;
			const TMap<EInterchangeTranslatorAssetType, FInterchangeDialogOverride>& ShowImportDialogOverride = bInterchangeGroupUsed
				? UsedInterchangeGroup.ShowImportDialogOverride
				: GetDefault<UInterchangeProjectSettings>()->ContentImportSettings.ShowImportDialogOverride;

			for (const TPair<EInterchangeTranslatorAssetType, FInterchangeDialogOverride>& ShowImportDialog : ShowImportDialogOverride)
			{
				if ((ShowImportDialog.Key == EInterchangeTranslatorAssetType::None && SupportedAssetTypes == EInterchangeTranslatorAssetType::None)
					|| (static_cast<uint8>(ShowImportDialog.Key & SupportedAssetTypes) > 0))
				{
					//Look if there is a per translator override
					const FInterchangeDialogOverride& InterchangeDialogOverride = ShowImportDialog.Value;
					const FInterchangePerTranslatorDialogOverride* FindPerTranslatorOverride = InterchangeDialogOverride.PerTranslatorImportDialogOverride.FindByPredicate([&TranslatorClass](const FInterchangePerTranslatorDialogOverride& InterchangePerTranslatorDialogOverride)
						{
							return (InterchangePerTranslatorDialogOverride.Translator.Get() == TranslatorClass);
						});
					if (FindPerTranslatorOverride)
					{
						bShowFromOverrideStack |= (bReImport ? FindPerTranslatorOverride->bShowReimportDialog : FindPerTranslatorOverride->bShowImportDialog);
					}
					else
					{
						//Simple override
						bShowFromOverrideStack |= (bReImport ? InterchangeDialogOverride.bShowReimportDialog : InterchangeDialogOverride.bShowImportDialog);
					}
					bFoundOverride = true;
				}
			}
			if (bFoundOverride)
			{
				bShowDialog = bShowFromOverrideStack;
			}
		}
	}

	return bShowDialog;
}

const FInterchangeGroup& FInterchangeProjectSettingsUtils::GetUsedGroup(FInterchangeGroup::EUsedGroupStatus& UsedGroupStatus)
{
	static const FInterchangeGroup InterchangeGroupNone;

	UsedGroupStatus = FInterchangeGroup::EUsedGroupStatus::NotSet;

	if (const UInterchangeEditorSettings* InterchangeEditorSettings = GetDefault<UInterchangeEditorSettings>())
	{
		const FGuid& InterchangeUsedGroupUID = InterchangeEditorSettings->GetUsedGroupUID();
		if (InterchangeUsedGroupUID.IsValid())
		{
			const UInterchangeProjectSettings* ImportSettings = GetDefault<UInterchangeProjectSettings>();
			for (const FInterchangeGroup& Group : ImportSettings->InterchangeGroups)
			{
				if (Group.UniqueID == InterchangeUsedGroupUID)
				{
					UsedGroupStatus = FInterchangeGroup::EUsedGroupStatus::SetAndValid;
					return Group;
				}
			}

			UsedGroupStatus = FInterchangeGroup::EUsedGroupStatus::SetAndInvalid;
		}
	}

	return InterchangeGroupNone;
}

TArray<FName> FInterchangeProjectSettingsUtils::GetGroupNames()
{
	TArray<FName> GroupNames;
	GroupNames.Add(FName());

	const TArray<FInterchangeGroup>& Groups = GetDefault<UInterchangeProjectSettings>()->InterchangeGroups;
	for (const FInterchangeGroup& Group : Groups)
	{
		GroupNames.Add(Group.DisplayName);
	}

	return GroupNames;
}

TArray<UInterchangePipelineBase*> UInterchangeProjectSettingsScript::GetPipelineArrayFromPipelineStack(const FInterchangePipelineStack& InterchangePipelineStack)
{
	return UE::Interchange::Private::GetPipelineArrayFromSoftObjectArray(InterchangePipelineStack.Pipelines);
}

TArray<UInterchangePipelineBase*> UInterchangeProjectSettingsScript::GetPipelineArrayFromTranslatorPipelines(const FInterchangeTranslatorPipelines& InterchangeTranslatorPipeline)
{
	return UE::Interchange::Private::GetPipelineArrayFromSoftObjectArray(InterchangeTranslatorPipeline.Pipelines);
}

TArray<UInterchangePipelineBase*> UInterchangeProjectSettingsScript::GetPipelineStackFromSourceData(const bool bIsSceneImport, const UInterchangeSourceData* SourceData)
{
	TArray<UInterchangePipelineBase*> ResultPipelines;
	FName DefaultStackName = FInterchangeProjectSettingsUtils::GetDefaultPipelineStackName(bIsSceneImport, *SourceData);
	const FInterchangeImportSettings& ImportSettings = FInterchangeProjectSettingsUtils::GetDefaultImportSettings(bIsSceneImport);
	if (!ImportSettings.PipelineStacks.Contains(DefaultStackName))
	{
		return ResultPipelines;
	}

	const FInterchangePipelineStack* InterchangePipelineStack = ImportSettings.PipelineStacks.Find(DefaultStackName);
	if (!InterchangePipelineStack)
	{
		return ResultPipelines;
	}

	UE::Interchange::FScopedTranslator ScopedTranslator(SourceData);
	const TArray<FSoftObjectPath>* Pipelines = &InterchangePipelineStack->Pipelines;
	// If applicable, check to see if a specific pipeline stack is associated with this translator
	for (const FInterchangeTranslatorPipelines& TranslatorPipelines : InterchangePipelineStack->PerTranslatorPipelines)
	{
		const UClass* TranslatorClass = TranslatorPipelines.Translator.LoadSynchronous();
		if (ScopedTranslator.GetTranslator() && ScopedTranslator.GetTranslator()->IsA(TranslatorClass))
		{
			Pipelines = &TranslatorPipelines.Pipelines;
			break;
		}
	}

	if(Pipelines)
	{
		//Copy the pipelines
		ResultPipelines = UE::Interchange::Private::GetPipelineArrayFromSoftObjectArray(*Pipelines);
	}
	return ResultPipelines;
}

void UInterchangeEditorSettings::SetUsedGroupName(const FName& InUsedGroupName)
{
	UsedGroupName = InUsedGroupName;

	UpdateUsedGroupUIDFromGroupName();
}

TArray<FName> UInterchangeEditorSettings::GetSelectableItems() const
{
	return FInterchangeProjectSettingsUtils::GetGroupNames();
}

void UInterchangeEditorSettings::UpdateUsedGroupName()
{
	FInterchangeGroup::EUsedGroupStatus UsedGroupStatus;
	const FInterchangeGroup& UsedInterchangeGroup = FInterchangeProjectSettingsUtils::GetUsedGroup(UsedGroupStatus);

	switch (UsedGroupStatus)
	{
		case FInterchangeGroup::NotSet:
			UsedGroupName = FName();
			break;
		case FInterchangeGroup::SetAndValid:
			UsedGroupName = UsedInterchangeGroup.DisplayName;
			break;
		case FInterchangeGroup::SetAndInvalid:
			UsedGroupName = FName("Invalid Group Used, Defaulting to No Group usage.");
			break;
		default:
			break;
	}
}

void UInterchangeEditorSettings::UpdateUsedGroupUIDFromGroupName()
{
	UsedGroupUID = FGuid();

	const UInterchangeProjectSettings* ImportSettings = GetDefault<UInterchangeProjectSettings>();
	for (const FInterchangeGroup& Group : ImportSettings->InterchangeGroups)
	{
		if (Group.DisplayName == UsedGroupName)
		{
			UsedGroupUID = Group.UniqueID;
		}
	}

	UpdateUsedGroupName();
}

#if WITH_EDITOR
void UInterchangeEditorSettings::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	Super::PostEditChangeProperty(PropertyChangedEvent);

	FProperty* PropertyThatChanged = PropertyChangedEvent.MemberProperty;
	const FString PropertyName = PropertyThatChanged ? PropertyThatChanged->GetName() : TEXT("");

	if (PropertyName == GET_MEMBER_NAME_STRING_CHECKED(UInterchangeEditorSettings, UsedGroupName))
	{
		UpdateUsedGroupUIDFromGroupName();
	}
}

void UInterchangeEditorSettings::PostInitProperties()
{
	Super::PostInitProperties();

	UpdateUsedGroupName();
}

#endif
