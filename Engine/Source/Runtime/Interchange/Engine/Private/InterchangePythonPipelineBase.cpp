// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangePythonPipelineBase.h"

#include "InterchangePipelineBase.h"
#include "JsonObjectConverter.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UnrealType.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangePythonPipelineBase)

#if WITH_EDITOR
void UInterchangePythonPipelineBase::PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent)
{
	//Save the change in the parent of class UInterchangePythonPipelineBase
	UObject* CurrentOuter = GetOuter();
	while(CurrentOuter)
	{
		if (UInterchangePythonPipelineAsset* PythonPipelineAsset = Cast<UInterchangePythonPipelineAsset>(CurrentOuter))
		{
			PythonPipelineAsset->SetupFromPipeline(this, false);
			break;
		}
		if (CurrentOuter->IsA<UPackage>())
		{
			break;
		}
		CurrentOuter = CurrentOuter->GetOuter();
	}

}
#endif

void UInterchangePythonPipelineAsset::PostLoad()
{
	GetPackage()->SetPackageFlags(PKG_EditorOnly);
	Super::PostLoad();
	GeneratePipeline();
}

#if WITH_EDITOR
void UInterchangePythonPipelineAsset::PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent)
{
	if (PropertyChangedEvent.GetPropertyName() == GET_MEMBER_NAME_CHECKED(UInterchangePythonPipelineAsset, PythonClass))
	{
		//Empty the properties data, the user will have to edit the default properties and the locks again
		JsonDefaultProperties.Empty();
		GeneratePipeline();
	}
}
#endif

void UInterchangePythonPipelineAsset::GeneratePipeline()
{
	GeneratedPipeline = nullptr;
	if (UClass* LoadedClass = PythonClass.LoadSynchronous())
	{
		GeneratedPipeline = NewObject<UInterchangePythonPipelineBase>(this, LoadedClass);

		TSharedPtr<FJsonObject> RootObject;
		TSharedRef< TJsonReader<> > JsonReader = TJsonReaderFactory<>::Create(JsonDefaultProperties);
		if (FJsonSerializer::Deserialize(JsonReader, RootObject))
		{
			const TSharedPtr<FJsonObject> JsonPipelineProperties = RootObject->GetObjectField(TEXT("GeneratedPipeline"));
			FJsonObjectConverter::JsonObjectToUStruct(JsonPipelineProperties.ToSharedRef(), LoadedClass, GeneratedPipeline, 0, 0);
		}
	}
}

void UInterchangePythonPipelineAsset::SetupFromPipeline(const UInterchangePythonPipelineBase* PythonPipeline, const bool bRegeneratePipeline /* = true*/)
{
	if (!PythonPipeline)
	{
		return;
	}

	PythonClass = PythonPipeline->GetClass();
	JsonDefaultProperties.Empty();
	if (UClass* LoadedClass = PythonClass.LoadSynchronous())
	{
		TSharedRef<FJsonObject> RootObject = MakeShareable(new FJsonObject);
		TSharedRef<FJsonObject> PipelinePropertiesObject = MakeShareable(new FJsonObject);
		if (FJsonObjectConverter::UStructToJsonObject(LoadedClass, PythonPipeline, PipelinePropertiesObject, 0, 0))
		{
			RootObject->SetField(TEXT("GeneratedPipeline"), MakeShareable(new FJsonValueObject(PipelinePropertiesObject)));
		}
		//Write the json file
		FString Json;
		TSharedRef<TJsonWriter<> > JsonWriter = TJsonWriterFactory<>::Create(&Json, 0);
		if (FJsonSerializer::Serialize(RootObject, JsonWriter))
		{
			JsonDefaultProperties = Json;
		}
	}
	
	if (bRegeneratePipeline)
	{
		GeneratePipeline();
	}
}
