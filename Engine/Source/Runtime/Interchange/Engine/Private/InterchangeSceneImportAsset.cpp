// Copyright Epic Games, Inc. All Rights Reserved.

#include "InterchangeSceneImportAsset.h"

#include "InterchangeAssetImportData.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeImportCommon.h"
#include "InterchangeManager.h"

#include "Engine/AssetUserData.h"
#include "Engine/Blueprint.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "EngineUtils.h"
#include "LevelInstance/LevelInstanceActor.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Package.h"

#include UE_INLINE_GENERATED_CPP_BY_NAME(InterchangeSceneImportAsset)

UInterchangeSceneImportAsset::~UInterchangeSceneImportAsset()
{
#if WITH_EDITOR
	if (bWorldRenameCallbacksRegistered)
	{
		FWorldDelegates::OnPreWorldRename.RemoveAll(this);
		FWorldDelegates::OnPostWorldRename.RemoveAll(this);
	}
#endif
}

void UInterchangeSceneImportAsset::GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const
{
	PRAGMA_DISABLE_DEPRECATION_WARNINGS;
	Super::GetAssetRegistryTags(OutTags);
	PRAGMA_ENABLE_DEPRECATION_WARNINGS;
}

void UInterchangeSceneImportAsset::GetAssetRegistryTags(FAssetRegistryTagsContext Context) const
{
#if WITH_EDITORONLY_DATA
	if (AssetImportData)
	{
		Context.AddTag(FAssetRegistryTag(SourceFileTagName(), AssetImportData->GetSourceData().ToJson(), FAssetRegistryTag::TT_Hidden));
	}
#endif

	Super::GetAssetRegistryTags(Context);
}

void UInterchangeSceneImportAsset::PostLoad()
{
	Super::PostLoad();

#if WITH_EDITORONLY_DATA
	UpdateSceneObjects();
#endif

#if WITH_EDITOR
	RegisterWorldRenameCallbacks();
#endif
}

#if WITH_EDITOR
void UInterchangeSceneImportAsset::RegisterWorldRenameCallbacks()
{
	if (!bWorldRenameCallbacksRegistered)
	{
		bWorldRenameCallbacksRegistered = true;
		FWorldDelegates::OnPreWorldRename.AddUObject(this, &UInterchangeSceneImportAsset::OnPreWorldRename);
		FWorldDelegates::OnPostWorldRename.AddUObject(this, &UInterchangeSceneImportAsset::OnPostWorldRename);
	}
}

void UInterchangeSceneImportAsset::OnPreWorldRename(UWorld* World, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename)
{
	// This method is called twice, first before the name change on the outermost then before the name change on the asset
	// Only cache path and names on the first call to this method.
	if (PreviousWorldPath.IsEmpty())
	{
		PreviousWorldPath = World->GetOutermost()->GetPathName();
		PreviousWorldName = World->GetName();
		PreviousLevelName = World->GetCurrentLevel()->GetName();
	}
}

void UInterchangeSceneImportAsset::OnPostWorldRename(UWorld* World)
{
	PreEditChange(nullptr);

	TArray< FSoftObjectPath> EntriesToUpdate;
	EntriesToUpdate.Reserve(SceneObjects.Num());

	for (TPair<FSoftObjectPath, FString>& SceneObject : SceneObjects)
	{
		if (SceneObject.Key.GetAssetPathString().StartsWith(PreviousWorldPath))
		{
			EntriesToUpdate.Add(SceneObject.Key);
		}
	}

	FString NewWorldPath = World->GetOutermost()->GetPathName();
	FString NewWorldName = World->GetName();
	FString NewPrefix = World->GetCurrentLevel()->GetName() + TEXT(".");

	for (FSoftObjectPath& EntryToRemove : EntriesToUpdate)
	{
		FString UniqueID;
		SceneObjects.RemoveAndCopyValue(EntryToRemove, UniqueID);

		UInterchangeFactoryBaseNode* FactoryNode = AssetImportData->GetStoredFactoryNode(UniqueID);
		if(ensure(FactoryNode))
		{
			const FString DisplayName = FactoryNode->GetDisplayLabel();
			const FSoftObjectPath ObjectPath{ FTopLevelAssetPath(FName(NewWorldPath), FName(NewWorldName)), NewPrefix + DisplayName };

			FactoryNode->SetCustomReferenceObject(ObjectPath);
			SceneObjects.Add(ObjectPath, UniqueID);
		}
	}

	// Reset previously cached path and names for subsequent call to OnPreWorldRename
	PreviousWorldPath.Empty();
	PreviousWorldName.Empty();
	PreviousLevelName.Empty();

	PostEditChange();
}
#endif

void UInterchangeSceneImportAsset::AddAssetUserData( UAssetUserData* InUserData )
{
#if WITH_EDITORONLY_DATA
	if ( InUserData != nullptr )
	{
		UAssetUserData* ExistingData = GetAssetUserDataOfClass( InUserData->GetClass() );
		if ( ExistingData != nullptr )
		{
			AssetUserData.Remove( ExistingData );
		}
		AssetUserData.Add( InUserData );
	}
#endif // #if WITH_EDITORONLY_DATA
}

UAssetUserData* UInterchangeSceneImportAsset::GetAssetUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass )
{
#if WITH_EDITORONLY_DATA
	for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if ( Datum != nullptr && Datum->IsA( InUserDataClass ) )
		{
			return Datum;
		}
	}
#endif // #if WITH_EDITORONLY_DATA
	return nullptr;

}

void UInterchangeSceneImportAsset::RemoveUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass )
{
#if WITH_EDITORONLY_DATA
	for ( int32 DataIdx = 0; DataIdx < AssetUserData.Num(); DataIdx++ )
	{
		UAssetUserData* Datum = AssetUserData[DataIdx];
		if ( Datum != nullptr && Datum->IsA(InUserDataClass ) )
		{
			AssetUserData.RemoveAt( DataIdx );
			return;
		}
	}
#endif // #if WITH_EDITORONLY_DATA
}

const TArray<UAssetUserData*>* UInterchangeSceneImportAsset::GetAssetUserDataArray() const
{
#if WITH_EDITORONLY_DATA
	return &ToRawPtrTArrayUnsafe(AssetUserData);
#else
	return nullptr;
#endif // #if WITH_EDITORONLY_DATA
}

void UInterchangeSceneImportAsset::UpdateSceneObjects()
{
#if WITH_EDITORONLY_DATA
	ensure(AssetImportData);

	SceneObjects.Reset();

	AssetImportData->GetNodeContainer()->IterateNodesOfType<UInterchangeFactoryBaseNode>(
		[this](const FString& NodeUid, UInterchangeFactoryBaseNode* FactoryNode)
		{
			if (FactoryNode)
			{
				FSoftObjectPath ObjectPath;
				if (FactoryNode->GetCustomReferenceObject(ObjectPath))
				{
					this->SceneObjects.Add(ObjectPath, NodeUid);
				}
			}
		}
	);
#endif
}

UObject* UInterchangeSceneImportAsset::GetSceneObject(const FString& PackageName, const FString& AssetName, const FString& SubPathString) const
{
#if WITH_EDITORONLY_DATA
	const FSoftObjectPath ObjectPath( FTopLevelAssetPath(FName(PackageName), FName(AssetName)), SubPathString);
	const FName ObjectSubPathBaseName = FActorSpawnUtils::GetBaseName(*SubPathString);

	UObject* SceneObject = nullptr;
	for (const TPair<FSoftObjectPath, FString>& SceneObjectPair : SceneObjects)
	{
		const FSoftObjectPath& SceneObjectPath = SceneObjectPair.Key;

		if (SceneObjectPath.GetLongPackageFName() == ObjectPath.GetLongPackageFName()
			&& SceneObjectPath.GetAssetFName() == ObjectPath.GetAssetFName())
		{
			//World partition actor have a guid that we need to remove before comparison
			const FString SceneObjectSubPathString = SceneObjectPath.GetSubPathString();
			if (SceneObjectSubPathString.Contains(SubPathString)
				&& FActorSpawnUtils::GetBaseName(*SceneObjectSubPathString) == ObjectSubPathBaseName)
			{
				SceneObject = SceneObjectPath.TryLoad();
				break;
			}
		}
	}
	
	if (SceneObject)
	{
		if (IsValid(SceneObject))
		{
			return SceneObject;
		}

		// SceneObject is still in memory but invalid. Move it to TransientPackage
		// Call UObject::Rename because for actors AActor::Rename will unnecessarily unregister and re-register components
		SceneObject->UObject::Rename(nullptr, GetTransientPackage(), REN_DontCreateRedirectors);
	}

#endif

	return nullptr;
}

const UInterchangeFactoryBaseNode* UInterchangeSceneImportAsset::GetFactoryNodeFromPath(const FString& PackageName, const FString& AssetName, const FString& SubPathString) const
{
#if WITH_EDITORONLY_DATA
	const FSoftObjectPath ObjectPath(FTopLevelAssetPath(FName(PackageName), FName(AssetName)), SubPathString);
	return GetFactoryNodeFromPath(ObjectPath);
#else
	return nullptr;
#endif
}

const UInterchangeFactoryBaseNode* UInterchangeSceneImportAsset::GetFactoryNodeFromPath(const FSoftObjectPath& SoftObjectPath) const
{
#if WITH_EDITORONLY_DATA
	if (!ensure(AssetImportData))
	{
		return nullptr;
	}

	const FName ObjectSubPathBaseName = FActorSpawnUtils::GetBaseName(*SoftObjectPath.GetSubPathString());
	const FString SoftObjectPathSubPathString = SoftObjectPath.GetSubPathString();

	UObject* SceneObject = nullptr;
	for (const TPair<FSoftObjectPath, FString>& SceneObjectPair : SceneObjects)
	{
		const FSoftObjectPath& SceneObjectPath = SceneObjectPair.Key;
		if (SceneObjectPath.GetLongPackageFName() == SoftObjectPath.GetLongPackageFName()
			&& SceneObjectPath.GetAssetFName() == SoftObjectPath.GetAssetFName())
		{
			//World partition actor have a guid that we need to remove before comparison
			const FString SceneObjectSubPathString = SceneObjectPath.GetSubPathString();
			if (SceneObjectSubPathString.Contains(SoftObjectPathSubPathString)
				&& FActorSpawnUtils::GetBaseName(*SceneObjectSubPathString) == ObjectSubPathBaseName)
			{
				return AssetImportData->GetStoredFactoryNode(SceneObjectPair.Value);
			}
		}
	}
#endif

	return nullptr;
}

const UInterchangeFactoryBaseNode* UInterchangeSceneImportAsset::GetFactoryNodeFromUniqueID(const FString& InNodeUniqueId) const
{
#if WITH_EDITORONLY_DATA
	if (!ensure(AssetImportData))
	{
		return nullptr;
	}

	return AssetImportData->GetStoredFactoryNode(InNodeUniqueId);
#else
	return nullptr;
#endif
}

TArray<const UInterchangeFactoryBaseNode*> UInterchangeSceneImportAsset::GetFactoryNodesOfClass(const UClass* Class) const
{
#if WITH_EDITORONLY_DATA
	if (!ensure(AssetImportData && Class && Class->IsChildOf(UInterchangeFactoryBaseNode::StaticClass())))
	{
		return {};
	}

	if (UInterchangeBaseNodeContainer* Container = AssetImportData->GetNodeContainer())
	{
		TArray<const UInterchangeFactoryBaseNode*> FactoryNodes;

		Container->IterateNodes([&FactoryNodes, Class](const FString& NodeUid, UInterchangeBaseNode* Node)
			{
				if (Node && Node->IsA(Class))
				{
					FactoryNodes.Add(Cast<UInterchangeFactoryBaseNode>(Node));
				}
			});

		return FactoryNodes;
	}
#endif

	return {};
}

void UInterchangeSceneImportAsset::GetSceneSoftObjectPaths(TArray<FSoftObjectPath>& SoftObjectPaths) const
{
#if WITH_EDITORONLY_DATA
	SoftObjectPaths.Reserve(SceneObjects.Num());

	for (const TPair< FSoftObjectPath, FString >& Entry : SceneObjects)
	{
		SoftObjectPaths.Add(Entry.Key);
	}
#endif
}

bool UInterchangeSceneImportAsset::ContainsObject(const FSoftObjectPath& SoftObjectPath) const
{
#if WITH_EDITORONLY_DATA
	return SceneObjects.Contains(SoftObjectPath);
#else
	return false;
#endif
}