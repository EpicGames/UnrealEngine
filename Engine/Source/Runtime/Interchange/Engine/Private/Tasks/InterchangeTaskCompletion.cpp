// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskCompletion.h"

#include "AssetRegistry/AssetRegistryModule.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "Engine/World.h"
#include "GameFramework/Actor.h"
#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeManager.h"
#include "InterchangePipelineBase.h"
#include "InterchangeResultsContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/UObjectHash.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE::Interchange::Private::ObjectDeletionUtils
{
	static void PurgeObject(UObject* Object)
	{
		if (Object)
		{
			Object->ClearFlags(RF_Standalone | RF_Public | RF_Transactional);
			Object->ClearInternalFlags(EInternalObjectFlags::Async);
			Object->SetFlags(RF_Transient);
			Object->MarkAsGarbage();
			Object->UObject::Rename(nullptr, GetTransientPackage(), REN_NonTransactional | REN_DontCreateRedirectors);
		}
	}
}

void UE::Interchange::FTaskPreCompletion_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskPreCompletion_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(PreCompletion)
#endif
	
	LLM_SCOPE_BYNAME(TEXT("Interchange"));
	
	check(IsInGameThread());

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	UInterchangeResultsContainer* Results = AsyncHelper->AssetImportResult->GetResults();

	bool bIsAsset = true;

	auto IterationCallback = [&AsyncHelper, &Results, &bIsAsset](int32 SourceIndex, const TArray<FImportAsyncHelper::FImportedObjectInfo>& ImportedObjects)
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));
		//Verify if the task was cancel
		if (AsyncHelper->bCancel)
		{
			for (const FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedObjects)
			{
				//Cancel factories so they can do proper cleanup
				if (ObjectInfo.Factory)
				{
					ObjectInfo.Factory->Cancel();
				}
			}
			//Skip if cancel
			return;
		}

		const bool bCallPostImportGameThreadCallback = ensure(AsyncHelper->SourceDatas.IsValidIndex(SourceIndex));

		UInterchangeFactoryBase::FSetupObjectParams Arguments;
		Arguments.SourceData = AsyncHelper->SourceDatas[SourceIndex];
		Arguments.NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
		Arguments.Pipelines = AsyncHelper->Pipelines;
		Arguments.OriginalPipelines = AsyncHelper->OriginalPipelines;
		Arguments.Translator = AsyncHelper->Translators[SourceIndex];

		//First iteration to call SetupObject_GameThread and pipeline ExecutePostFactoryPipeline
		for (const FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedObjects)
		{
			UObject* ImportedObject = ObjectInfo.GetImportedObject();
			//In case Some factory code cannot run outside of the main thread we offer this callback to finish the work before calling post edit change (building the asset)
			if (bCallPostImportGameThreadCallback && ObjectInfo.Factory)
			{
				Arguments.ImportedObject = ImportedObject;
				Arguments.FactoryNode = ObjectInfo.FactoryNode;
				Arguments.NodeUniqueID = ObjectInfo.FactoryNode ? ObjectInfo.FactoryNode->GetUniqueID() : FString();
				Arguments.bIsReimport = ObjectInfo.bIsReimport;
				ObjectInfo.Factory->SetupObject_GameThread(Arguments);
			}

			if (ImportedObject == nullptr || !IsValid(ImportedObject))
			{
				continue;
			}

			UInterchangeResultSuccess* Message = Results->Add<UInterchangeResultSuccess>();
			Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
			Message->DestinationAssetName = ImportedObject->GetPathName();
			Message->AssetType = ImportedObject->GetClass();

			//Clear any async flag from the created asset and all its subobjects
			const EInternalObjectFlags AsyncFlags = EInternalObjectFlags::Async;
			ImportedObject->ClearInternalFlags(AsyncFlags);

			TArray<UObject*> ImportedSubobjects;
			GetObjectsWithOuter(ImportedObject, ImportedSubobjects, EGetObjectsFlags::IncludeNestedObjects);
			for (UObject* ImportedSubobject : ImportedSubobjects)
			{
				ImportedSubobject->ClearInternalFlags(AsyncFlags);
			}

			//Make sure the package is dirty
			ImportedObject->MarkPackageDirty();

			if (!bIsAsset)
			{
				if (AActor* Actor = Cast<AActor>(ImportedObject))
				{
#if WITH_EDITOR
					Message->AssetFriendlyName = Actor->GetActorLabel();
#endif
					Actor->RegisterAllComponents();
				}
				else if (UActorComponent* Component = Cast<UActorComponent>(ImportedObject))
				{
					Component->RegisterComponent();
				}
			}

			for (UInterchangePipelineBase* PipelineBase : AsyncHelper->Pipelines)
			{
				PipelineBase->ScriptedExecutePostFactoryPipeline(AsyncHelper->BaseNodeContainers[SourceIndex].Get()
					, ObjectInfo.FactoryNode ? ObjectInfo.FactoryNode->GetUniqueID() : FString()
					, ImportedObject
					, ObjectInfo.bIsReimport);
			}
		}

#if WITH_EDITOR
		//Second iteration to call BuildObject
		for (const FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedObjects)
		{
			UObject* ImportedObject = ObjectInfo.GetImportedObject();
			if (ImportedObject == nullptr || !IsValid(ImportedObject))
			{
				continue;
			}
			//The base class of the factory will call posteditchange, but other factory can instead simply build
			//the asset asynchronously and the post edit change will be call later
			Arguments.ImportedObject = ImportedObject;
			Arguments.FactoryNode = ObjectInfo.FactoryNode;
			Arguments.NodeUniqueID = ObjectInfo.FactoryNode ? ObjectInfo.FactoryNode->GetUniqueID() : FString();
			Arguments.bIsReimport = ObjectInfo.bIsReimport;
			ObjectInfo.Factory->BuildObject_GameThread(Arguments, ObjectInfo.bPostEditChangeCalled);
		}
#endif //WITH_EDITOR

		//Third iteration to register the assets
		for (const FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedObjects)
		{
			UObject* ImportedObject = ObjectInfo.GetImportedObject();
			if (ImportedObject == nullptr || !IsValid(ImportedObject))
			{
				continue;
			}
			//Register the assets
			if (bIsAsset)
			{
				AsyncHelper->AssetImportResult->AddImportedObject(ImportedObject);
			}
			else
			{
				AsyncHelper->SceneImportResult->AddImportedObject(ImportedObject);
			}
		}
	};

	//Asset import
	bIsAsset = true;
	AsyncHelper->IterateImportedAssetsPerSourceIndex(IterationCallback);

	//Scene import
	bIsAsset = false;
	AsyncHelper->IterateImportedSceneObjectsPerSourceIndex(IterationCallback);
}


void UE::Interchange::FTaskCompletion_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskCompletion_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(Completion)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));
	check(IsInGameThread());

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	if (AsyncHelper->AnalyticsHandler.IsValid())
	{
		FInterchangeImportResultAnalyticsInfo ImportResultAnalyticsInfo = FImportAsyncHelper::CreateImportResultAnalyticsInfo(*AsyncHelper.Get());
		AsyncHelper->AnalyticsHandler->Send(ImportResultAnalyticsInfo);
	}
	
	//No need anymore of the translators sources
	AsyncHelper->ReleaseTranslatorsSource();

	bool bImportedSomething = false;

	if (!AsyncHelper->bCancel)
	{
		//Broadcast OnAssetPostImport/OnAssetPostReimport for each imported asset
		AsyncHelper->IterateImportedAssetsPerSourceIndex([AsyncHelper, &bImportedSomething](int32 SourceIndex, const TArray<FImportAsyncHelper::FImportedObjectInfo>& AssetInfos)
			{
				for (const FImportAsyncHelper::FImportedObjectInfo& AssetInfo : AssetInfos)
				{
					if (UObject* Asset = AssetInfo.GetImportedObject())
					{
						if (Asset->HasAnyFlags(RF_MirroredGarbage))
						{
							UE::Interchange::Private::ObjectDeletionUtils::PurgeObject(Asset);
							continue;
						}

						if (!AsyncHelper->TaskData.ReimportObject)
						{
							//Notify the asset registry, only when we have created the asset
							FAssetRegistryModule::AssetCreated(Asset);
							bImportedSomething = true;
						}
						else if (AsyncHelper->TaskData.ReimportObject && AsyncHelper->TaskData.ReimportObject == Asset)
						{
							UInterchangeManager::GetInterchangeManager().OnAssetPostReimport.Broadcast(Asset);
							bImportedSomething = true;
						}
						//We broadcast this event for both import and reimport.
						UInterchangeManager::GetInterchangeManager().OnAssetPostImport.Broadcast(Asset);
					}
				}

				//Do a second pass for the post broadcast pipeline call
				for (const FImportAsyncHelper::FImportedObjectInfo& AssetInfo : AssetInfos)
				{
					if (UObject* Asset = AssetInfo.GetImportedObject())
					{
						for (UInterchangePipelineBase* PipelineBase : AsyncHelper->Pipelines)
						{
							PipelineBase->ScriptedExecutePostBroadcastPipeline(AsyncHelper->BaseNodeContainers[SourceIndex].Get()
								, AssetInfo.FactoryNode ? AssetInfo.FactoryNode->GetUniqueID() : FString()
								, Asset
								, AssetInfo.bIsReimport);
						}
					}
				}

				UE_LOGF(LogInterchangeEngine, Display, "Interchange import completed [%ls]", *AsyncHelper->SourceDatas[SourceIndex]->ToDisplayString());
			});

		//Iterate the Scene Actors
		AsyncHelper->IterateImportedSceneObjectsPerSourceIndex([AsyncHelper, &bImportedSomething](int32 SourceIndex, const TArray<FImportAsyncHelper::FImportedObjectInfo>& AssetInfos)
			{
				for (const FImportAsyncHelper::FImportedObjectInfo& SceneObjectInfo : AssetInfos)
				{
					if (AActor* Actor = Cast<AActor>(SceneObjectInfo.GetImportedObject()))
					{
						bImportedSomething = true;
						for (UInterchangePipelineBase* PipelineBase : AsyncHelper->Pipelines)
						{
							PipelineBase->ScriptedExecutePostBroadcastPipeline(AsyncHelper->BaseNodeContainers[SourceIndex].Get()
								, SceneObjectInfo.FactoryNode ? SceneObjectInfo.FactoryNode->GetUniqueID() : FString()
								, Actor
								, SceneObjectInfo.bIsReimport);
						}
					}
				}

				UE_LOGF(LogInterchangeEngine, Display, "Interchange import completed [%ls]", *AsyncHelper->SourceDatas[SourceIndex]->ToDisplayString());
			});
	}
	else
	{
		//If task is canceled, delete all created assets by this task
		AsyncHelper->IterateImportedAssetsPerSourceIndex([AsyncHelper](int32 SourceIndex, const TArray<FImportAsyncHelper::FImportedObjectInfo>& AssetInfos)
			{
				UE_LOGF(LogInterchangeEngine, Display, "Interchange import cancelled [%ls]", *AsyncHelper->SourceDatas[SourceIndex]->ToDisplayString());
				for (const FImportAsyncHelper::FImportedObjectInfo& AssetInfo : AssetInfos)
				{
					using namespace UE::Interchange::Private::ObjectDeletionUtils;

					//Make any created asset go away
					UObject* ObjectToPurge = AssetInfo.GetImportedObject();
					PurgeObject(ObjectToPurge);
				}
			});

		//If task is canceled, remove all actors from their world
		AsyncHelper->IterateImportedSceneObjectsPerSourceIndex([AsyncHelper](int32 SourceIndex, const TArray<FImportAsyncHelper::FImportedObjectInfo>& AssetInfos)
			{
				for (const FImportAsyncHelper::FImportedObjectInfo& SceneObjectInfo : AssetInfos)
				{
					if (AActor* Actor = Cast<AActor>(SceneObjectInfo.GetImportedObject()))
					{
						if (UWorld* ActorWorld = Actor->GetWorld())
						{
							const bool bModifyLevel = false; //This isn't undoable
							ActorWorld->RemoveActor(Actor, bModifyLevel);
						}
					}
				}

				UE_LOGF(LogInterchangeEngine, Display, "Interchange import cancelled [%ls]", *AsyncHelper->SourceDatas[SourceIndex]->ToDisplayString());
			});
	}

	if (!bImportedSomething && !AsyncHelper->bCancel)
	{
		for (TObjectPtr<UInterchangeTranslatorBase> Translator : AsyncHelper->Translators)
		{
			// We need a valid translator to add a result message
			if (!Translator)
			{
				continue;
			}

			UInterchangeResultError_Generic* ErrorResult = NewObject<UInterchangeResultError_Generic>(GetTransientPackage(), UInterchangeResultError_Generic::StaticClass());
			ErrorResult->Text = NSLOCTEXT("InterchangeTaskCompletion", "NothingImported", "There was nothing to import from the provided source data using the chosen pipeline options.");
			ErrorResult->SourceAssetName = Translator->GetSourceData()->GetFilename();
			Translator->AddMessage(ErrorResult);
		}
	}

	AsyncHelper->AssetImportResult->SetDone();
	AsyncHelper->SceneImportResult->SetDone();

	//Release the async helper
	AsyncHelper = nullptr;
	InterchangeManager->ReleaseAsyncHelper(WeakAsyncHelper);
}
