// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeManager.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE
{
	namespace Interchange
	{
		class FTaskPreCompletion_GameThread : public FInterchangeTaskBase
		{
		private:
			UInterchangeManager* InterchangeManager;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		public:
			FTaskPreCompletion_GameThread(UInterchangeManager* InInterchangeManager, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: InterchangeManager(InInterchangeManager)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};

		class FTaskCompletion_GameThread : public FInterchangeTaskBase
		{
		private:
			UInterchangeManager* InterchangeManager;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		public:
			FTaskCompletion_GameThread(UInterchangeManager* InInterchangeManager, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: InterchangeManager(InInterchangeManager)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};

	} //ns Interchange
}//ns UE

