// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskCreateSceneObjects.h"

#include "CoreMinimal.h"

#include "Async/TaskGraphInterfaces.h"
#include "Engine/Level.h"
#include "Engine/World.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeImportCommon.h"
#include "InterchangeManager.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "PackageUtils/PackageUtils.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"

UE::Interchange::FTaskCreateSceneObjects_GameThread::FTaskCreateSceneObjects_GameThread(const FString& InPackageBasePath, const int32 InSourceIndex, TWeakPtr<FImportAsyncHelper> InAsyncHelper, TArrayView<UInterchangeFactoryBaseNode*> InFactoryNodes, const UClass* InFactoryClass)
	: PackageBasePath(InPackageBasePath)
	, SourceIndex(InSourceIndex)
	, WeakAsyncHelper(InAsyncHelper)
	, FactoryNodes(InFactoryNodes)
	, FactoryClass(InFactoryClass)
{
	check(FactoryClass);
}

void UE::Interchange::FTaskCreateSceneObjects_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskCreateSceneObjects_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(SpawnActor)
#endif
	using namespace UE::Interchange;

	check(IsInGameThread());
	TSharedPtr<FImportAsyncHelper> AsyncHelper = WeakAsyncHelper.Pin();
	check(WeakAsyncHelper.IsValid());

	//Verify if the task was canceled
	if (AsyncHelper->bCancel)
	{
		return;
	}

	UObject* ReimportObject = AsyncHelper->TaskData.ReimportObject;
	ULevel* ImportLevel = AsyncHelper->TaskData.ImportLevel ? AsyncHelper->TaskData.ImportLevel : GWorld->GetCurrentLevel();
	UWorld* ImportWorld = ImportLevel->GetWorld();

	for (UInterchangeFactoryBaseNode* FactoryNode : FactoryNodes)
	{
		// Don't create actors on a first import if they're disabled
		if (!FactoryNode || !FactoryNode->IsEnabled())
		{
			continue;
		}

		UInterchangeFactoryBase* Factory = NewObject<UInterchangeFactoryBase>(GetTransientPackage(), FactoryClass);
		Factory->SetResultsContainer(AsyncHelper->AssetImportResult->GetResults());
		AsyncHelper->AddCreatedFactory(FactoryNode->GetUniqueID(), Factory);

		FString SceneNodeName = FactoryNode->GetAssetName();
		UInterchangeManager::GetInterchangeManager().SanitizeNameInline(SceneNodeName, ESanitizeNameTypeFlags::ObjectName | ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);

		ULevel* FactoryImportLevel = ImportLevel;
		UWorld* FactoryImportWorld = ImportWorld;

		// Check whether the factory node is part of a specific sub-level. If so, set the latter as the target level.
		FString FactoryNodeLevelUid;
		if (FactoryNode->GetCustomLevelUid(FactoryNodeLevelUid))
		{
			if (UInterchangeFactoryBaseNode* LevelFactoryNode = AsyncHelper->BaseNodeContainers[SourceIndex]->GetFactoryNode(FactoryNodeLevelUid))
			{
				FSoftObjectPath LevelAssetPath;
				if (LevelFactoryNode->GetCustomReferenceObject(LevelAssetPath))
				{
					if (UWorld* FactoryNodeWorld = Cast<UWorld>(LevelAssetPath.TryLoad()))
					{
#if WITH_EDITOR
						// Because of the level instance's referenced world being edited then saved on reimports
						// The referenced world's cleanup state get updated to a non zero value which prevents
						// the registration of all the world's components.
						// A temporary fix is to reset that state. This may have side effects. No found any so far.
						// In any case, the creation and update of level instances and its associated level needs to be
						// revisited to better align on how this is done in the editor in next release.
						// #interchange_levelinstance_todo: Revisit this logic when reworking the handling of level instances
						if (FactoryNodeWorld->IsCleanedUp())
						{
							UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
							Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
							Message->DestinationAssetName = SceneNodeName;
							Message->AssetType = LevelFactoryNode->GetObjectClass();
							Message->Text = NSLOCTEXT("FTaskCreateSceneObjects_GameThread", "DirtyWorld", "World has not been properly cleaned up. Reseting state.");
							
							FactoryNodeWorld->ResetCleanedUpState();
						}
#endif

						if (FactoryNodeWorld->PersistentLevel)
						{
							FactoryImportWorld = FactoryNodeWorld;
							FactoryImportLevel = FactoryNodeWorld->PersistentLevel;
						}
					}
				}
			}
		}
		const FString FactoryNodeWorldPath = FactoryImportWorld->GetOutermost()->GetPathName();
		const FString FactoryNodeWorldName = FactoryImportWorld->GetName();
		const FString FactoryNodeSceneNodeName = FactoryImportLevel->GetName() + TEXT(".") + SceneNodeName;

		UInterchangeFactoryBase::FImportSceneObjectsParams CreateSceneObjectsParams;
		CreateSceneObjectsParams.ObjectName = SceneNodeName;
		CreateSceneObjectsParams.FactoryNode = FactoryNode;
		CreateSceneObjectsParams.Level = FactoryImportLevel;
		CreateSceneObjectsParams.ReimportObject = FFactoryCommon::GetObjectToReimport(Factory, ReimportObject, *FactoryNode, FactoryNodeWorldPath, FactoryNodeWorldName, FactoryNodeSceneNodeName);
		CreateSceneObjectsParams.ReimportFactoryNode = FFactoryCommon::GetFactoryNode(ReimportObject, FactoryNodeWorldPath, FactoryNodeWorldName, FactoryNodeSceneNodeName);
		CreateSceneObjectsParams.bIsReimport = ReimportObject != nullptr;
		CreateSceneObjectsParams.AsyncHelperUid = AsyncHelper->UniqueId;

		// Skip factory node if conditions are not meant
		if (!FFactoryCommon::CanProceedWithFactoryNode(*FactoryNode, ReimportObject, CreateSceneObjectsParams.ReimportObject, FactoryNodeWorldPath, FactoryNodeWorldName, FactoryNodeSceneNodeName))
		{
			FactoryNode->SetSkipNodeImport();
			continue;
		}

		if (AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex))
		{
			CreateSceneObjectsParams.NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
			CreateSceneObjectsParams.SourceData = AsyncHelper->SourceDatas[SourceIndex];
		}

		UObject* SceneObject = Factory->ImportSceneObject_GameThread(CreateSceneObjectsParams);
		if (SceneObject)
		{
			UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ObjectInfo = AsyncHelper->AddDefaultImportedSceneObjectGetRef(SourceIndex);
			ObjectInfo.SetImportedObject(SceneObject);
			ObjectInfo.Factory = Factory;
			ObjectInfo.FactoryNode = FactoryNode;
			FactoryNode->SetCustomReferenceObject(FSoftObjectPath(SceneObject));
		}
	}
}
