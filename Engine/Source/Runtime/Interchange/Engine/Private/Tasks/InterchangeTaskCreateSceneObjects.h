// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Nodes/InterchangeBaseNode.h"

class UInterchangeBaseNode;
class UInterchangeFactoryBaseNode;

namespace UE
{
	namespace Interchange
	{
		class FImportAsyncHelper;

		class FTaskCreateSceneObjects_GameThread : public FInterchangeTaskBase
		{
		private:
			FString PackageBasePath;
			int32 SourceIndex;
			TWeakPtr<FImportAsyncHelper> WeakAsyncHelper;
			TArray<UInterchangeFactoryBaseNode*> FactoryNodes;
			const UClass* FactoryClass;

		public:
			explicit FTaskCreateSceneObjects_GameThread(const FString& InPackageBasePath, const int32 InSourceIndex, TWeakPtr<FImportAsyncHelper> InAsyncHelper, TArrayView<UInterchangeFactoryBaseNode*> InNodes, const UClass* InFactoryClass);

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};
	} //ns Interchange
}//ns UE
