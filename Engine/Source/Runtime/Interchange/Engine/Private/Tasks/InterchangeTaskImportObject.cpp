// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskImportObject.h"

#include "AssetCompilingManager.h"
#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "Engine/World.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeImportCommon.h"
#include "InterchangeManager.h"
#include "InterchangeResult.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Misc/App.h"
#include "Misc/MessageDialog.h"
#include "Misc/Paths.h"
#include "PackageUtils/PackageUtils.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/Object.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/ObjectMacros.h"
#include "UObject/ObjectRedirector.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "InterchangeHelper.h"

namespace UE::Interchange::Private
{
	void InternalGetPackageName(const UE::Interchange::FImportAsyncHelper& AsyncHelper, const int32 SourceIndex, const FString& PackageBasePath, const UInterchangeFactoryBaseNode* FactoryNode, FString& OutPackageName, FString& OutAssetName)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::Private::InternalGetPackageName)
		const UInterchangeSourceData* SourceData = AsyncHelper.SourceDatas[SourceIndex];
		check(SourceData);
		FString NodeDisplayName = FactoryNode->GetAssetName();
		
		// Set the asset name and the package name
		OutAssetName = NodeDisplayName;
		UInterchangeManager::GetInterchangeManager().SanitizeNameInline(OutAssetName, ESanitizeNameTypeFlags::ObjectName | ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);
		FString SubPath;
		FactoryNode->GetCustomSubPath(SubPath);

		FString PackagePath = FPaths::Combine(*PackageBasePath, *SubPath);
		int32 MaxPath = GetAssetNameMaxCharCount(*PackagePath);

		// We don't want to take into account the '/Game' length, hence number 5
		OutAssetName = GetAssetNameWBudget(OutAssetName, MaxPath);

		OutPackageName = FPaths::Combine(*PackagePath, *OutAssetName);
		UInterchangeManager::GetInterchangeManager().SanitizeNameInline(OutPackageName, ESanitizeNameTypeFlags::ObjectPath | ESanitizeNameTypeFlags::LongPackage);
	}

	bool ShouldReimportFactoryNode(UInterchangeFactoryBaseNode* FactoryNode, const UInterchangeBaseNodeContainer* NodeContainer, UObject* ReimportObject)
	{
		TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::Private::ShouldReimportFactoryNode)

		if (!NodeContainer)
		{
			return false;
		}
		//Find all potential factory node
		TArray<UInterchangeFactoryBaseNode*> PotentialFactoryNodes;
		UClass* FactoryClass = FactoryNode->GetObjectClass();
		NodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([FactoryClass, &PotentialFactoryNodes](const FString& NodeUniqueID, UInterchangeFactoryBaseNode* CurrentFactoryNode)
			{
				if (UClass* CurrentFactoryClass = CurrentFactoryNode->GetObjectClass())
				{
					if (CurrentFactoryClass->IsChildOf(FactoryClass))
					{
						PotentialFactoryNodes.Add(CurrentFactoryNode);
					}
				}
			});
		
		if (PotentialFactoryNodes.Num() == 1)
		{
			//There is only one factory node that will generate this class of UObject, no need to match the unique id or the name.
			ensure(PotentialFactoryNodes[0] == FactoryNode);
			//The class of the object must fit with the factory object class.
			return ReimportObject->GetClass()->IsChildOf(FactoryClass);
		}

		//See if the FactoryNode match the ReimportObject.
		if (UInterchangeAssetImportData* OriginalAssetImportData = UInterchangeAssetImportData::GetFromObject(ReimportObject))
		{
			if (UInterchangeBaseNodeContainer* OriginalNodeContainer = OriginalAssetImportData->GetNodeContainer())
			{
				//Find the original factory node used by the last ReimportObject import
				if (UInterchangeFactoryBaseNode* OriginalFactoryNode = OriginalNodeContainer->GetFactoryNode(OriginalAssetImportData->NodeUniqueID))
				{
					//Compare the original factory node UObject class to the factory node UObject class
					if (OriginalFactoryNode->GetObjectClass()->IsChildOf(FactoryClass))
					{
						//Compare the original factory node, unique id to the factory node unique id
						if (OriginalFactoryNode->GetUniqueID().Equals(FactoryNode->GetUniqueID()))
						{
							return true;
						}
						//Compare the original factory node, name to the factory node name
						if (OriginalFactoryNode->GetDisplayLabel().Equals(FactoryNode->GetDisplayLabel()))
						{
							FString PackageSubPath;
							FactoryNode->GetCustomSubPath(PackageSubPath);
							FString OriginalPackageSubPath;
							OriginalFactoryNode->GetCustomSubPath(PackageSubPath);
							//Make sure both sub path are equal
							if (PackageSubPath.Equals(OriginalPackageSubPath))
							{
								return true;
							}
						}
					}
				}
			}
		}

		return false;
	}




	bool CanImportClass(UE::Interchange::FImportAsyncHelper& AsyncHelper, UInterchangeFactoryBaseNode& FactoryNode, int32 SourceIndex, bool bLogError = false)
	{
		if (AsyncHelper.bRuntimeOrPIE && !FactoryNode.IsRuntimeImportAllowed())
		{
			if (bLogError)
			{
				UE_LOGF(LogInterchangeEngine, Error, "Cannot import %ls asset at runtime. This is an editor-only feature.", *FactoryNode.GetTypeName());
			}
			return false;
		}

		if (UClass* Class = FactoryNode.GetObjectClass())
		{
			return AsyncHelper.IsClassImportAllowed(Class);
		}
		
		return false;
	}

	UInterchangeFactoryBase::FImportAssetResult InternalImportObjectStartup(TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper
		, UInterchangeFactoryBaseNode* FactoryNode
		, int32 SourceIndex
		, const FString& PackageBasePath
		, TFunction<UInterchangeFactoryBase::FImportAssetResult(UInterchangeFactoryBase::FImportAssetObjectParams&)>FactoryOperationLambda)
	{
		check(AsyncHelper.IsValid());
		UInterchangeFactoryBase::FImportAssetResult ErrorImportAssetResult;

		//Verify if the task was cancel
		if (AsyncHelper->bCancel || !FactoryNode || !Private::CanImportClass(*AsyncHelper, *FactoryNode, SourceIndex))
		{
			return ErrorImportAssetResult;
		}

		if (FactoryNode->ShouldSkipNodeImport())
		{
			return ErrorImportAssetResult;
		}

		UInterchangeFactoryBase* Factory = AsyncHelper->GetCreatedFactory(FactoryNode->GetUniqueID());
		if(!ensure(Factory))
		{
			UE_LOGF(LogInterchangeEngine, Error, "The factory to import the factory node was not created. FactoryNode: class '%ls' name '%ls'.", *FactoryNode->GetObjectClass()->GetName(), *FactoryNode->GetDisplayLabel());
			return ErrorImportAssetResult;
		}

		UPackage* Pkg = nullptr;
		FString PackageName;
		FString AssetName;
		Private::InternalGetPackageName(*AsyncHelper, SourceIndex, PackageBasePath, FactoryNode, PackageName, AssetName);
		bool bSkipAsset = false;
		UObject* ObjectToReimport = UE::Interchange::FFactoryCommon::GetObjectToReimport(Factory, AsyncHelper->TaskData.ReimportObject, *FactoryNode, PackageName, AssetName);
		if (ObjectToReimport)
		{
			UInterchangeBaseNodeContainer* NodeContainer = nullptr;
			if (AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex))
			{
				NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
			}

			if (Private::ShouldReimportFactoryNode(FactoryNode, NodeContainer, ObjectToReimport))
			{
				Pkg = ObjectToReimport->GetPackage();
				PackageName = Pkg->GetPathName();
				AssetName = ObjectToReimport->GetName();
			}
			else
			{
				bSkipAsset = true;
			}
		}
		else
		{
			// Check if the object has not been deleted after an import
			if (const UInterchangeFactoryBaseNode* ReimportFactoryNode = FFactoryCommon::GetFactoryNode(AsyncHelper->TaskData.ReimportObject, PackageName, AssetName))
			{
				// if the reimport of the node has not been set, do not recreate the asset
				bSkipAsset = !FactoryNode->ShouldForceNodeReimport();
			}

			if (!bSkipAsset)
			{
				Pkg = AsyncHelper->GetCreatedPackage(PackageName);

				if (!Pkg)
				{
					UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
					Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
					Message->DestinationAssetName = AssetName;
					Message->AssetType = FactoryNode->GetObjectClass();
					Message->Text = NSLOCTEXT("InternalImportObjectStartup", "BadPackage", "It was not possible to create the asset because its package was not created correctly.");
					return ErrorImportAssetResult;
				}

				if (!AsyncHelper->SourceDatas.IsValidIndex(SourceIndex) || !AsyncHelper->Translators.IsValidIndex(SourceIndex))
				{
					UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
					Message->DestinationAssetName = AssetName;
					Message->AssetType = FactoryNode->GetObjectClass();
					Message->Text = NSLOCTEXT("InternalImportObjectStartup", "SourceDataOrTranslatorInvalid", "It was not possible to create the asset because its translator was not created correctly.");
					return ErrorImportAssetResult;
				}
			}
		}

		if (!bSkipAsset)
		{
			if (AsyncHelper->TaskData.bFollowRedirectors)
			{
				// Check to see if we were redirected. If so, and if the asset name matched the package name, change the name of the asset to match the new package as well
				if (Pkg->GetName() != PackageName && FPackageName::GetLongPackageAssetName(PackageName) == AssetName)
				{
					AssetName = FPackageName::GetLongPackageAssetName(Pkg->GetName());
				}
			}

			UInterchangeTranslatorBase* Translator = AsyncHelper->Translators[SourceIndex];
			//Import Asset describe by the node
			UInterchangeFactoryBase::FImportAssetObjectParams CreateAssetParams;
			CreateAssetParams.AssetName = AssetName;
			CreateAssetParams.AssetNode = FactoryNode;
			CreateAssetParams.Parent = Pkg;
			CreateAssetParams.SourceData = AsyncHelper->SourceDatas[SourceIndex];
			CreateAssetParams.Translator = Translator;
			if (AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex))
			{
				CreateAssetParams.NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
			}
			CreateAssetParams.ReimportObject = ObjectToReimport;

			return FactoryOperationLambda(CreateAssetParams);
		}
		return ErrorImportAssetResult;
	}
}//ns UE::Interchange::Private

void UE::Interchange::FTaskImportObject_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskImportObject_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(TaskImportObject_GameThread)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	using namespace UE::Interchange;

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	//Set it to true if everything is fine and you don't want to skip the node import
	bool bCanContinueImport = false;
	ON_SCOPE_EXIT
	{
		if (FactoryNode && !bCanContinueImport)
		{
			  FactoryNode->SetSkipNodeImport();
		}
	};
	
	//Verify if the task was cancel
	if (AsyncHelper->bCancel || !FactoryNode)
	{
		return;
	}

	//The create package thread must always execute on the game thread
	check(IsInGameThread());

	// Create factory
	UInterchangeFactoryBase* Factory = AsyncHelper->GetCreatedFactory(FactoryNode->GetUniqueID());
	if (!ensure(Factory))
	{
		return;
	}

	UPackage* Pkg = nullptr;
	FString PackageName;
	FString AssetName;
	UObject* ReimportObject = AsyncHelper->TaskData.ReimportObject;
	Private::InternalGetPackageName(*AsyncHelper, SourceIndex, PackageBasePath, FactoryNode, PackageName, AssetName);
	UObject* ObjectToReimport = FFactoryCommon::GetObjectToReimport(Factory, ReimportObject, *FactoryNode, PackageName, AssetName);

	// Skip factory node if conditions are not meant
	if (!FFactoryCommon::CanProceedWithFactoryNode(*FactoryNode, ReimportObject, ObjectToReimport, PackageName, AssetName))
	{
		FactoryNode->SetSkipNodeImport();
		return;
	}

	if (!ensure(!IsGarbageCollecting()))
	{
		//Skip this asset
		UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
		Message->DestinationAssetName = ObjectToReimport ? ObjectToReimport->GetName() : FactoryNode->GetAssetName();
		Message->AssetType = FactoryNode->GetObjectClass();
		Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "GarbageCollectIsRunning_Error", "Cannot import asset '{0}'. The garbage collector is running during the import process.")
			, FText::FromString(AssetName));
		return;
	}

	if (!AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex))
	{
		//Skip this asset
		UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
		Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
		Message->DestinationAssetName = ObjectToReimport ? ObjectToReimport->GetName() : FactoryNode->GetAssetName();
		Message->AssetType = FactoryNode->GetObjectClass();
		Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "InvalidBaseNodeContainer", "Cannot import asset '{0}'. The node container is invalid for this source index.")
			, FText::FromString(AssetName));
		return;
	}
	UInterchangeBaseNodeContainer* NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
	bool bShouldReimportFactoryNode = ObjectToReimport && Private::ShouldReimportFactoryNode(FactoryNode, NodeContainer, ObjectToReimport);

	//Check if class Import is allowed or not.
	if (!Private::CanImportClass(*AsyncHelper, *FactoryNode, SourceIndex, !ObjectToReimport || bShouldReimportFactoryNode))
	{
		return;
	}

	bool bSkipObjectNoReplace = false;
	UObject* ExistingAsset = nullptr;

	auto FollowRedirectorCode = [AsyncHelper, &Pkg, &PackageName, &AssetName]()
		{
			if (AsyncHelper->TaskData.bFollowRedirectors)
			{
				if (UObjectRedirector* Redirector = FindObject<UObjectRedirector>(Pkg, *AssetName))
				{
					if (Redirector->DestinationObject)
					{
						Pkg = Redirector->DestinationObject->GetPackage();
						if (FPackageName::GetLongPackageAssetName(PackageName) == AssetName)
						{
							AssetName = FPackageName::GetLongPackageAssetName(Pkg->GetName());
						}
					}
				}
			}
		};

	//Check if the node reference a map
	const bool bRefObjectIsMap = FactoryNode->GetObjectClass()->IsChildOf<UWorld>();

	//If the factory node is disable see if there is an existing UObject for it
	if (!FactoryNode->IsEnabled())
	{
		Private::InternalGetPackageName(*AsyncHelper, SourceIndex, PackageBasePath, FactoryNode, PackageName, AssetName);
		if (bRefObjectIsMap || !FPackageUtils::IsMapPackageAsset(PackageName))
		{
			//Try to find the package in memory
			Pkg = FindPackage(nullptr, *PackageName);
			if (!Pkg)
			{
				//Try to load the package from disk
				Pkg = LoadPackage(nullptr, *PackageName, LOAD_NoWarn | LOAD_Quiet);
			}
			if (Pkg)
			{
				FollowRedirectorCode();
				ExistingAsset = StaticFindObject(nullptr, Pkg, *AssetName);
				if (ExistingAsset)
				{
					//Do not call factory for a disabled node but ensure a valid custom reference object for other nodes that depend on this one
					FSoftObjectPath Reference;
					if (!FactoryNode->GetCustomReferenceObject(Reference))
					{
						FactoryNode->SetCustomReferenceObject(ExistingAsset);
					}
				}
			}
		}
		//The node is disabled return now
		return;
	}

	UInterchangeManager& InterchangeManager = UInterchangeManager::GetInterchangeManager();

	//If we do a reimport no need to create a package
	if (ObjectToReimport)
	{
		if (bShouldReimportFactoryNode)
		{
			FactoryNode->SetDisplayLabel(ObjectToReimport->GetName());
			FactoryNode->SetAssetName(ObjectToReimport->GetName());
			Pkg = ObjectToReimport->GetPackage();
			PackageName = Pkg->GetPathName();
			AssetName = ObjectToReimport->GetName();
		}
		else
		{
			//Skip this asset, reimport object original factory node does not match this factory node
			return;
		}
		ExistingAsset = ObjectToReimport;
	}
	else
	{
		// Check if the object has not been deleted after an import
		if (FFactoryCommon::GetFactoryNode(AsyncHelper->TaskData.ReimportObject, PackageName, AssetName))
		{
			if (!FactoryNode->ShouldForceNodeReimport())
			{
				//Skip this object, simply assign it to the node, so we can retrieve it later.
				bSkipObjectNoReplace = true;
			}
		}

		Private::InternalGetPackageName(*AsyncHelper, SourceIndex, PackageBasePath, FactoryNode, PackageName, AssetName);
		// We can not create assets that share the name of a map file in the same location
		//Except if we reference a map
		if (!bRefObjectIsMap && FPackageUtils::IsMapPackageAsset(PackageName))
		{
			//Skip this asset
			UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
			Message->DestinationAssetName = AssetName;
			Message->AssetType = FactoryNode->GetObjectClass();
			Message->Text = NSLOCTEXT("FTaskImportObject_GameThread", "MapExistsWithSameName", "You cannot create an asset with this name because there is already a map file with the same name in this folder.");
			return;
		}

		//Try to find the package in memory
		bool bPackageWasCreated = false;
		Pkg = FindPackage(nullptr, *PackageName);
		if (!Pkg)
		{
			//Try to load the package from disk
			Pkg = LoadPackage(nullptr, *PackageName, LOAD_NoWarn | LOAD_Quiet);
			if (!Pkg)
			{
				//Create the package
				Pkg = CreatePackage(*PackageName);
				Pkg->SetPackageFlags(PKG_NewlyCreated);
				bPackageWasCreated = true;
			}
		}

		if (Pkg == nullptr)
		{
			//Skip this asset
			UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
			Message->DestinationAssetName = AssetName;
			Message->AssetType = FactoryNode->GetObjectClass();
			Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "CouldntCreatePackage", "It was not possible to create a package named '{0}'. The asset will not be imported.")
				, FText::FromString(PackageName));
			return;
		}

		if (!bPackageWasCreated)
		{
			FollowRedirectorCode();
		}
		ExistingAsset = StaticFindObject(nullptr, Pkg, *AssetName);
		if (!bSkipObjectNoReplace && ExistingAsset && !AsyncHelper->TaskData.bReplaceExisting)
		{
			const FString AssetFullName = ExistingAsset->GetFullName();
			//If the bReplaceExistingAllDialogAnswer was set do not show again the message dialog, simply reuse the previous answer.
			if (InterchangeManager.GetReplaceExistingAlldialogAnswer().IsSet())
			{
				bSkipObjectNoReplace = !InterchangeManager.GetReplaceExistingAlldialogAnswer().GetValue();
			}
			else
			{
				bSkipObjectNoReplace = true;
				FText OverrideDialogMessage = FText::Format(NSLOCTEXT("InterchangeTaskimportObject", "OverrideAssetMessage", "Are you sure you want to override asset '{0}'?")
					, FText::FromString(AssetFullName));
				if (!GIsAutomationTesting && !FApp::IsUnattended() && !FApp::IsGame())
				{
					EAppReturnType::Type DialogResult = FMessageDialog::Open(EAppMsgType::YesNoYesAllNoAll, OverrideDialogMessage);
					switch (DialogResult)
					{
						case EAppReturnType::YesAll:
							InterchangeManager.SetReplaceExistingAlldialogAnswer(true);
						case EAppReturnType::Yes:
							bSkipObjectNoReplace = false;
							break;
						case EAppReturnType::NoAll:
							InterchangeManager.SetReplaceExistingAlldialogAnswer(false);
						case EAppReturnType::No:
							bSkipObjectNoReplace = true;
					}
				}
			}

			if (bSkipObjectNoReplace)
			{
				//Do not replace existing asset, the option tell us to not override it. Skip this asset with a display message
				UInterchangeResultWarning_Generic* Message = Factory->AddMessage<UInterchangeResultWarning_Generic>();
				Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
				Message->DestinationAssetName = AssetFullName;
				Message->AssetType = FactoryNode->GetObjectClass();
				Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "CouldntReplaceExistingAsset", "The option bReplaceExisting is false, we are not overriding the asset named '{0}'.")
					, FText::FromString(AssetFullName));
			}
		}
	}

	if (ExistingAsset)
	{
		if (!ExistingAsset->GetClass() || !ExistingAsset->GetClass()->IsChildOf(FactoryNode->GetObjectClass()))
		{
			//Skip this asset
			UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
			Message->DestinationAssetName = AssetName;
			Message->AssetType = FactoryNode->GetObjectClass();

			const FText TargetClassNameText = FText::FromString(FactoryNode->GetObjectClass()->GetName());
			const FText ExistingClassNameText = FText::FromString(ExistingAsset->GetClass()->GetName());
			const FText AssetNameText = FText::FromString(AssetName);
			const FText FolderNameText = FText::FromString(FPaths::GetPath(Pkg->GetPathName()));
			Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "AssetVersusFactoryClassWrong_Error", "You cannot create an asset of class '{0}' named '{1}' in '{2}' because there is already an asset of class '{3}' with the same name in that folder.")
				, TargetClassNameText
				, AssetNameText
				, FolderNameText
				, ExistingClassNameText);
			return;
		}

		if (!IsValid(ExistingAsset))
		{
			ExistingAsset->ClearGarbage();
			if (!IsValid(ExistingAsset))
			{
				//Skip this node, the existing asset is not a valid asset and we cannot import over it
				UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
				Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
				Message->DestinationAssetName = AssetName;
				Message->AssetType = FactoryNode->GetObjectClass();
				Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "InvalidExistingAsset", "An invalid asset exists at the same asset location [{0}]. This asset will be skipped.")
					, FText::FromString(AssetName));
				return;
			}
		}

		//Set the factory node reference to the existing object, so we do not need to find the asset in the factory
		FactoryNode->SetCustomReferenceObject(FSoftObjectPath(ExistingAsset));
		if (bSkipObjectNoReplace)
		{
			// #interchange_levelinstance_rework: Need better way to indicate that actors should not be added to level
			FactoryNode->SetSkipNodeImport();
			return;
		}

		if (InterchangeManager.IsObjectBeingImported(ExistingAsset))
		{
			//Skip this node, it is currently being imported by another import task
			UInterchangeResultError_Generic* Message = Factory->AddMessage<UInterchangeResultError_Generic>();
			Message->SourceAssetName = AsyncHelper->SourceDatas[SourceIndex]->GetFilename();
			Message->DestinationAssetName = AssetName;
			Message->AssetType = FactoryNode->GetObjectClass();
			Message->Text = FText::Format(NSLOCTEXT("FTaskImportObject_GameThread", "AssetCollisionBetweenImportTask", "Multiple import tasks are importing the same asset at the same location [{0}]. This asset will be skipped. See the log for more information.")
				, FText::FromString(AssetName));
			return;
		}

		if (IInterface_AsyncCompilation* AssetCompilationInterface = Cast<IInterface_AsyncCompilation>(ExistingAsset))
		{
			//If an asset exist make sure we wait until its compiled before calling the factory
			FAssetCompilingManager::Get().FinishCompilationForObjects({ ExistingAsset });
		}
	}

	//Import Asset describe by the node
	UInterchangeFactoryBase::FImportAssetObjectParams CreateAssetParams;
	CreateAssetParams.AssetName = AssetName;
	CreateAssetParams.AssetNode = FactoryNode;
	CreateAssetParams.Parent = Pkg;
	CreateAssetParams.SourceData = AsyncHelper->SourceDatas[SourceIndex];
	CreateAssetParams.Translator = AsyncHelper->Translators[SourceIndex];
	CreateAssetParams.NodeContainer = NodeContainer;
	CreateAssetParams.ReimportObject = ObjectToReimport;
	//Make sure the asset UObject is created with the correct type on the main thread
	UInterchangeFactoryBase::FImportAssetResult ImportAssetResult = Factory->BeginImportAsset_GameThread(CreateAssetParams);
	if (ImportAssetResult.ImportedObject)
	{
		//If the factory skip the asset, we simply set the node custom reference object
		if (!ImportAssetResult.bIsFactorySkipAsset)
		{
			if (!ImportAssetResult.ImportedObject->HasAnyInternalFlags(EInternalObjectFlags::Async))
			{
				//Since the async flag is not set we must be in the game thread
				ensure(IsInGameThread());
				ImportAssetResult.ImportedObject->SetInternalFlags(EInternalObjectFlags::Async);
			}
			FImportAsyncHelper::FImportedObjectInfo& AssetInfo = AsyncHelper->AddDefaultImportedAssetGetRef(SourceIndex);

			AssetInfo.SetImportedObject(ImportAssetResult.ImportedObject);
			AssetInfo.Factory = Factory;
			AssetInfo.FactoryNode = FactoryNode;
			AssetInfo.bIsReimport = bool(ObjectToReimport != nullptr);
		}
		// Make sure the destination package is loaded
		Pkg->FullyLoad();
		if (!ObjectToReimport)
		{
			AsyncHelper->AddCreatedPackage(PackageName, Pkg);
		}

		//this was verified before calling the factory
		ensure(ImportAssetResult.ImportedObject->GetClass()->IsChildOf(FactoryNode->GetObjectClass()));

		FactoryNode->SetCustomReferenceObject(FSoftObjectPath(ImportAssetResult.ImportedObject));
		//Make sure the node is not skipped when the scope will be terminate
		bCanContinueImport = true;
	}
}

void UE::Interchange::FTaskImportObject_Async::Execute()
{
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(TaskImportObject_Async)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	using namespace UE::Interchange;

	TOptional<FGCScopeGuard> GCScopeGuard;
	if (!IsInGameThread())
	{
		GCScopeGuard.Emplace();
	}

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	//Verify if the task was cancel
	if (AsyncHelper->bCancel || !FactoryNode || !Private::CanImportClass(*AsyncHelper, *FactoryNode, SourceIndex))
	{
		return;
	}

	UInterchangeFactoryBase* Factory = AsyncHelper->GetCreatedFactory(FactoryNode->GetUniqueID());
	if(!Factory)
	{
		return;
	}

	Private::InternalImportObjectStartup(AsyncHelper
		, FactoryNode
		, SourceIndex
		, PackageBasePath
		, [&Factory](UInterchangeFactoryBase::FImportAssetObjectParams& ImportAssetObjectParams)
		{
			LLM_SCOPE_BYNAME(TEXT("Interchange"));
			return Factory->ImportAsset_Async(ImportAssetObjectParams);
		});
}

void UE::Interchange::FTaskImportObjectFinalize_GameThread::Execute()
{
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(TaskImportObjectFinalize_GameThread)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	using namespace UE::Interchange;

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	//Verify if the task was cancel
	if (AsyncHelper->bCancel || !FactoryNode || !Private::CanImportClass(*AsyncHelper, *FactoryNode, SourceIndex))
	{
		return;
	}

	UInterchangeFactoryBase* Factory = AsyncHelper->GetCreatedFactory(FactoryNode->GetUniqueID());
	if(!Factory)
	{
		return;
	}

	check(IsInGameThread());

	UInterchangeFactoryBase::FImportAssetResult ImportAssetResult = Private::InternalImportObjectStartup(AsyncHelper
		, FactoryNode
		, SourceIndex
		, PackageBasePath
		, [&Factory](UInterchangeFactoryBase::FImportAssetObjectParams& ImportAssetObjectParams)
		{
			LLM_SCOPE_BYNAME(TEXT("Interchange"));
			return Factory->EndImportAsset_GameThread(ImportAssetObjectParams);
		});

	if (ImportAssetResult.ImportedObject)
	{
		//If the factory skip the asset, we simply set the node custom reference object
		if (!ImportAssetResult.bIsFactorySkipAsset)
		{
			const FImportAsyncHelper::FImportedObjectInfo* AssetInfoPtr = AsyncHelper->FindImportedAssets(SourceIndex, [&ImportAssetResult](const FImportAsyncHelper::FImportedObjectInfo& CurInfo)
				{
					return CurInfo.GetImportedObject() == ImportAssetResult.ImportedObject;
				});

			if (!AssetInfoPtr)
			{
				FImportAsyncHelper::FImportedObjectInfo& AssetInfo = AsyncHelper->AddDefaultImportedAssetGetRef(SourceIndex);
				AssetInfo.SetImportedObject(ImportAssetResult.ImportedObject);
				AssetInfo.Factory = Factory;
				AssetInfo.FactoryNode = FactoryNode;


				const FString PackagePathName = ImportAssetResult.ImportedObject->GetPathName();
				const FString AssetName = ImportAssetResult.ImportedObject->GetName();
				const UObject* ObjectToReimport = FFactoryCommon::GetObjectToReimport(Factory, AsyncHelper->TaskData.ReimportObject, *FactoryNode, PackagePathName, AssetName);
				AssetInfo.bIsReimport = bool(ObjectToReimport != nullptr);
			}

			// Fill in destination asset and type in any results which have been added previously by a translator or pipeline, now that we have a corresponding factory.
			UInterchangeResultsContainer* Results = AsyncHelper->AssetImportResult->GetResults();
			for (UInterchangeResult* Result : Results->GetResults())
			{
				if (!Result->InterchangeKey.IsEmpty() && (Result->DestinationAssetName.IsEmpty() || Result->AssetType == nullptr))
				{
					TArray<FString> TargetAssets;
					FactoryNode->GetTargetNodeUids(TargetAssets);
					if (TargetAssets.Contains(Result->InterchangeKey))
					{
						Result->DestinationAssetName = ImportAssetResult.ImportedObject->GetPathName();
						Result->AssetType = ImportAssetResult.ImportedObject->GetClass();
					}
				}
			}
		}
		FactoryNode->SetCustomReferenceObject(FSoftObjectPath(ImportAssetResult.ImportedObject));
	}
}
