// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeManager.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Nodes/InterchangeFactoryBaseNode.h"

namespace UE::Interchange
{
	/**
		* This task create UPackage and UObject, Cook::PackageTracker::NotifyUObjectCreated is not thread safe, so we need to create the packages on the main thread
		*/
	class FTaskImportObject_GameThread : public FInterchangeTaskBase
	{
	private:
		FString PackageBasePath;
		int32 SourceIndex;
		TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		UInterchangeFactoryBaseNode* FactoryNode;
		const UClass* FactoryClass;

	public:
		FTaskImportObject_GameThread(const FString& InPackageBasePath, const int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper, UInterchangeFactoryBaseNode* InFactoryNode, const UClass* InFactoryClass)
			: PackageBasePath(InPackageBasePath)
			, SourceIndex(InSourceIndex)
			, WeakAsyncHelper(InAsyncHelper)
			, FactoryNode(InFactoryNode)
			, FactoryClass(InFactoryClass)
		{
			check(FactoryNode);
			check(FactoryClass);
		}

		virtual EInterchangeTaskThread GetTaskThread() const override
		{
			return EInterchangeTaskThread::GameThread;
		}

		virtual void Execute() override;
	};

	class FTaskImportObject_Async : public FInterchangeTaskBase
	{
	private:
		FString PackageBasePath;
		int32 SourceIndex;
		TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		UInterchangeFactoryBaseNode* FactoryNode;

	public:
		FTaskImportObject_Async(const FString& InPackageBasePath, const int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper, UInterchangeFactoryBaseNode* InFactoryNode)
			: PackageBasePath(InPackageBasePath)
			, SourceIndex(InSourceIndex)
			, WeakAsyncHelper(InAsyncHelper)
			, FactoryNode(InFactoryNode)
		{
			check(FactoryNode);
		}

		virtual EInterchangeTaskThread GetTaskThread() const override
		{
			TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
			if (AsyncHelper.IsValid() && AsyncHelper->bRunSynchronous)
			{
				return EInterchangeTaskThread::GameThread;
			}

			return EInterchangeTaskThread::AsyncThread;
		}

		virtual void Execute() override;
	};

	/**
		* This task create UPackage and UObject, Cook::PackageTracker::NotifyUObjectCreated is not thread safe, so we need to create the packages on the main thread
		*/
	class FTaskImportObjectFinalize_GameThread : public FInterchangeTaskBase
	{
	private:
		FString PackageBasePath;
		int32 SourceIndex;
		TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		UInterchangeFactoryBaseNode* FactoryNode;
		const UClass* FactoryClass;

	public:
		FTaskImportObjectFinalize_GameThread(const FString& InPackageBasePath, const int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper, UInterchangeFactoryBaseNode* InFactoryNode)
			: PackageBasePath(InPackageBasePath)
			, SourceIndex(InSourceIndex)
			, WeakAsyncHelper(InAsyncHelper)
			, FactoryNode(InFactoryNode)
		{
			check(FactoryNode);
		}

		virtual EInterchangeTaskThread GetTaskThread() const override
		{
			return EInterchangeTaskThread::GameThread;
		}

		virtual void Execute() override;
	};

}//ns UE::Interchange
