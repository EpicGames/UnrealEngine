// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskParsing.h"

#include "Async/TaskGraphInterfaces.h"
#include "CoreMinimal.h"
#include "GameFramework/Actor.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeFactoryBase.h"
#include "InterchangeManager.h"
#include "InterchangeSourceData.h"
#include "InterchangeTaskCompletion.h"
#include "InterchangeTaskCreateSceneObjects.h"
#include "InterchangeTaskImportObject.h"
#include "InterchangeTaskPipeline.h"
#include "InterchangeTranslatorBase.h"
#include "Misc/Paths.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "PackageUtils/PackageUtils.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/UObjectGlobals.h"
#include "UObject/WeakObjectPtrTemplates.h"
#include "Algo/TopologicalSort.h"

namespace UE::InterchangeTaskParsing::Private
{
	struct FTaskParsingRenameInfo
	{
		UInterchangeFactoryBaseNode* FactoryNode = nullptr;
		int32 SourceIndex = INDEX_NONE;
		FString OriginalName = FString();
		FString NewName = FString();
	};

	struct FTaskData
	{
		FString UniqueID;
		int32 SourceIndex = INDEX_NONE;
		bool bIsSceneNode = false;
		TSet<FString> Dependencies;
		uint64 GraphEventRef = INTERCHANGE_INVALID_TASK_ID;
		TArray<uint64> Prerequisites;
		const UClass* FactoryClass;

		TArray<UInterchangeFactoryBaseNode*, TInlineAllocator<1>> Nodes; // For scenes, we can group multiple nodes into a single task as they are usually very light
	};

	uint64 CreateTasksFromData(
		FTaskData& InTaskData,
		const TSharedPtr<Interchange::FImportAsyncHelper, ESPMode::ThreadSafe>& InAsyncHelper,
		TMap<FString, FTaskParsingRenameInfo>& OutRenameAssets,
		TSet<FString>& OutCreatedTasksAssetNames
	)
	{
		LLM_SCOPE_BYNAME(TEXT("Interchange"));
		TRACE_CPUPROFILER_EVENT_SCOPE(CreateTasksFromData::CreateTasksFromData);

		using namespace UE::Interchange;

		// We expect 1 node per asset task
		if (!ensure(InTaskData.Nodes.Num() == 1))
		{
			for (TObjectPtr<UInterchangeTranslatorBase> Translator : InAsyncHelper->Translators)
			{
				if (!Translator || InAsyncHelper->bCancel)
				{
					continue;
				}

				UInterchangeResultWarning_Generic* WarningResult = NewObject<UInterchangeResultWarning_Generic>(
					GetTransientPackage(),
					UInterchangeResultWarning_Generic::StaticClass()
				);
				WarningResult->Text = NSLOCTEXT("InterchangeTaskParsing", "TranslatorNoAssetImported_MultipleNodes", "Found multiple nodes in an asset task.");
				Translator->AddMessage(WarningResult);
			}

			constexpr uint64 EmptyGraphEvent = 0;
			return EmptyGraphEvent;
		}

		const int32 SourceIndex = InTaskData.SourceIndex;
		const UClass* const FactoryClass = InTaskData.FactoryClass;
		UInterchangeFactoryBaseNode* FactoryNode = InTaskData.Nodes[0];

		if (InTaskData.bIsSceneNode)
		{
			TSharedPtr<UE::Interchange::FTaskCreateSceneObjects_GameThread, ESPMode::ThreadSafe> TaskCreateSceneObjects = MakeShared<UE::Interchange::FTaskCreateSceneObjects_GameThread, ESPMode::ThreadSafe>(InAsyncHelper->ContentBasePath
				, SourceIndex
				, InAsyncHelper
				, InTaskData.Nodes
				, FactoryClass);

			return InAsyncHelper->SceneTasks.Add_GetRef(UE::Interchange::FInterchangeTaskSystem::Get().AddTask(TaskCreateSceneObjects, InTaskData.Prerequisites));
		}
		else
		{
			FString PackageSubPath;
			FactoryNode->GetCustomSubPath(PackageSubPath);

			FString AssetFullPath = FPaths::Combine(InAsyncHelper->ContentBasePath, PackageSubPath, FactoryNode->GetAssetName());

			// Make sure there is no duplicate name full path as we'll need to create a new asset
			bool bCheckForNameClash = FactoryNode->IsEnabled();
			if (bCheckForNameClash)
			{
				uint32 NameIndex = 0;
				FString NewName = AssetFullPath;
				while (OutCreatedTasksAssetNames.Contains(NewName))
				{
					NewName = AssetFullPath + FString::FromInt(++NameIndex);
				}

				// If we renamed anything
				if (NameIndex != 0)
				{
					FTaskParsingRenameInfo& RenameInfo = OutRenameAssets.FindOrAdd(AssetFullPath);
					RenameInfo.FactoryNode = FactoryNode;
					RenameInfo.OriginalName = AssetFullPath;
					RenameInfo.NewName = NewName;
					RenameInfo.SourceIndex = SourceIndex;

					const FString NameIndexString = FString::FromInt(NameIndex);
					FactoryNode->SetDisplayLabel(FactoryNode->GetDisplayLabel() + NameIndexString);
					if (FactoryNode->HasAttribute(UE::Interchange::FBaseNodeStaticData::AssetNameKey()))
					{
						//TextureFactorNodes automatically set up the AssetNames which then are used for asset creation:
						FactoryNode->SetAssetName(FactoryNode->GetDisplayLabel());
					}

					AssetFullPath = NewName;
				}
			}

			if (FactoryNode->IsEnabled() && ensureMsgf(!OutCreatedTasksAssetNames.Contains(AssetFullPath),
				TEXT("Found multiple task data with the same asset name (%s). Only one will be executed."), *AssetFullPath))
			{
				LLM_SCOPE_BYNAME(TEXT("Interchange"));

				// Create factory
				UInterchangeFactoryBase* Factory = NewObject<UInterchangeFactoryBase>(GetTransientPackage(), FactoryClass);
				Factory->SetResultsContainer(InAsyncHelper->AssetImportResult->GetResults());
				InAsyncHelper->AddCreatedFactory(FactoryNode->GetUniqueID(), Factory);

				//If the factory wants to get async payloads it can add the task here so the payloads are in the dependency graph
				TArray<uint64> ImportObjectQueryPayloadsTasksPrerequistes;
				{
					TArray<TSharedPtr<FInterchangeTaskBase>> PayloadTasks;
					UInterchangeFactoryBase::FImportAssetObjectParams ImportAssetObjectParams;
					ImportAssetObjectParams.AssetName = FactoryNode->GetDisplayLabel();
					ImportAssetObjectParams.AssetNode = FactoryNode;
					ImportAssetObjectParams.NodeContainer = InAsyncHelper->BaseNodeContainers[SourceIndex].Get();
					ImportAssetObjectParams.SourceData = InAsyncHelper->SourceDatas[SourceIndex];
					ImportAssetObjectParams.Translator = InAsyncHelper->Translators[SourceIndex];
					bool bAsyncPayloads = !InAsyncHelper->bRunSynchronous;
					if (!ImportAssetObjectParams.Translator || !ImportAssetObjectParams.Translator->IsThreadSafe())
					{
						//Translator not thread safe cannot get payloads asynchronously
						bAsyncPayloads = false;
					}

					if (!InAsyncHelper->bRuntimeOrPIE || FactoryNode->IsRuntimeImportAllowed())
					{
						//Only create payloads if it's allowed.
						Factory->CreatePayloadTasks(ImportAssetObjectParams, bAsyncPayloads, PayloadTasks);
					}

					if (!PayloadTasks.IsEmpty())
					{
						for (TSharedPtr<FInterchangeTaskBase> PayloadTask : PayloadTasks)
						{
							int32 ImportObjectQueryPayloadsTaskIndex = InAsyncHelper->ImportObjectQueryPayloadsTasks.Add(FInterchangeTaskSystem::Get().AddTask(PayloadTask, InTaskData.Prerequisites));
							ImportObjectQueryPayloadsTasksPrerequistes.Add(InAsyncHelper->ImportObjectQueryPayloadsTasks[ImportObjectQueryPayloadsTaskIndex]);
						}
					}
					else
					{
						ImportObjectQueryPayloadsTasksPrerequistes = InTaskData.Prerequisites;
					}
				}

				TArray<uint64> ImportObjectTasksPrerequistes;
				TSharedPtr<FTaskImportObject_GameThread, ESPMode::ThreadSafe> TaskImportObject_GameThread = MakeShared<FTaskImportObject_GameThread, ESPMode::ThreadSafe>(InAsyncHelper->ContentBasePath
					, SourceIndex
					, InAsyncHelper
					, FactoryNode
					, FactoryClass);
				int32 BeginImportObjectTaskIndex = InAsyncHelper->BeginImportObjectTasks.Add(FInterchangeTaskSystem::Get().AddTask(TaskImportObject_GameThread, ImportObjectQueryPayloadsTasksPrerequistes));
				ImportObjectTasksPrerequistes.Add(InAsyncHelper->BeginImportObjectTasks[BeginImportObjectTaskIndex]);

				TSharedPtr<FTaskImportObject_Async, ESPMode::ThreadSafe> TaskImportObject_Async = MakeShared<FTaskImportObject_Async, ESPMode::ThreadSafe>(InAsyncHelper->ContentBasePath
					, SourceIndex
					, InAsyncHelper
					, FactoryNode);
				int32 ImportObjectTaskIndex = InAsyncHelper->ImportObjectTasks.Add(FInterchangeTaskSystem::Get().AddTask(TaskImportObject_Async, ImportObjectTasksPrerequistes));

				TArray<uint64> FinalizeImportObjectTasksPrerequistes;
				FinalizeImportObjectTasksPrerequistes.Add(InAsyncHelper->ImportObjectTasks[ImportObjectTaskIndex]);

				TSharedPtr<FTaskImportObjectFinalize_GameThread, ESPMode::ThreadSafe> TaskImportObjectFinalize_GameThread = MakeShared<FTaskImportObjectFinalize_GameThread, ESPMode::ThreadSafe>(InAsyncHelper->ContentBasePath
					, SourceIndex
					, InAsyncHelper
					, FactoryNode);
				int32 FinalizeCreateTaskIndex = InAsyncHelper->FinalizeImportObjectTasks.Add(FInterchangeTaskSystem::Get().AddTask(TaskImportObjectFinalize_GameThread, FinalizeImportObjectTasksPrerequistes));

				//Only add the name if the factory node is enabled
				if (FactoryNode->IsEnabled())
				{
					OutCreatedTasksAssetNames.Add(AssetFullPath);
				}

				return InAsyncHelper->FinalizeImportObjectTasks[FinalizeCreateTaskIndex];
			}
			else
			{
				constexpr uint64 EmptyGraphEvent = 0;
				return EmptyGraphEvent;
			}
		}
	}
}

void UE::Interchange::FTaskParsing::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskParsing::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(ParsingGraph)
#endif
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	using namespace UE::InterchangeTaskParsing::Private;

	TOptional<FGCScopeGuard> GCScopeGuard;
	if (!IsInGameThread())
	{
		GCScopeGuard.Emplace();
	}

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	//Parse each graph and prepare import task data, we will then be able to create all the task with the correct dependencies
	TArray<FTaskData> TaskDatas;
	TMap<FString, int32> TaskUniqueIdToIndex;

	//Avoid creating asset if the asynchronous import is canceled, just create the completion task
	if (!AsyncHelper->bCancel)
	{
		for (int32 SourceIndex = 0; SourceIndex < AsyncHelper->SourceDatas.Num(); ++SourceIndex)
		{
			if (!AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex))
			{
				continue;
			}

			UInterchangeBaseNodeContainer* BaseNodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
			if (!BaseNodeContainer)
			{
				continue;
			}

			//Translation and pipelines are not executed, compute the children cache for translated and factory nodes
			BaseNodeContainer->ComputeChildrenCache();

			const bool bCanImportSceneNode = AsyncHelper->TaskData.ImportType == EImportType::ImportType_Scene;
			BaseNodeContainer->IterateNodesOfType<UInterchangeFactoryBaseNode>([&](const FString& NodeUID, UInterchangeFactoryBaseNode* FactoryNode)
			{
				UClass* ObjectClass = FactoryNode->GetObjectClass();
				if (ObjectClass != nullptr)
				{
					if (const UClass* RegisteredFactoryClass = InterchangeManager->GetRegisteredFactoryClass(ObjectClass))
					{
						if (UInterchangeFactoryBase* FactoryCDO = Cast<UInterchangeFactoryBase>(RegisteredFactoryClass->GetDefaultObject()))
						{
							const bool bIsSceneNode = (FactoryCDO->GetFactoryAssetType() == EInterchangeFactoryAssetType::None);
							if (bIsSceneNode && !bCanImportSceneNode)
							{
								//nothing we can import from this element
								return;
							}

							FTaskData NodeTaskData;
							NodeTaskData.UniqueID = FactoryNode->GetUniqueID();
							NodeTaskData.SourceIndex = SourceIndex;
							NodeTaskData.bIsSceneNode = bIsSceneNode;
							NodeTaskData.Nodes.Add(FactoryNode);
							NodeTaskData.Dependencies = FactoryNode->GetFactoryDependencies();
							NodeTaskData.FactoryClass = RegisteredFactoryClass;
							int32 TaskDataIndex = TaskDatas.Add(NodeTaskData);
							TaskUniqueIdToIndex.Add(NodeTaskData.UniqueID, TaskDataIndex);
						}
					}
				}
			});
		}
	}

	TMap<FString, FTaskParsingRenameInfo> RenameAssets;
	TSet<FString> CreatedTasksAssetNames; // Tracks for which asset name we have created a task so that we don't have 2 tasks for the same asset name

	auto GetTaskDependencies = [&TaskDatas, &TaskUniqueIdToIndex](int32 Index)
	{
		TArray<int32> TaskDependencies;
		TaskDependencies.Reserve(TaskDatas[Index].Dependencies.Num());
		for(const FString& DependencyTaskID : TaskDatas[Index].Dependencies)
		{
			if (int32* DependencyTaskIndex = TaskUniqueIdToIndex.Find(DependencyTaskID))
			{
				TaskDependencies.Add(*DependencyTaskIndex);
			}
		}
		return TaskDependencies;
	};

	TArray<int32> TasksIndexArray;
	TasksIndexArray.SetNumUninitialized(TaskDatas.Num());
	for (int32 i = 0; i < TaskDatas.Num(); i++)
	{
		TasksIndexArray[i] = i;
	}

	// Sort Tasks According to their dependency in order that their GraphEventRef are created in in the right order
	Algo::TopologicalSort(TasksIndexArray, GetTaskDependencies);

	TArray<uint64> CompletionPrerequistes;
	for (int32 TaskIndex : TasksIndexArray)
	{
		FTaskData& TaskData = TaskDatas[TaskIndex];
		for(const FString& DependencyTaskID : TaskData.Dependencies)
		{
			if (int32* DependencyTaskIndex = TaskUniqueIdToIndex.Find(DependencyTaskID))
			{
				ensure(TaskDatas[*DependencyTaskIndex].GraphEventRef != INTERCHANGE_INVALID_TASK_ID);				
				TaskData.Prerequisites.Add(TaskDatas[*DependencyTaskIndex].GraphEventRef);
			}
		}
		TaskData.GraphEventRef = CreateTasksFromData(TaskData, AsyncHelper, RenameAssets, CreatedTasksAssetNames);
		CompletionPrerequistes.Add(TaskData.GraphEventRef);
	}

	if (!RenameAssets.IsEmpty())
	{
		TMap<TWeakObjectPtr<UInterchangeTranslatorBase>, FString> TranslatorMessageMap;
		for (const TPair<FString, FTaskParsingRenameInfo>& RenameAssetKvp : RenameAssets)
		{
			const FTaskParsingRenameInfo& RenameInfo = RenameAssetKvp.Value;
			FString& Message = TranslatorMessageMap.FindOrAdd(AsyncHelper->Translators[RenameInfo.SourceIndex]);
			Message += FText::Format(NSLOCTEXT("InterchangeTaskParsingDoTask", "RenamedAssetMessagePerAsset", "\n OriginalName:[{0}] NewName:[{1}]")
				, FText::FromString(RenameInfo.OriginalName)
				, FText::FromString(RenameInfo.NewName)).ToString();
		}
		for (const TPair<TWeakObjectPtr<UInterchangeTranslatorBase>, FString>& MessagePerTranslator : TranslatorMessageMap)
		{

			FText ResultText = FText::Format(NSLOCTEXT("InterchangeTaskParsingDoTask", "RenamedAssetsMessageHeader", "Renamed Assets: {0}")
				, FText::FromString(MessagePerTranslator.Value));

			if (GIsAutomationTesting)
			{
				UInterchangeResultDisplay_Generic* DisplayResult = NewObject<UInterchangeResultDisplay_Generic>(GetTransientPackage(), UInterchangeResultDisplay_Generic::StaticClass());
				DisplayResult->Text = ResultText;
				MessagePerTranslator.Key->AddMessage(DisplayResult);
			}
			else
			{
				UInterchangeResultWarning_Generic* WarningResult = NewObject<UInterchangeResultWarning_Generic>(GetTransientPackage(), UInterchangeResultWarning_Generic::StaticClass());
				WarningResult->Text = ResultText;
				MessagePerTranslator.Key->AddMessage(WarningResult);
			}
		}
	}

	//Add an async task for pre completion
	TArray<uint64> PreCompletionPrerequistes;
	TSharedPtr<FTaskPreCompletion_GameThread, ESPMode::ThreadSafe> TaskPreCompletion = MakeShared<FTaskPreCompletion_GameThread, ESPMode::ThreadSafe>(InterchangeManager, WeakAsyncHelper);
	AsyncHelper->PreCompletionTask = FInterchangeTaskSystem::Get().AddTask(TaskPreCompletion, CompletionPrerequistes);
	PreCompletionPrerequistes.Add(AsyncHelper->PreCompletionTask);

	TArray<uint64> AssetCompilationPrerequistes;
	//Start the wait for asset compilation task on a async task
	for (int32 SourceIndex = 0; SourceIndex < AsyncHelper->SourceDatas.Num(); ++SourceIndex)
	{
		TSharedPtr<FTaskWaitAssetCompilation_GameThread, ESPMode::ThreadSafe> TaskWaitAssetCompilation = MakeShared<FTaskWaitAssetCompilation_GameThread, ESPMode::ThreadSafe>(SourceIndex, WeakAsyncHelper);
		int32 WaitAssetCompilationTaskIndex = AsyncHelper->WaitAssetCompilationTasks.Add(FInterchangeTaskSystem::Get().AddTask(TaskWaitAssetCompilation, PreCompletionPrerequistes));
		AssetCompilationPrerequistes.Add(AsyncHelper->WaitAssetCompilationTasks[WaitAssetCompilationTaskIndex]);
	}

	//Start the Post pipeline task
	for (int32 SourceIndex = 0; SourceIndex < AsyncHelper->SourceDatas.Num(); ++SourceIndex)
	{

		TSharedPtr<FTaskPostImport_GameThread, ESPMode::ThreadSafe> TaskPostImport = MakeShared<FTaskPostImport_GameThread, ESPMode::ThreadSafe>(SourceIndex, WeakAsyncHelper);
		int32 PostImportTaskIndex = AsyncHelper->PostImportTasks.Add(FInterchangeTaskSystem::Get().AddTask(TaskPostImport, AssetCompilationPrerequistes));

		//Ensure we run the pipeline in the same order we create the task, since the pipeline modifies the node container, its important that its not processed in parallel, Adding the one we start to the prerequisites
		//is the way to go here
		AssetCompilationPrerequistes.Add(AsyncHelper->PostImportTasks[PostImportTaskIndex]);
	}

	TSharedPtr<FTaskCompletion_GameThread, ESPMode::ThreadSafe> TaskCompletion = MakeShared<FTaskCompletion_GameThread, ESPMode::ThreadSafe>(InterchangeManager, WeakAsyncHelper);
	AsyncHelper->CompletionTask = FInterchangeTaskSystem::Get().AddTask(TaskCompletion, AssetCompilationPrerequistes);
}
