// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeManager.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE
{
	namespace Interchange
	{

		class FTaskParsing : public FInterchangeTaskBase
		{
		private:
			UInterchangeManager* InterchangeManager;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		public:
			FTaskParsing(UInterchangeManager* InInterchangeManager, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: InterchangeManager(InInterchangeManager)
				, WeakAsyncHelper(InAsyncHelper)
			{
				check(InterchangeManager);
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				// TODO: Find a way to run this task on Async Thread without causing issues for GC.
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};

	} //ns Interchange
}//ns UE
