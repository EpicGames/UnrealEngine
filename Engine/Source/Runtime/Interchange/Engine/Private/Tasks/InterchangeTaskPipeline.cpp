// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskPipeline.h"

#include "AssetCompilingManager.h"
#include "Async/Async.h"
#include "CoreMinimal.h"
#include "GenericPlatform/GenericPlatformProcess.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeManager.h"
#include "InterchangePipelineBase.h"
#include "InterchangeFactoryBase.h"
#include "Interfaces/Interface_AsyncCompilation.h"
#include "Materials/MaterialInterface.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Nodes/InterchangeFactoryBaseNode.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"
#include "UObject/WeakObjectPtrTemplates.h"


void UE::Interchange::FTaskPipeline::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskPipeline::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(PipelinePreImport)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));

	TOptional<FGCScopeGuard> GCScopeGuard;
	if (!IsInGameThread())
	{
		GCScopeGuard.Emplace();
	}

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	if (UInterchangePipelineBase* Pipeline = PipelineBase.Get())
	{
		Pipeline->SetResultsContainer(AsyncHelper->AssetImportResult->GetResults());

		for (int32 GraphIndex = 0; GraphIndex < AsyncHelper->BaseNodeContainers.Num(); ++GraphIndex)
		{
			//Verify if the task was cancel
			if (AsyncHelper->bCancel)
			{
				return;
			}

			if (ensure(AsyncHelper->BaseNodeContainers[GraphIndex].IsValid()))
			{
				Pipeline->ScriptedExecutePipeline(AsyncHelper->BaseNodeContainers[GraphIndex].Get(), AsyncHelper->SourceDatas, AsyncHelper->ContentBasePath);
			}
		}
	}
}

void UE::Interchange::FTaskWaitAssetCompilation_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskWaitAssetCompilation_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(WaitAssetCompilation)
#endif
	LLM_SCOPE_BYNAME(TEXT("Interchange"));

#if WITH_EDITOR
	check(IsInGameThread());
	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	if (!ensure(AsyncHelper.IsValid()) || AsyncHelper->bCancel)
	{
		return;
	}

	if (!ensure(AsyncHelper->SourceDatas.IsValidIndex(SourceIndex)))
	{
		return;
	}

	TArray<UObject*> ImportedObjects;

	auto FillImportedObjectsFromSource = [&ImportedObjects](const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedInfos)
		{
			ImportedObjects.Reserve(ImportedObjects.Num() + ImportedInfos.Num());
			for (const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ImportedInfo : ImportedInfos)
			{
				ImportedObjects.Add(ImportedInfo.GetImportedObject());
			}
		};

	AsyncHelper->IterateImportedAssets(SourceIndex, FillImportedObjectsFromSource);
	AsyncHelper->IterateImportedSceneObjects(SourceIndex, FillImportedObjectsFromSource);

	bool bCompilationFinish = true;
	for (int32 ObjectIndex = 0; ObjectIndex < ImportedObjects.Num(); ++ObjectIndex)
	{
		UObject* ImportObject = ImportedObjects[ObjectIndex];
		if (UMaterialInterface* MaterialInterface = Cast<UMaterialInterface>(ImportObject))
		{
			if (MaterialInterface->IsCompiling())
			{
				bCompilationFinish = false;
				break;
			}
		}
		if (IInterface_AsyncCompilation* AssetCompilationInterface = Cast<IInterface_AsyncCompilation>(ImportObject))
		{
			if (AssetCompilationInterface->IsCompiling())
			{
				bCompilationFinish = false;
				break;
			}
		}
	}

	if (!bCompilationFinish)
	{
		//re-enqueue this task so it doesn't execute the subsequent tasks
		SetTaskStatus(EInterchangeTaskStatus::Waiting);
	}
#endif //WITH_EDITOR
}

void UE::Interchange::FTaskPostImport_GameThread::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskPostImport_GameThread::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(PipelinePostImport)
#endif

	LLM_SCOPE_BYNAME(TEXT("Interchange"));
	check(IsInGameThread());

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	if (!ensure(AsyncHelper.IsValid()) || AsyncHelper->bCancel)
	{
		return;
	}

	if (!ensure(AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex)))
	{
		return;
	}

	//Get the Container from the async helper
	UInterchangeBaseNodeContainer* NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
	if (!ensure(NodeContainer))
	{
		return;
	}

#if WITH_EDITOR
	auto CallPostEditChangeForAsset = [](const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedInfos)
		{
			for (const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedInfos)
			{
				UObject* ImportedObject = ObjectInfo.GetImportedObject();
				if (ImportedObject && !ObjectInfo.bPostEditChangeCalled)
				{
					ObjectInfo.bPostEditChangeCalled = true;
					ImportedObject->PostEditChange();
				}
			}
		};

	AsyncHelper->IterateImportedAssets(SourceIndex, CallPostEditChangeForAsset);
	AsyncHelper->IterateImportedSceneObjects(SourceIndex, CallPostEditChangeForAsset);

#endif //WITH_EDITOR

	TArray<FString> NodeUniqueIDs;
	TArray<UObject*> ImportedObjects;
	TArray<bool> IsAssetsReimported;

	auto FillImportedObjectsFromSource =
		[&AsyncHelper, &NodeUniqueIDs, &ImportedObjects, &IsAssetsReimported, this](const TArray<UE::Interchange::FImportAsyncHelper::FImportedObjectInfo>& ImportedInfos)
		{
			NodeUniqueIDs.Reserve(NodeUniqueIDs.Num() + ImportedInfos.Num());
			ImportedObjects.Reserve(ImportedObjects.Num() + ImportedInfos.Num());
			IsAssetsReimported.Reserve(IsAssetsReimported.Num() + ImportedInfos.Num());

			const bool bCallPostImportGameThreadCallback = ensure(AsyncHelper->SourceDatas.IsValidIndex(SourceIndex));

			//Call the pipeline for each asset created by this import
			UInterchangeFactoryBase::FSetupObjectParams Arguments;
			Arguments.SourceData = bCallPostImportGameThreadCallback ? AsyncHelper->SourceDatas[SourceIndex] : nullptr;
			Arguments.NodeContainer = AsyncHelper->BaseNodeContainers[SourceIndex].Get();
			Arguments.Pipelines = AsyncHelper->Pipelines;
			Arguments.OriginalPipelines = AsyncHelper->OriginalPipelines;
			Arguments.Translator = AsyncHelper->Translators[SourceIndex];

			for (const UE::Interchange::FImportAsyncHelper::FImportedObjectInfo& ObjectInfo : ImportedInfos)
			{
				UObject* ImportedObject = ObjectInfo.GetImportedObject();

				//In case Some factory code cannot run outside of the main thread we offer this callback to finish the work after asset build is finish.
				if (bCallPostImportGameThreadCallback && ObjectInfo.Factory)
				{
					Arguments.ImportedObject = ImportedObject;
					Arguments.FactoryNode = ObjectInfo.FactoryNode;
					Arguments.NodeUniqueID = ObjectInfo.FactoryNode ? ObjectInfo.FactoryNode->GetUniqueID() : FString();
					Arguments.bIsReimport = ObjectInfo.bIsReimport;
					ObjectInfo.Factory->FinalizeObject_GameThread(Arguments);
				}

				if (!IsValid(ImportedObject))
				{
					continue;
				}

				NodeUniqueIDs.Add(ObjectInfo.FactoryNode->GetUniqueID());
				ImportedObjects.Add(ObjectInfo.GetImportedObject());
				IsAssetsReimported.Add(ObjectInfo.bIsReimport);
			}
		};

	AsyncHelper->IterateImportedAssets(SourceIndex, FillImportedObjectsFromSource);
	AsyncHelper->IterateImportedSceneObjects(SourceIndex, FillImportedObjectsFromSource);

	if (!ensure(NodeUniqueIDs.Num() == ImportedObjects.Num()))
	{
		//We do not execute the script if we cannot give proper parameter
		return;
	}

	// Execute post-import script on all imported objects for all pipelines
	for (int32 PipelineIndex = 0; PipelineIndex < AsyncHelper->Pipelines.Num(); ++PipelineIndex)
	{
		for (int32 ObjectIndex = 0; ObjectIndex < ImportedObjects.Num(); ++ObjectIndex)
		{
			UInterchangePipelineBase* Pipeline = AsyncHelper->Pipelines[PipelineIndex];
			Pipeline->ScriptedExecutePostImportPipeline(NodeContainer, NodeUniqueIDs[ObjectIndex], ImportedObjects[ObjectIndex], IsAssetsReimported[ObjectIndex]);
		}
	}
}
