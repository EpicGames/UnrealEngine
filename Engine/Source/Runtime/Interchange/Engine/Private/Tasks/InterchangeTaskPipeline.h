// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeManager.h"
#include "InterchangePipelineBase.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE
{
	namespace Interchange
	{

		class FTaskPipeline : public FInterchangeTaskBase
		{
		private:
			TWeakObjectPtr<UInterchangePipelineBase> PipelineBase;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;
		public:
			FTaskPipeline(TWeakObjectPtr<UInterchangePipelineBase> InPipelineBase, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: PipelineBase(InPipelineBase)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
				if (AsyncHelper.IsValid() && AsyncHelper->bRunSynchronous)
				{
					return EInterchangeTaskThread::GameThread;
				}

				if (!ensure(PipelineBase.IsValid()))
				{
					return EInterchangeTaskThread::GameThread;
				}

				//Scripted (python) cannot run outside of the game thread, it will lock forever if we do this
				if (PipelineBase.Get()->IsScripted())
				{
					return EInterchangeTaskThread::GameThread;
				}

				return PipelineBase.Get()->CanExecuteOnAnyThread(EInterchangePipelineTask::PostTranslator) ? EInterchangeTaskThread::AsyncThread : EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};

		//We want to be sure any asset compilation is finish before calling subsequents tasks
		class FTaskWaitAssetCompilation_GameThread : public FInterchangeTaskBase
		{
		private:
			int32 SourceIndex;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;

		public:
			FTaskWaitAssetCompilation_GameThread(int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: SourceIndex(InSourceIndex)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				//This task is re-enqueue and don't stall the game thread
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};

		class FTaskPostImport_GameThread : public FInterchangeTaskBase
		{
		private:
			int32 SourceIndex;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;

		public:
			FTaskPostImport_GameThread(int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: SourceIndex(InSourceIndex)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				return EInterchangeTaskThread::GameThread;
			}

			virtual void Execute() override;
		};
	} //ns Interchange
}//ns UE
