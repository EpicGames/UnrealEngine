// Copyright Epic Games, Inc. All Rights Reserved.
#include "InterchangeTaskTranslator.h"

#include "CoreMinimal.h"
#include "InterchangeEngineLogPrivate.h"
#include "InterchangeManager.h"
#include "InterchangeSourceData.h"
#include "InterchangeTaskSystem.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Stats/Stats.h"
#include "Templates/SharedPointer.h"
#include "UObject/GarbageCollection.h"
#include "UObject/WeakObjectPtrTemplates.h"

void UE::Interchange::FTaskTranslator::Execute()
{
	TRACE_CPUPROFILER_EVENT_SCOPE(UE::Interchange::FTaskTranslator::DoTask)
#if INTERCHANGE_TRACE_ASYNCHRONOUS_TASK_ENABLED
	INTERCHANGE_TRACE_ASYNCHRONOUS_TASK(TaskTranslator)
#endif

	
	TOptional<FGCScopeGuard> GCScopeGuard;
	if (!IsInGameThread())
	{
		GCScopeGuard.Emplace();
	}

	TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
	check(AsyncHelper.IsValid());

	UInterchangeTranslatorBase* Translator = AsyncHelper->Translators.IsValidIndex(SourceIndex) ? AsyncHelper->Translators[SourceIndex] : nullptr;
	if (!Translator)
	{
		return;
	}

	check(AsyncHelper->AssetImportResult->GetResults() != nullptr);
	check(Translator->SourceData != nullptr);
	Translator->SetResultsContainer(AsyncHelper->AssetImportResult->GetResults());

	if (!AsyncHelper->BaseNodeContainers.IsValidIndex(SourceIndex) || !AsyncHelper->BaseNodeContainers[SourceIndex].IsValid())
	{
		return;
	}

	//Verify if the task was cancel
	if (AsyncHelper->bCancel)
	{
		return;
	}

	//Translate the source data
	UInterchangeBaseNodeContainer& BaseNodeContainer = *(AsyncHelper->BaseNodeContainers[SourceIndex].Get());
	Translator->Translate(BaseNodeContainer);

	//Note: At this point All childrenCache should already be set, as SetNodeParentUid already handles the ChildrenCache.
	//		(and FTaskParsing ComputesChildrenCache for any prior existing data/node set.)
}
