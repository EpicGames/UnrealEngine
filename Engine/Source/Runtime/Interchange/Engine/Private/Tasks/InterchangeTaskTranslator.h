// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "InterchangeManager.h"
#include "InterchangeTaskSystem.h"
#include "Stats/Stats.h"
#include "UObject/WeakObjectPtrTemplates.h"

namespace UE
{
	namespace Interchange
	{
		class FTaskTranslator : public FInterchangeTaskBase
		{
		private:
			int32 SourceIndex = INDEX_NONE;
			TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> WeakAsyncHelper;

		public:
			FTaskTranslator(int32 InSourceIndex, TWeakPtr<FImportAsyncHelper, ESPMode::ThreadSafe> InAsyncHelper)
				: SourceIndex(InSourceIndex)
				, WeakAsyncHelper(InAsyncHelper)
			{
			}

			virtual EInterchangeTaskThread GetTaskThread() const override
			{
				TSharedPtr<FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper = WeakAsyncHelper.Pin();
				if (AsyncHelper.IsValid() && AsyncHelper->bRunSynchronous)
				{
					return EInterchangeTaskThread::GameThread;
				}

				return EInterchangeTaskThread::AsyncThread;
			}

			virtual void Execute();
		};
	} //ns Interchange
} //ns UE
