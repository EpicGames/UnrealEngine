// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/WeakObjectPtrTemplates.h"

#include "Containers/Array.h"
#include "Containers/Map.h"
#include "Containers/UnrealString.h"

#include "HAL/CriticalSection.h"

#include "Misc/Guid.h"
#include "Misc/Optional.h"
#include "Misc/NotNull.h"
#include "AnalyticsEventAttribute.h"

#include "InterchangeAnalyticsHandlerBase.generated.h"

#define UE_API INTERCHANGEENGINE_API

class UInterchangePipelineBase;

class UInterchangeSourceData;
class UInterchangeSourceNode;

class UInterchangeResultsContainer;

/**
 * If you add or remove from this enum, make sure to update the corresponding strings
 * in UInterchangeAnalyticsHandlerBase::GetDefaultAnalyticsCategoryName.
 */
enum class EInterchangeAnalyticsDefaultCategory : uint8
{
	NotSpecified = 0,
	Models,
	Scenes,
	Materials,
	Textures,
	Sounds,
	Meshes,
	StaticMeshes,
	SkeletalMeshesAndAnimations,
	GeometryCaches,
	Grooms,
	Physics,
	InterchangeAnalyticsCategory_COUNT
};

struct FInterchangeImportResultAnalyticsInfo
{
	bool bCancel;
	int32 UniqueId;

	TNotNull<const UInterchangeSourceData*> PrimarySourceData;
	const UInterchangeSourceNode* PrimarySourceNode;

	TOptional<const UInterchangeResultsContainer*> AssetImportResultsContainer;
	TOptional<const UInterchangeResultsContainer*> SceneImportResultsContainer;

	TMap<int32, TArray<TObjectPtr<UObject>>> ImportedAssetsPerSourceIndex;
	TMap<int32, TArray<TObjectPtr<UObject>>> ImportedSceneObjectsPerSourceIndex;
};

/**
 * This is the base class implementation for the Interchange Analytics. This will NOT collect any analytics BY DESIGN.
 * DO NOT MAKE CHANGES TO THE IMPLEMENTATIONS OF THE FUNCTIONS FOR THIS CLASS. See UInterchangeAnalyticsHandlerDefault instead 
 * for the default implementation.
 * 
 * Interchange Manager exposes API to set the TSoftClassPtr for a class to use to collect Analytics.
 * By default InterchangeAnalyticsModule registers UInterchangeAnalyticsHandlerDefault as the Analytics Handler Class.
 */
UCLASS(BlueprintType, Blueprintable, Abstract, MinimalAPI)
class UInterchangeAnalyticsHandlerBase : public UObject
{
    GENERATED_BODY()

public:
	UE_API static FAnalyticsEventAttribute CreateAnalyticsIDAttribute(FGuid AnalyticsID);

	UE_API static FString GetDefaultAnalyticsCategoryName(EInterchangeAnalyticsDefaultCategory DefaultCategory);

protected:
	UE_API static FString GetPipelineAnalyticsEventIdentifier();

	UE_API static FString GetImportResultEventIdentifier();

public:
	/**
	 * Can only be set once per instance of the Analytics Handler.
	 */
	UE_API void SetGUID(const FGuid& InGUID);

	/**
	 * Adds an array of attributes to the event identifier in a thread safe manner.
	 */
    UE_API void Append(const FString& Identifier, const TArray<FAnalyticsEventAttribute>& ToAdd);

	/**
	 * Add the analytics attribute to an event identifier in a thread safe manner.
	 */
	UE_API void Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry);

	/**
	 * Clears accumulated attributes for an event identifier. Useful in Translator Analytic Events context.
	 */
	UE_API void Clear(const FString& Identifier);

	/**
	 * Send Accumulated Analytics.
	 */
    UE_API void Flush();

	/**
	 * Send Pipelines Analytics.
	 */
	virtual void Send(const TArray<UInterchangePipelineBase*>& Pipelines, const int32 AsyncHelperUniqueId) PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Send, )
	
	/**
	 * Send Import Result Analytics.
	 */
	virtual void Send(const FInterchangeImportResultAnalyticsInfo& ImportResultAnalyticsInfo) PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Send, )
	
	/**
	* Creates an Analytics Event Attribute for the current instance of the analytics helper.
	* Helpful to be used on external analytics events not sent by AnalyticsHelper.
	*/
    UE_API virtual FAnalyticsEventAttribute GetAnalyticsIDAttribute() const;

protected:
	virtual void Execute_Append(const FString& Identifier, const TArray<FAnalyticsEventAttribute>& ToAdd) PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Execute_Append, )
		
	virtual void Execute_Add(const FString& Identifier, const FAnalyticsEventAttribute& Entry) PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Execute_Add, )

	virtual void Execute_Clear(const FString& Identifier) PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Execute_Clear, )
	
	virtual void Execute_Flush() PURE_VIRTUAL(UInterchangeAnalyticsHandlerBase::Execute_Flush, )

private:
	FCriticalSection CriticalSection;
    FGuid AnalyticsID;
};


#undef UE_API