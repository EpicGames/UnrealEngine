// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "EditorFramework/AssetImportData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNode.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "UObject/AssetRegistryTagsContext.h"
#include "UObject/Class.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeAssetImportData.generated.h"

class UInterchangePipelineBase;

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeAssetImportData : public UAssetImportData
{
	GENERATED_BODY()
public:
	// Begin UObject interface
	virtual void PostLoad() override;
	virtual void Serialize(FArchive& Ar) override;
	// End UObject interface

#if WITH_EDITOR
	INTERCHANGEENGINE_API  virtual bool ConvertAssetImportDataToNewOwner(UObject* Owner) override;
#endif

	/**
	 * Return the first filename stored in this data. The resulting filename will be absolute (that is, not relative to the asset).
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	FString ScriptGetFirstFilename() const
	{
		//TODO make sure this work at runtime
#if WITH_EDITORONLY_DATA
		return GetFirstFilename();
#else
		return FString();
#endif
	}

	/**
	 * Extract all the (resolved) filenames.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	TArray<FString> ScriptExtractFilenames() const
	{
		//TODO make sure this work at runtime
#if WITH_EDITORONLY_DATA
		return ExtractFilenames();
#else
		return TArray<FString>();
#endif
	}

	/**
	 * Extract all the filename labels.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	TArray<FString> ScriptExtractDisplayLabels() const
	{
		TArray<FString> TempDisplayLabels;
		//TODO make sure this work at runtime
#if WITH_EDITORONLY_DATA
		ExtractDisplayLabels(TempDisplayLabels);
#endif
		return TempDisplayLabels;
	}

#if WITH_EDITORONLY_DATA
#if WITH_EDITOR
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	virtual void AppendAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) override
	{
		PRAGMA_DISABLE_DEPRECATION_WARNINGS;
		Super::AppendAssetRegistryTags(OutTags);
		PRAGMA_ENABLE_DEPRECATION_WARNINGS;
	}

	/**
	 * This function adds tags to the asset registry.
	 */
	virtual void AppendAssetRegistryTags(FAssetRegistryTagsContext Context) override
	{
		if(const UInterchangeBaseNodeContainer* NodeContainerTmp = GetNodeContainer())
		{
			if (NodeContainerTmp->IsNodeUidValid(NodeUniqueID))
			{
				if (const UInterchangeBaseNode* Node = GetStoredNode(NodeUniqueID))
				{
					PRAGMA_DISABLE_DEPRECATION_WARNINGS;
					TArray<UObject::FAssetRegistryTag> DeprecatedFunctionTags;
					Node->AppendAssetRegistryTags(DeprecatedFunctionTags);
					for (UObject::FAssetRegistryTag& Tag : DeprecatedFunctionTags)
					{
						Context.AddTag(MoveTemp(Tag));
					}
					PRAGMA_ENABLE_DEPRECATION_WARNINGS;
					Node->AppendAssetRegistryTags(Context);
				}
			}
		}
		Super::AppendAssetRegistryTags(Context);
	}
#endif
#endif

	/** On a level import, set to the UInterchangeSceneImportAsset created during the import. */
	UPROPERTY(EditAnywhere, Category = "Interchange | AssetImportData")
	FSoftObjectPath SceneImportAsset;

	/** Returns a pointer to the UInterchangeAssetImportData referred to by the input object, if applicable. */
	static UInterchangeAssetImportData* GetFromObject(UObject* Object)
	{
		if (Object)
		{
			TArray<UObject*> SubObjects;
			GetObjectsWithOuter(Object, SubObjects);
			for (UObject* SubObject : SubObjects)
			{
				if (UInterchangeAssetImportData* AssetImportData = Cast<UInterchangeAssetImportData>(SubObject))
				{
					return AssetImportData;
				}
			}
		}

		return nullptr;
	}

	/** The Node UID passed to the factory that existed in the graph that was used to create this asset. */
	UPROPERTY(VisibleAnywhere, Category = "Interchange | AssetImportData")
	FString NodeUniqueID;


	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API UInterchangeBaseNodeContainer* GetNodeContainer() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API void SetNodeContainer(UInterchangeBaseNodeContainer* InNodeContainer) const;

	/**
	* Returns Array of non-null pipelines
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API TArray<UObject*> GetPipelines() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API int32 GetNumberOfPipelines() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API void SetPipelines(const TArray<UObject*>& InPipelines);


	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API const UInterchangeBaseNode* GetStoredNode(const FString& InNodeUniqueId) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API UInterchangeFactoryBaseNode* GetStoredFactoryNode(const FString& InNodeUniqueId) const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API const UInterchangeTranslatorSettings* GetTranslatorSettings() const;

	UFUNCTION(BlueprintCallable, Category = "Interchange | AssetImportData")
	INTERCHANGEENGINE_API void SetTranslatorSettings(UInterchangeTranslatorSettings* TranslatorSettings) const;

	/** Won't overwrite an existing backup. Backup has to be cleared before a new one can be stored. */
	INTERCHANGEENGINE_API void BackupSourceData() const;
	INTERCHANGEENGINE_API void ClearBackupSourceData() const;
	INTERCHANGEENGINE_API void ReinstateBackupSourceData();

private:
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use GetNodeContainer/SetNodeContainer instead."))
	TObjectPtr<UInterchangeBaseNodeContainer> NodeContainer_DEPRECATED;
	UPROPERTY(meta = (DeprecatedProperty, DeprecationMessage = "Use GetPipelines/SetPipelines instead."))
	TArray<TObjectPtr<UObject>> Pipelines_DEPRECATED;

	UPROPERTY(Transient)
	mutable TObjectPtr<UInterchangeBaseNodeContainer> TransientNodeContainer;

	UPROPERTY(Transient)
	mutable TArray<TObjectPtr<UObject>> TransientPipelines;

	UPROPERTY(Transient)
	mutable TObjectPtr<UInterchangeTranslatorSettings> TransientTranslatorSettings;

	void ProcessContainerCache() const;
	void ProcessPipelinesCache() const;
	void ProcessDeprecatedData() const;
	void ProcessTranslatorCache() const;
	mutable TArray64<uint8> CachedNodeContainer;
	mutable TArray<TPair<FString, FString>> CachedPipelines; //Class, Data(serialized JSON) pair
	mutable FCustomVersionContainer CachedCustomVersions; // Custom versions of the archive we deserialized from, to help us deserialize our cached data
	mutable TPair<FString, FString> CachedTranslatorSettings;

#if WITH_EDITORONLY_DATA
	/** Source file data describing the files that were used to import this asset. Temporary, Primary usage is for Re-import cancellations. */
	mutable FAssetImportInfo SourceDataBackup;
#endif
	// #todo_interchange: TODO: Move to as a IntercnageManager utility method
	static TMap<FString, UClass*> GetPipelineClassPerName();
};

/**
 * Base class to create an asset import data converter.
 */
UCLASS(Abstract, MinimalAPI)
class UInterchangeAssetImportDataConverterBase : public UObject
{
	GENERATED_BODY()
public:

	/* Converter must return which class they can convert to and from UInterchangeAssetImportData. */
	virtual bool CanConvertClass(const UClass* SourceClass, const UClass* DestinationClass) const { return false; }

	/**
	 * Convert the asset import data from the one that is in the Object to
	 * one that supports the target extension (for example, legacy FBX to Interchange or vice-versa)
	 * The function should return true only if it has converted the asset import data, or false otherwise.
	 * 
	 * The system will call all objects that derive from this class until one converts the data.
	 * 
	 * @Param Asset - Represent an asset that have a member AssetImportData (StaticMesh, SkeletalMesh, Texture....)
	 */
	virtual bool ConvertImportData(UObject* Asset, const FString& TargetExtension) const
	{
		return false;
	}

	/**
	 * Convert the asset import data from the source to the destination.
	 * The function should return true only if it has convert the asset import data, false otherwise.
	 *
	 * The system will call all object deriving from this class until one convert the data.
	 */
	virtual bool ConvertImportData(const UObject* SourceImportData, const UClass* DestinationClass, UObject** DestinationImportData) const
	{
		return false;
	}
};
