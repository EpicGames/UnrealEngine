// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/Blueprint.h"
#include "InterchangePipelineBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


#include "InterchangeBlueprintPipelineBase.generated.h"

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeBlueprintPipelineBase : public UBlueprint
{
	GENERATED_BODY()

public:
	UInterchangeBlueprintPipelineBase()
	{
		ParentClass = UInterchangePipelineBase::StaticClass();
		//We must make sure the GeneratedClass is generated after the blueprint is loaded
		bRecompileOnLoad = true;
	}
};
