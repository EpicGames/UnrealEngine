// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeEditorUtilitiesBase.generated.h"

UCLASS(MinimalAPI)
class UInterchangeEditorUtilitiesBase : public UObject
{
	GENERATED_BODY()

public:
	virtual bool SaveAsset(UObject* Asset) const
	{
		return false;
	}

	virtual bool IsRuntimeOrPIE() const
	{
#if WITH_EDITOR
		return false;
#else
		return true;
#endif
	}

	virtual bool ClearEditorSelection() const
	{
		return false;
	}
};
