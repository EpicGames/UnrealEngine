// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Templates/SharedPointer.h"

namespace UE
{
	namespace Interchange
	{
		class FImportResult;

		using FAssetImportResultRef = TSharedRef< FImportResult, ESPMode::ThreadSafe >;
		using FSceneImportResultRef = TSharedRef< FImportResult, ESPMode::ThreadSafe >;
	}
}
