// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Containers/UnrealString.h"
#include "InterchangeTranslatorBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeFilePickerBase.generated.h"

USTRUCT(BlueprintType)
struct FInterchangeFilePickerParameters
{
	GENERATED_BODY()

	//If true, the user will be able to select multiple files.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | File Picker")
	bool bAllowMultipleFiles = false;

	//If not empty, this will override the default title.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | File Picker")
	FText Title = FText();

	//Set the default open path that the dialog will show to the user.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | File Picker")
	FString DefaultPath;

	//If true, the user will be able to select any unreal editor factory + interchange file types.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | File Picker")
	bool bShowAllFactoriesExtension = false;

	//Add some extension to the picker. Format text item that way TEXT("fbx;Filmbox")
	UPROPERTY(EditAnywhere, Category = "Interchange | File Picker")
	TArray<FString> ExtraFormats;
};

UCLASS(Abstract, BlueprintType, Blueprintable, MinimalAPI)
class UInterchangeFilePickerBase : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement FilePickerForTranslatorAssetType().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Utilities")
	INTERCHANGEENGINE_API bool ScriptedFilePickerForTranslatorAssetType(const EInterchangeTranslatorAssetType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual FilePickerForTranslatorAssetType(). */
	bool ScriptedFilePickerForTranslatorAssetType_Implementation(const EInterchangeTranslatorAssetType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames)
	{
		//By default we call the virtual function FilePickerForTranslatorAssetType
		return FilePickerForTranslatorAssetType(TranslatorAssetType, Parameters, OutFilenames);
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement FilePickerForTranslatorType().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Utilities")
	INTERCHANGEENGINE_API bool ScriptedFilePickerForTranslatorType(const EInterchangeTranslatorType TranslatorType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual FilePickerForTranslatorType(). */
	bool ScriptedFilePickerForTranslatorType_Implementation(const EInterchangeTranslatorType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames)
	{
		//By default we call the virtual function FilePickerForTranslatorType
		return FilePickerForTranslatorType(TranslatorAssetType, Parameters, OutFilenames);
	}


protected:

	/**
	 * This function must set OutFilename with a valid file that can be import to create the specified TranslatorAssetType.
	 * We expect the user to see a file picker dialog when the editor is available.
	 * 
	 * @Return - true if OutFilename was set with a valid file path, or false otherwise.
	 */
	virtual bool FilePickerForTranslatorAssetType(const EInterchangeTranslatorAssetType TranslatorAssetType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames) { return false; }

	/**
	 * This function must set OutFilename with a valid file that can be import to create the specified TranslatorType.
	 * We expect the user to see a file picker dialog when the editor is available.
	 *
	 * @Return - true if OutFilename was set with a valid file path, or false otherwise.
	 */
	virtual bool FilePickerForTranslatorType(const EInterchangeTranslatorType TranslatorType, const FInterchangeFilePickerParameters& Parameters, TArray<FString>& OutFilenames) { return false; }

};
