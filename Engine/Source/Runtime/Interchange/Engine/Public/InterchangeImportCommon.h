// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreMinimal.h"
#include "CoreTypes.h"
#include "EditorFramework/AssetImportData.h"

#define UE_API INTERCHANGEENGINE_API

class UAssetImportData;
class UInterchangeAssetImportData;
class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeFactoryBase;
class UInterchangeFactoryBaseNode;
class UInterchangePipelineBase;
class UInterchangeSourceData;
class UInterchangeTranslatorBase;
class UObject;
template <typename FuncType> class TFunctionRef;

namespace UE::Interchange
{
	/**
	* All the code we cannot put in the base factory class because of dependencies (like Engine dep)
	* Will be available here.
	*/
	class FFactoryCommon
	{
	public:

		struct FUpdateImportAssetDataParameters
		{
			UObject* AssetImportDataOuter = nullptr;
			UAssetImportData* AssetImportData = nullptr;
			const UInterchangeSourceData* SourceData = nullptr;
			FString NodeUniqueID;
			UInterchangeBaseNodeContainer* NodeContainer;
			const TArray<UObject*> Pipelines;
			const UInterchangeTranslatorBase* Translator;

			UE_API FUpdateImportAssetDataParameters(UObject* InAssetImportDataOuter
																, UAssetImportData* InAssetImportData
																, const UInterchangeSourceData* InSourceData
																, FString InNodeUniqueID
																, UInterchangeBaseNodeContainer* InNodeContainer
																, const TArray<UObject*>& InPipelines
																, const UInterchangeTranslatorBase* InTranslator);
		};

		/**
		 * Update the AssetImportData source file of the specified asset in the parameters. Also update the node container and the node unique id.
		 * If the AssetImportData is null it will create one. If the AssetImportData is not an UInterchangeAssetImportData it will create a new one.
		 * @return The source data that should be stored on the asset or nullptr if a parameter is invalid
		 */
		static UE_API UAssetImportData* UpdateImportAssetData(FUpdateImportAssetDataParameters& Parameters);

		/**
		 * Update the AssetImportData of the specified asset in the parameters. Also update the node container and the node unique id.
		 * If the AssetImportData is null it will create one. If the AssetImportData is not an UInterchangeAssetImportData it will create a new one.
		 * The file source update is done by calling the function parameter CustomFileSourceUpdate, so its the client responsability to properly update the file source.
		 * @return The source data that should be stored on the asset or nullptr if a parameter is invalid
		 */
		static UE_API UAssetImportData* UpdateImportAssetData(FUpdateImportAssetDataParameters& Parameters, TFunctionRef<void(UInterchangeAssetImportData*)> CustomFileSourceUpdate);

#if WITH_EDITORONLY_DATA
		struct FSetImportAssetDataParameters : public FUpdateImportAssetDataParameters
		{
			// Allow the factory to provide is own list of source files.
			TArray<FAssetImportInfo::FSourceFile> SourceFiles;

			UE_API FSetImportAssetDataParameters(UObject* InAssetImportDataOuter
				, UAssetImportData* InAssetImportData
				, const UInterchangeSourceData* InSourceData
				, FString InNodeUniqueID
				, UInterchangeBaseNodeContainer* InNodeContainer
				, const TArray<UObject*>& InPipelines
				, const UInterchangeTranslatorBase* InTranslator);
		};

		/**
		 * Set the AssetImportData source file of the specified asset in the parameters. Also update the node container and the node unique id.
		 * If the AssetImportData is null it will create one. If the AssetImportData is not an UInterchangeAssetImportData it will create a new one.
		 * @return The source data that should be stored on the asset or nullptr if a parameter is invalid
		 */
		static UE_API UAssetImportData* SetImportAssetData(FSetImportAssetDataParameters& Parameters);

		/**
		 * Fills the OutSourceFilenames array with the list of source files contained in the asset source data.
		 * Returns true if the operation was successful.
		 */
		static UE_API bool GetSourceFilenames(const UAssetImportData* AssetImportData, TArray<FString>& OutSourceFilenames);

		/**
		 * Backups the SourceData. Primary usage for re-instating SourceData on Re-Import cancellation.
		 */
		static UE_API void BackupSourceData(const UAssetImportData* AssetImportData);

		/**
		 * Reinstates the backedup SourceData. Primary usage for re-instating SourceData on Re-Import cancellation.
		 */
		static UE_API void ReinstateSourceData(UAssetImportData* AssetImportData);

		/**
		 * Clears the backedup SourceData. Primary usage for re-instating SourceData on Re-Import cancellation.
		 */
		static UE_API void ClearBackupSourceData(const UAssetImportData* AssetImportData);

		/**
		 * Sets the SourceFileName value at the specified index.
		 */
		static UE_API bool SetSourceFilename(UAssetImportData* AssetImportData, const FString& SourceFilename, int32 SourceIndex, const FString& SourceLabel = FString());

		/**
		 * Set the object's reimport source at the specified index value.
		 */
		static UE_API bool SetReimportSourceIndex(const UObject* Object, UAssetImportData* AssetImportData, int32 SourceIndex);

#endif // WITH_EDITORONLY_DATA

		/**
		 * Apply the current strategy to the PipelineAssetNode
		 */
		static UE_API void ApplyReimportStrategyToAsset(UObject* Asset
											, const UInterchangeFactoryBaseNode* PreviousAssetNode
											, const UInterchangeFactoryBaseNode* CurrentAssetNode
											, UInterchangeFactoryBaseNode* PipelineAssetNode);

		/**
		 * If the ReimportObject is a UInterchangeSceneImportAsset, returns the UObject which asset path
		 * is //PackageName.AssetName[:SubPathString].
		 * Returns ReimportObject otherwise.
		 * @param ReimportObject: Object that a reimport has been initialized from
		 * @param FactoryNode: Factory node being reimported
		 * @param PackageName: Package path of the actual object to reimport
		 * @param AssetName: Asset name of the actual object to reimport
		 * @param SubPathString: Optional subobject name
		 */
		static 	UE_API UObject* GetObjectToReimport(UInterchangeFactoryBase* Factory, UObject* ReimportObject, const UInterchangeFactoryBaseNode& FactoryNode, const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString());

		/**
		 * If the ReimportObject is a UInterchangeSceneImportAsset, returns the factory node
		 * associated with the asset which path is //PackageName.AssetName[:SubPathString].
		 * Returns nullptr otherwise.
		 * @param ReimportObject: Object that a reimport has been initialized from
		 * @param PackageName: Package path of the actual object to reimport
		 * @param AssetName: Asset name of the actual object to reimport
		 * @param SubPathString: Optional subobject name
		 */
		static 	UE_API const UInterchangeFactoryBaseNode* GetFactoryNode(UObject* ReimportObject, const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString());

		/**
		 * Evaluates whether or not a factory should be called on a factory node in a given reimport context.
		 * @param FactoryNode: Factory node to evaluate
		 * @param ReimportObject: Object a reimport has been initialized from
		 * @param ObjectToReimport: Actual object to reimport
		 * @param PackageName: Package path of the object associated with the factory node to reimport
		 * @param AssetName: Asset name of the object associated with the factory node to reimport
		 * @param SubPathString: Optional subobject name
		 * Returns:
		 *		- true if it is an import. ReimportObject is null.
		 *      - true if, based on its path, object associated with the factory node was not part of the previous import.
		 *		- true if factory node was part of the previous import and ObjectToReimport is null but
		 *			FactoryNode has been flagged as required for the reimport
		 *		- true ObjectToReimport is not null
		 *		- false otherwise
		 *	@note The path of the missing associated object will be set on the factory node if ObjectToReimport is null
		 */
		static 	bool UE_API CanProceedWithFactoryNode(UInterchangeFactoryBaseNode& FactoryNode, UObject* ReimportObject, UObject* ObjectToReimport, const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString());
		/**
		 * Factory helper use to find the object we have create in the game thread when we do the async part of the factory import.
		 */
		static UE_API UObject* AsyncFindObject(UInterchangeFactoryBaseNode* FactoryNode, const UClass* FactoryClass, UObject* Parent, const FString& AssetName);
	};
} //ns UE::Interchange

#undef UE_API
