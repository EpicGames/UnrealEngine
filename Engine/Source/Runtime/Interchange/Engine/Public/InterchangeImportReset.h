// Copyright Epic Games, Inc. All Rights Reserved.
#pragma once

#include "Containers/Array.h"
#include "Containers/UnrealString.h"
#include "CoreTypes.h"
#include "Delegates/DelegateCombinations.h"
#include "Templates/UniquePtr.h"
#include "UObject/ObjectPtr.h"
#include "UObject/SoftObjectPath.h"

class FAutoConsoleVariableRef;

class UInterchangeBaseNode;
class UInterchangeBaseNodeContainer;
class UInterchangeFactoryBase;
class UInterchangeFactoryBaseNode;
class UInterchangePipelineBase;
class UInterchangeTranslatorBase;
class UInterchangeResultsContainer;
class UInterchangeSceneImportAsset;

DECLARE_DELEGATE_OneParam(FOnResetPhaseCompletedDelegate, class FInterchangeResetParameters&)
DECLARE_DELEGATE_RetVal_TwoParams(bool, FOnFilterFactoryNodeDelegate, class FInterchangeResetParameters&, const UInterchangeFactoryBaseNode*)
DECLARE_DELEGATE_ThreeParams(FOnNodeProcessedDelegate, class FInterchangeResetParameters&, const UInterchangeFactoryBase*, const UInterchangeFactoryBaseNode*)

DECLARE_LOG_CATEGORY_EXTERN(LogInterchangeReset, Display, All)

struct FResetPhaseDelegates
{
	FOnFilterFactoryNodeDelegate OnNodeFilter;
	FOnNodeProcessedDelegate OnNodeProcessed;
	FOnResetPhaseCompletedDelegate OnCompleted;
};

/* Used for storing objects that might be needed to be referenced later. */
struct FInterchangeResetContextData
{
	TMap<const UInterchangeFactoryBaseNode*, TArray<TObjectPtr<UObject>>> ObjectsSpawnedDuringReset;
};

class FInterchangeResetParameters
{
	struct FFactoryNodeData
	{
		FSoftObjectPath ReferencedObjectPath;
		TObjectPtr<UInterchangeFactoryBase> Factory;
		TArray<TObjectPtr<UObject>> ObjectsToReset;
	};

public:
	INTERCHANGEENGINE_API FInterchangeResetParameters(const UInterchangeSceneImportAsset* InSceneImportAsset);
	INTERCHANGEENGINE_API FInterchangeResetParameters(const UInterchangeSceneImportAsset* InSceneImportAsset, TUniquePtr<FInterchangeResetContextData> InResetContextData);
	INTERCHANGEENGINE_API ~FInterchangeResetParameters();

	INTERCHANGEENGINE_API void AddObjectInstanceToReset(const UInterchangeFactoryBaseNode* FactoryNode, UObject* ObjectToReset);

	void SetupFactoryNode(const UInterchangeFactoryBaseNode* FactoryNode);
	TObjectPtr<UInterchangeFactoryBase> GetFactoryForNode(const UInterchangeFactoryBaseNode* FactoryNode) const;
	TArray<TObjectPtr<UObject>> GetObjectInstancesForFactoryNode(const UInterchangeFactoryBaseNode* FactoryNode);


	const UInterchangeBaseNodeContainer* GetBaseNodeContainer() const;
	const UInterchangeSceneImportAsset* GetSceneImportAsset() const;
	const UInterchangeResultsContainer* GetResultsContainer() const;

	FResetPhaseDelegates PreResetDelegates;
	FResetPhaseDelegates ResetDelegates;
	FResetPhaseDelegates PostResetDelegates;


	TUniquePtr<FInterchangeResetContextData> ResetContextData;
	TSet<const UInterchangeFactoryBaseNode*> FilteredNodes;
private:
	TObjectPtr<const UInterchangeSceneImportAsset> SceneImportAsset;
	TObjectPtr<UInterchangeResultsContainer> ResultsContainer;

	TMap<const UInterchangeFactoryBaseNode*, FFactoryNodeData> FactoryNodeDataCache;
};


class FInterchangeReset
{
public:
	static INTERCHANGEENGINE_API void ExecuteReset(FInterchangeResetParameters& ResetObjectParameters);
};

namespace UE::Interchange::InterchangeReset
{
	INTERCHANGEENGINE_API extern FAutoConsoleVariableRef CCvarInterchangeResetFilteredNodes;
	
	namespace Constants
	{
		INTERCHANGEENGINE_API extern const FString SceneImportAssetPathKey;
		INTERCHANGEENGINE_API extern const FString FactoryNodeUidPathKey;
	}

}

