// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include <atomic>

#include "CoreMinimal.h"

#include "Async/TaskGraphInterfaces.h"
#include "Containers/Queue.h"
#include "Delegates/DelegateCombinations.h"
#include "HAL/CriticalSection.h"
#include "HAL/Thread.h"
#include "HAL/ThreadSafeBool.h"
#include "InterchangeAssetImportData.h"
#include "InterchangeAnalyticsHandlerBase.h"
#include "InterchangeEditorUtilitiesBase.h"
#include "InterchangeFactoryBase.h"
#include "InterchangePipelineConfigurationBase.h"
#include "InterchangeResultsContainer.h"
#include "InterchangeSourceData.h"
#include "InterchangeTaskSystem.h"
#include "InterchangeTranslatorBase.h"
#include "InterchangeWriterBase.h"
#include "Misc/Guid.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Templates/Tuple.h"
#include "UObject/GCObject.h"
#include "UObject/Package.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "UObject/StrongObjectPtr.h"
#include "Containers/Ticker.h"

#include "InterchangeManager.generated.h"

#define UE_API INTERCHANGEENGINE_API

class FAsyncTaskNotification;
class UInterchangeBlueprintPipelineBase;
class UInterchangeFactoryBaseNode;
class UInterchangePipelineBase;
class UInterchangePythonPipelineBase;
class ULevel;

struct FAnalyticsEventAttribute;

/** Some utility delegates for automating Interchange. */
DECLARE_DYNAMIC_DELEGATE_OneParam(FOnObjectImportDoneDynamic, UObject*, Object);
DECLARE_DELEGATE_OneParam(FOnObjectImportDoneNative, UObject*);

DECLARE_DYNAMIC_DELEGATE_OneParam(FOnImportDoneDynamic, const TArray<UObject*>&, Objects);
DECLARE_DELEGATE_OneParam(FOnImportDoneNative, const TArray<UObject*>&);

/** Delegate Type that is fired when Interchange starts importing. Won't fire when a new import process starts while one is already in progress. Primarily Used for AutoSave setting. */
DECLARE_MULTICAST_DELEGATE(FOnImportStarted);
/** Delegate Type that is fired when Interchange finished importing. Won't fire when an import process finishes while one is still in progress. Primarily Used for AutoSave setting. */
DECLARE_MULTICAST_DELEGATE(FOnImportFinished);

enum class ESanitizeNameTypeFlags : uint8
{
	None = 0x00,
	Name = 0x01,
	ObjectName = 0x02,
	ObjectPath = 0x04,
	LongPackage = 0x08
};
ENUM_CLASS_FLAGS(ESanitizeNameTypeFlags)

//Thread safe delegate since this can be broadcast in any thread
DECLARE_TS_MULTICAST_DELEGATE_TwoParams(FOnSanitizeName, FString& /*NameToSanitize*/, const ESanitizeNameTypeFlags /*TypeName*/);

namespace UE
{
	namespace Interchange
	{
		class FScopedInterchangeImportEnableState
		{
		public:
			UE_API explicit FScopedInterchangeImportEnableState(const bool bScopeValue);
			UE_API ~FScopedInterchangeImportEnableState();
		private:
			bool bOriginalInterchangeImportEnableState;
		};

		class FScopedSourceData
		{
		public:
			UE_API explicit FScopedSourceData(const FString& Filename);
			UE_API ~FScopedSourceData();
			UE_API UInterchangeSourceData* GetSourceData() const;
		private:
			TStrongObjectPtr<UInterchangeSourceData> SourceDataPtr = nullptr;
		};

		class FScopedTranslator
		{
		public:
			UE_API explicit FScopedTranslator(const UInterchangeSourceData* SourceData);
			UE_API ~FScopedTranslator();
			UE_API UInterchangeTranslatorBase* GetTranslator();

		private:
			TStrongObjectPtr<UInterchangeTranslatorBase> ScopedTranslatorPtr = nullptr;
		};

		class FScopedBaseNodeContainer
		{
		public:
			UE_API FScopedBaseNodeContainer();
			UE_API ~FScopedBaseNodeContainer();
			UE_API UInterchangeBaseNodeContainer* GetBaseNodeContainer();

		private:
			TStrongObjectPtr<UInterchangeBaseNodeContainer> ScopedBaseNodeContainerPtr = nullptr;
		};

		enum class EImportType : uint8
		{
			ImportType_None,
			ImportType_Asset,
			ImportType_Scene
		};

		struct FImportAsyncHelperData
		{
			// True if the import process is unattended. We cannot show UI if the import is automated.
			bool bIsAutomated = false;

			// True if redirectors will be followed when determining what location to import an asset.
			bool bFollowRedirectors = false;

			// We can import assets or full scene.
			EImportType ImportType = EImportType::ImportType_None;

			// True if we are reimporting assets or a scene.
			UObject* ReimportObject = nullptr;

			// Level to import into, if we are doing a scene import.
			ULevel* ImportLevel = nullptr;

			/** Optional custom name for the import. */
			FString DestinationName;

			/** Whether or not to overwrite existing assets. */
			bool bReplaceExisting = true;
		};

		class FImportResult : protected FGCObject
		{
		public:
			UE_API FImportResult();

			FImportResult(FImportResult&&) = delete;
			FImportResult& operator=(FImportResult&&) = delete;

			FImportResult(const FImportResult&) = delete;
			FImportResult& operator=(const FImportResult&) = delete;

			virtual ~FImportResult() = default;

		public:
			enum class EStatus
			{
				Invalid,
				InProgress,
				Done
			};

			UE_API EStatus GetStatus() const;

			UE_API bool IsValid() const;

			UE_API void SetInProgress();
			UE_API void SetDone();
			UE_API void WaitUntilDone(bool bSynchronous = false);

			// Assets are only made available once they have been completely imported (passed through the entire import pipeline).
			// While the status isn't EStatus::Done, the list can grow between subsequent calls.
			// FAssetImportResult holds a reference to the assets so that they aren't garbage collected.
			UE_API const TArray< UObject* >& GetImportedObjects() const;

			// Helper to get the first asset of a certain class. Use when expecting a single asset of that class to be imported, because the order isn't deterministic.
			UE_API UObject* GetFirstAssetOfClass(UClass* InClass) const;

			// Return the results of this asset import operation.
			UInterchangeResultsContainer* GetResults() const { return Results; }

			// Adds an asset to the list of imported assets.
			UE_API void AddImportedObject(UObject* ImportedObject);

			// Callback when the status switches to done.
			UE_API void OnDone(TFunction< void(FImportResult&) > Callback);

			//Set the async helper that own this import result
			UE_API void SetAsyncHelper(TWeakPtr<class FImportAsyncHelper> InAsyncHelper);

			// Internal delegates. To set these, use the FImportAssetParameters when calling the Interchange import functions.
			FOnObjectImportDoneDynamic OnObjectDone;
			FOnObjectImportDoneNative OnObjectDoneNative;

			FOnImportDoneDynamic OnImportDone;
			FOnImportDoneNative OnImportDoneNative;

		protected:
			/* FGCObject interface */
			UE_API virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
			virtual FString GetReferencerName() const override
			{
				return TEXT("UE::Interchange::FImportResult");
			}

		private:
			std::atomic< EStatus > ImportStatus;

			TArray< TObjectPtr<UObject> > ImportedObjects;
			mutable FRWLock ImportedObjectsRWLock;
			TObjectPtr<UInterchangeResultsContainer> Results;

			TWeakPtr<class FImportAsyncHelper> AsyncHelper = nullptr;

			TFunction< void(FImportResult&) > DoneCallback;
		};

		using FAssetImportResultRef = TSharedRef< FImportResult, ESPMode::ThreadSafe >;
		using FSceneImportResultRef = TSharedRef< FImportResult, ESPMode::ThreadSafe >;
		using FAssetImportResultPtr = TSharedPtr< FImportResult, ESPMode::ThreadSafe >;
		using FSceneImportResultPtr = TSharedPtr< FImportResult, ESPMode::ThreadSafe >;

		class FImportAsyncHelper : protected FGCObject
		{
		public:
			static FInterchangeImportResultAnalyticsInfo CreateImportResultAnalyticsInfo(FImportAsyncHelper& AsyncHelper);

		public:
			FImportAsyncHelper();

			~FImportAsyncHelper();

			/* FGCObject interface */
			virtual void AddReferencedObjects(FReferenceCollector& Collector) override;
			virtual FString GetReferencerName() const override
			{
				return TEXT("UE::Interchange::FImportAsyncHelper");
			}

			bool bRunSynchronous = false;
			bool bRuntimeOrPIE = false;

			/** Unique ID for this async helper. */
			int32 UniqueId;

			//The base path to import the content.
			FString ContentBasePath;

			//The following arrays are per source data
			TArray<TStrongObjectPtr<UInterchangeBaseNodeContainer>> BaseNodeContainers;
			TArray<TObjectPtr<UInterchangeSourceData>> SourceDatas;
			TArray<TObjectPtr<UInterchangeTranslatorBase>> Translators;

			//The pipelines array is not per source data.
			TArray<TObjectPtr<UInterchangePipelineBase>> Pipelines;
			//The original pipelines asset to save in the asset reimport data. The original pipeline can restore Python class member value.
			//Python class instanced assets cannot be saved, so we have to serialize in JSON the data to restore it when we do a reimport.
			TArray<UObject*> OriginalPipelines;

			TArray<uint64> TranslatorTasks;
			TArray<uint64> PipelineTasks;
			TArray<uint64> WaitAssetCompilationTasks;
			TArray<uint64> PostImportTasks;
			uint64 ParsingTask;
			TArray<uint64> ImportObjectQueryPayloadsTasks;
			TArray<uint64> BeginImportObjectTasks;
			TArray<uint64> ImportObjectTasks;
			TArray<uint64> FinalizeImportObjectTasks;
			TArray<uint64> SceneTasks;

			uint64 PreCompletionTask;
			uint64 CompletionTask;

			//Return true if we can import this class, or false otherwise.
			bool IsClassImportAllowed(UClass* Class);

			UPackage* GetCreatedPackage(const FString& PackageName) const;
			void AddCreatedPackage(const FString& PackageName, UPackage* Package);

			UInterchangeFactoryBase* GetCreatedFactory(const FString& FactoryNodeUniqueId) const;
			void AddCreatedFactory(const FString& FactoryNodeUniqueId, UInterchangeFactoryBase* Factory);

			struct FImportedObjectInfo
			{
				UE_DEPRECATED(5.8, "Direct use of this variable is deprecated, Please use GetImportedObject()/SetImportedObject() instead")
				FSoftObjectPath ImportedObject; // The object that was imported.

				UObject* GetImportedObject() const 
				{ 
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					return ImportedObject.TryLoad(); 
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}

				void SetImportedObject(UObject* InImportedObject) 
				{ 
					PRAGMA_DISABLE_DEPRECATION_WARNINGS
					ImportedObject = InImportedObject; 
					PRAGMA_ENABLE_DEPRECATION_WARNINGS
				}

				UInterchangeFactoryBase* Factory = nullptr; //The factory that created the imported object.
				UInterchangeFactoryBaseNode* FactoryNode = nullptr; //The node that describes the object.
				bool bIsReimport;
				mutable bool bPostEditChangeCalled = false; //This field is set by the PreCompletionTask and need to be mutable
			};

			UE_API FImportedObjectInfo& AddDefaultImportedAssetGetRef(int32 SourceIndex);
			const FImportedObjectInfo* FindImportedAssets(int32 SourceIndex, TFunction< bool(const FImportedObjectInfo& ImportedObjects) > Predicate) const;
			void IterateImportedAssets(int32 SourceIndex, TFunction< void(const TArray<FImportedObjectInfo>& ImportedObjects) > Callback) const;
			void IterateImportedAssetsPerSourceIndex(TFunction< void(int32 SourceIndex, const TArray<FImportedObjectInfo>& ImportedObjects) > Callback) const;

			FImportedObjectInfo& AddDefaultImportedSceneObjectGetRef(int32 SourceIndex);
			const FImportedObjectInfo* FindImportedSceneObjects(int32 SourceIndex, TFunction< bool(const FImportedObjectInfo& ImportedObjects) > Predicate) const;
			void IterateImportedSceneObjects(int32 SourceIndex, TFunction< void(const TArray<FImportedObjectInfo>& ImportedObjects) > Callback) const;
			void IterateImportedSceneObjectsPerSourceIndex(TFunction< void(int32 SourceIndex, const TArray<FImportedObjectInfo>& ImportedObjects) > Callback) const;

			/*
			 * Return true if the Object is imported by this async import, or false otherwise.
			 */
			bool IsImportingObject(UObject* Object) const;

			FImportAsyncHelperData TaskData;

			FAssetImportResultRef AssetImportResult;
			FSceneImportResultRef SceneImportResult;
			
			//If we cancel the tasks, we set this Boolean to true.
			std::atomic<bool> bCancel;

			void ReleaseTranslatorsSource();

			/**
			 * Wait synchronously after the graph parsing task is done, and return the GraphEventArray up to the completion TaskGraphEvent.
			 */
			TArray<uint64> GetCompletionTaskGraphEvent();

			void InitCancel();

			void CleanUp();

			double TaskExecuteStarted;

			// This class owns the Analytics Handler and manages its lifetime.
			// Analytics Handler will be released at the end of import when the ImportAsyncHandler is deleted.
			TStrongObjectPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandler;
		private:
				static TMap<int32, TArray<TObjectPtr<UObject>>> ExtractImportedObjectsPerSourceIndex(TMap<int32, TArray<FImportedObjectInfo>>& InObjectInfoPerSourceIndex);

		private:
			FCriticalSection ClassPermissionLock;
			// Set of classes whose creation has been denied.
			TSet<UClass*> DeniedClasses;
			// Set of classes whose creation is allowed.
			TSet<UClass*> AllowedClasses;

			//Created package map. The key is the package name. We cannot create packages asynchronously, so we have to create a game thread task to do this.
			mutable FCriticalSection CreatedPackagesLock;
			TMap<FString, UPackage*> CreatedPackages;

			// Created factories map. The key is the factory node UID.
			mutable FCriticalSection CreatedFactoriesLock;
			TMap<FString, TObjectPtr<UInterchangeFactoryBase>> CreatedFactories;

			mutable FCriticalSection ImportedAssetsPerSourceIndexLock;
			TMap<int32, TArray<FImportedObjectInfo>> ImportedAssetsPerSourceIndex;

			mutable FCriticalSection ImportedSceneObjectsPerSourceIndexLock;
			TMap<int32, TArray<FImportedObjectInfo>> ImportedSceneObjectsPerSourceIndex;
		};

		/* This function takes an asset that represents a pipeline and generates a UInterchangePipelineBase asset. */
		UE_API UInterchangePipelineBase* GeneratePipelineInstance(const FSoftObjectPath& PipelineInstance);
	} //ns Interchange
} //ns UE

/**
 * This class is used to pass override pipelines in the ImportAssetTask Options member.
 */
UCLASS(Transient, BlueprintType, MinimalAPI)
class UInterchangePipelineStackOverride : public UObject
{
	GENERATED_BODY()
public:

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	TArray<FSoftObjectPath> OverridePipelines;

	UFUNCTION(BlueprintCallable, Category = "Interchange | PipelineStackOverride")
	UE_API void AddPythonPipeline(UInterchangePythonPipelineBase* PipelineBase);

	UFUNCTION(BlueprintCallable, Category = "Interchange | PipelineStackOverride")
	UE_API void AddBlueprintPipeline(UInterchangeBlueprintPipelineBase* PipelineBase);

	UFUNCTION(BlueprintCallable, Category = "Interchange | PipelineStackOverride")
	UE_API void AddPipeline(UInterchangePipelineBase* PipelineBase);
};

USTRUCT(BlueprintType)
struct FImportAssetParameters
{
	GENERATED_USTRUCT_BODY()

	// If the import is a reimport for a specific asset, set the asset to reimport here.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	TObjectPtr<UObject> ReimportAsset = nullptr;

	// If we are doing a reimport, set the source index here. Some assets have more then one source file that each contains part of the asset content.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	int32 ReimportSourceIndex = INDEX_NONE;

	// Tell Interchange that import is automated and it shouldn't present a modal window.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	bool bIsAutomated = false;

	// Tell Interchange to follow redirectors when determining the location an asset will be imported.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	bool bFollowRedirectors = false;

	// Adding overrides tells Interchange to use the specific custom set of pipelines instead of letting the user or the system choose.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangePipelineBase, /Script/InterchangeEngine.InterchangeBlueprintPipelineBase, /Script/InterchangeEngine.InterchangePythonPipelineAsset"))
	TArray<FSoftObjectPath> OverridePipelines;

	//Level to import into when doing a scene import.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	TObjectPtr<ULevel> ImportLevel = nullptr;

	/** Optional custom name for the import. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	FString DestinationName;

	/** Determies whether to overwrite existing assets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	bool bReplaceExisting = true;

	/** If true this import must show the import dialog and ignore the show dialog settings. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset")
	bool bForceShowDialog = false;

	/* Delegates used to track the imported objects. */

	// This is called each time an asset is imported or reimported from the import call.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset", meta=(PinHiddenByDefault))
	FOnObjectImportDoneDynamic OnAssetDone;
	FOnObjectImportDoneNative OnAssetDoneNative;

	// This is called when all the assets were imported from the source data.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset", meta=(PinHiddenByDefault))
	FOnImportDoneDynamic OnAssetsImportDone;
	FOnImportDoneNative OnAssetsImportDoneNative;

	// This is called each time an object in the scene is imported or reimported from the import call.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset", meta=(PinHiddenByDefault))
	FOnObjectImportDoneDynamic OnSceneObjectDone;
	FOnObjectImportDoneNative OnSceneObjectDoneNative;

	// This is called when all the scene objects were imported from the source data.
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange|ImportAsset", meta=(PinHiddenByDefault))
	FOnImportDoneDynamic OnSceneImportDone;
	FOnImportDoneNative OnSceneImportDoneNative;

	// Tell Interchange that import must run synchronously on the game thread.
	//This is an internal property set by the Import API
	mutable bool bRunSynchronous;
};

UCLASS(Transient, BlueprintType, CustomConstructor, MinimalAPI)
class UInterchangeManager : public UObject
{
	GENERATED_BODY()
public:

	UInterchangeManager(const FObjectInitializer& ObjectInitializer);

	/**
	 * Return the pointer to the Interchange Manager singleton.
	 *
	 * @note - We need to return a pointer to have a valid Blueprint-callable function.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	static UInterchangeManager* GetInterchangeManagerScripted()
	{
		return &GetInterchangeManager();
	}

	/** Return the Interchange Manager singleton. */
	static UE_API UInterchangeManager& GetInterchangeManager();

	/** Return the CVar that enables or disables Interchange. */
	static UE_API bool IsInterchangeImportEnabled();
	/** Set the CVar that enables or disables Interchange. */
	static UE_API void SetInterchangeImportEnabled(bool bEnabled);

	/** Checks if there are any imports in progress.*/
	static UE_API bool IsImporting();

	/** Delegate type that is fired when new assets have been imported. Note: InCreatedObject can be NULL if the import failed. Params: UFactory* InFactory, UObject* InCreatedObject. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FInterchangeOnAssetPostImport, UObject*);
	/** Delegate type that is fired when new assets have been reimported. Note: InCreatedObject can be NULL if the import failed. Params: UObject* InCreatedObject. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FInterchangeOnAssetPostReimport, UObject*);
	/** Delegate type that is fired when the import results in an error. */
	DECLARE_MULTICAST_DELEGATE_OneParam(FInterchangeOnBatchImportComplete, TStrongObjectPtr<UInterchangeResultsContainer>);
	/** Delegate type that is fired when the import is completed and results are available to iterate on. */
	using FSourceIndexObjectsMap = TMap<int32, TArray<TObjectPtr<UObject>>>;
	DECLARE_MULTICAST_DELEGATE_TwoParams(FInterchangeOnImportedObjectsAvailable, FSourceIndexObjectsMap, FSourceIndexObjectsMap);

	// Delegates used to register and unregister.

	FInterchangeOnAssetPostImport OnAssetPostImport;
	FInterchangeOnAssetPostReimport OnAssetPostReimport;
	FInterchangeOnBatchImportComplete OnBatchImportComplete;

	//Fires when the first import process starts.
	FOnImportStarted OnImportStarted;

	//Fires when the imported assets and scene actors are available.
	FInterchangeOnImportedObjectsAvailable OnImportedObjectsAvailable;

	//Fires when the last import process finishes.
	FOnImportFinished OnImportFinished;

	//Fire when we need to sanitize a name, make sure your delegate code is thread safe, since it will be broadcast in any thread.
	FOnSanitizeName OnSanitizeName;

	// Called when before the application is exiting.
	FSimpleMulticastDelegate OnPreDestroyInterchangeManager;

	UE_API void SanitizeNameInline(FString& NameToSanitize, const ESanitizeNameTypeFlags NameType);
	/**
	 * All translators must be registered with the manager.
	 * @Param Translator - The UClass of the translator you want to register.
	 * @return true if the translator class can be registered, or false otherwise.
	 *
	 * @Note If you register the same class multiple times, this returns true for every call.
	 * @Note The order in which the translators are registered will be the same as the order used to select a translator to import a file.
	 */
	UE_API bool RegisterTranslator(const UClass* TranslatorClass);

	/**
	 * All factories must be registered with the manager.
	 * @Param Factory - The UClass of the factory you want to register.
	 * @return true if the factory class can be registered, or false otherwise.
	 *
	 * @Note If you register the same class multiple time, this returns true for every call.
	 */
	UE_API bool RegisterFactory(const UClass* Factory);

	/**
	 * All writers must be registered with the manager,
	 * @Param Writer - The UClass of the writer you want to register,
	 * @return true if the writer class can be registered, or false otherwise.
	 *
	 * @Note If you register the same class multiple time, this returns true for every call.
	 */
	UE_API bool RegisterWriter(const UClass* Writer);

	/**
	 * All converters must be registered with the manager.
	 * @Param Converter - The UClass of the converter you want to register.
	 * @return true if the converter class can be registered, or false otherwise.
	 *
	 * @Note If you register the same class multiple time, this returns true for every call.
	 */
	UE_API bool RegisterImportDataConverter(const UClass* Converter);

	/**
	 * Call all the registered converters to see if any converter can convert the data.
	 * @Param Asset - The asset we want to convert the import data.
	 * @Param Extension - The file extension we want to import.
	 * @return true if one of the converter has converted the data, or false otherwise.
	 */
	UE_API bool ConvertImportData(UObject* Asset, const FString& Extension) const;

	/**
	 * Call all the registered converter, until one can convert the source import data to UInterchangeAssetImportData
	 * @Param SourceImportData - The source import data options.
	 * @Param ImportAssetParameters - The interchange import asset parameters.
	 * @return true if one of the converter has convert the data. False otherwise.
	 */
	UE_API bool ConvertImportData(const UObject* SourceImportData, FImportAssetParameters& ImportAssetParameters) const;

	/**
	 * Call all the registered converter, until one can convert the source data to the destination
	 * @Param SourceImportData - The source import data options.
	 * @Param DestinationImportData - The destination import data options we have to create and fill.
	 * @Param DestinationClass - The class representing the DestinationImportData
	 * @return true if one of the converter has convert the data. False otherwise.
	 */
	UE_API bool ConvertImportData(const UObject* SourceImportData, const UClass* DestinationClass, UObject** DestinationImportData) const;

	/**
	 * Returns the list of supported formats for a given translator type.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API TArray<FString> GetSupportedFormats(const EInterchangeTranslatorType ForTranslatorType) const;

	/**
	 * Returns the list of formats supporting the specified translator asset type.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API TArray<FString> GetSupportedAssetTypeFormats(const EInterchangeTranslatorAssetType ForTranslatorAssetType, const EInterchangeTranslatorType ForTranslatorType = EInterchangeTranslatorType::Invalid, bool bStrictMatchTranslatorType = false) const;

	/**
	 * Returns the list of supported formats for a given Object.
	 * 
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API TArray<FString> GetSupportedFormatsForObject(const UObject* Object, int32 SourceFileIndex) const;

	/**
	 * Check whether there is a registered translator for this source data.
	 * This allows us to bypass the original asset tools system to import supported assets.
	 * @Param SourceData - The source data input we want to translate to Uod.
	 * @return True if there is a registered translator that can handle handle this source data, or false otherwise, when bSceneImportOnly is false.
	 * Otherwise, returns true only if the translator supports level import
	 * @Note: Temporary until FBX level import is production ready
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool CanTranslateSourceData(const UInterchangeSourceData* SourceData, bool bSceneImportOnly = false) const;

	/**
	 * Returns true if Interchange can create this type of asset and is able to translate its source files.
	 * @Param Object - The object we want to reimport.
	 * @Param OutFilenames - An array that is filled with the object's source filenames if the operation is successful.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool CanReimport(const UObject* Object, TArray<FString>& OutFilenames) const;

	/**
	 * Call this to start a synchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import succeeds, or false otherwise.
	 * 
	 * @Note - In blueprint depending on the event you use to start the import its possible to have a deadlock, use the async function if its what you are experimenting
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool ImportAsset(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters, TArray<UObject*>& OutImportedObjects);

	/**
	 * Call this to start a synchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import succeeds, or false otherwise.
	 *
	 */
	UE_API bool ImportAsset(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start a synchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return return an import result which can be use to know when the asynchronous import is terminate.
	 */
	UE_API UE::Interchange::FAssetImportResultRef ImportAssetWithResult(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start an asynchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return return an import result which can be use to know when the asynchronous import is terminate.
	 */
	UE_API UE::Interchange::FAssetImportResultRef ImportAssetAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);
	
	/**
	 * Call this from blueprint or python to start an asynchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import was started, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool ScriptedImportAssetAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start a synchronous asset re-import process.
	 * This process can re-import many different assets into the game content.
	 *
	 * @Param ObjectToReimport - The object to re-import.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import succeeds, or false otherwise.
	 * 
	 * @Note - The interchange manager will by default use the last file use to re-import the ObjectToReimport. If the file doesn't exist
	 *         and the bAutomated flag is:
	 *         - true, the function will return false and log a warning
	 *         - false, a dialog will ask 
	 * @Note - In blueprint depending on the event you use to start the import its possible to have a deadlock, use the async function if its what you are experimenting
	 */
 	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
 	UE_API bool ReimportAsset(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters, TArray<UObject*>& OutImportedObjects);

	/**
	 * Call this to start an asynchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return return an import result which can be use to know when the asynchronous import is terminate.
	 */
	UE_API UE::Interchange::FAssetImportResultRef ReimportAssetAsync(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this from blueprint or python to start an asynchronous asset import process.
	 * This process can import many different assets into the game content.
	 *
	 * @Param ObjectToReimport - The object to re-import.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import was started, or false otherwise.
	 * 
	 * @Note - The interchange manager will by default use the last file use to re-import the ObjectToReimport. If the file doesn't exist
	 *         and the bAutomated flag is:
	 *         - true, the function will return false and log a warning
	 *         - false, a dialog will ask 
	 * @Note - If you need an event for when the import is done use the ImportAssetParameters events.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool ScriptedReimportAssetAsync(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start a synchronous scene import process.
	 * This process can import many different assets and their transforms (USceneComponent).
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate. This object will be duplicated to allow thread-safe operations.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import succeeds, or false otherwise.
	 * 
	 * @Note - In blueprint depending on the event you use to start the import its possible to have a deadlock, use the async function if its what you are experimenting
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool ImportScene(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start a asynchronous scene import process.
	 * This process can import many different assets and their transforms (USceneComponent).
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate. This object will be duplicated to allow thread-safe operations.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return true if the import was started, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool ScriptedImportSceneAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start a asynchronous scene import process.
	 * This process can import many different assets and their transforms (USceneComponent), store the result in a Blueprint, and add the Blueprint to the level.
	 *
	 * @Param ContentPath - The path where the imported assets will be created.
	 * @Param SourceData - The source data input to translate. This object will be duplicated to allow thread-safe operations.
	 * @param ImportAssetParameters - All parameters that need to be passed to the import asset function.
	 * @return return a pair of import result which can be use to know when the asynchronous import is terminate.
	 */
	UE_API TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>
	ImportSceneAsync(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters);

	/**
	 * Call this to start an asset export process. The caller must specify a source data.
	 * 
	 * @Param Asset - The asset to export.
	 * @Param bIsAutomated - If true, the exporter will not show any UI or dialogs.
	 * @return true if the export succeeds, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Export Manager")
	UE_API bool ExportAsset(const UObject* Asset, bool bIsAutomated = false);

	/**
	 * Call this to start a scene export process. The caller must specify a source data.
	 * @Param World - The scene to export.
	 * @Param bIsAutomated - If true, the import process will not show any UI or dialogs.
	 * @return true if the export succeeds, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Export Manager")
	UE_API bool ExportScene(const UObject* World, bool bIsAutomated = false);

	/*
	* Script helper to create a source data object that points to a file on disk.
	* @Param InFilename: Specify a file on disk.
	* @return: A new UInterchangeSourceData.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	static UE_API UInterchangeSourceData* CreateSourceData(const FString& InFileName);

	/**
	* Script helper to get a registered factory for a specified class.
	* @Param FactoryClass: The class whose registered factory you want to find.
	* @return: The registered factory class if found, or NULL if no registered factory was found.
	*/
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API const UClass* GetRegisteredFactoryClass(const UClass* ClassToMake) const;

	/**
	 * Initializes an FImportAsyncHelper with correct task data and import asset parameters. The Async Helper is deleted when ReleaseAsyncHelper is called.
	 * @param Data - The data to pass to the different import tasks.
	 */
	UE_API void InitAsyncHelper(TSharedRef<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe>& AsyncHelper, const UE::Interchange::FImportAsyncHelperData& Data, const FImportAssetParameters& ImportAssetParameters);

	/** Delete the specified AsyncHelper and remove it from the array that was holding it. */
	UE_API void ReleaseAsyncHelper(TWeakPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper);

	/**
	 * Return a pointer to an FImportAsyncHelper that is the specified UniqueId. Return an empty shared pointer if the unique id is not matching any async helper
	 * The return asynchelper should be release before the end of the import process to make sure all resource are release when the import is completed.
	 */
	UE_API TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> GetAsyncHelper(int32 UniqueId);

	/*
	 * Return the first translator that can translate the source data.
	 * @Param SourceData - The source data that you want a translator for.
	 * @return return a matching translator, or nullptr if no translators exist for the source data.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API UInterchangeTranslatorBase* GetTranslatorForSourceData(const UInterchangeSourceData* SourceData) const;

	/**
	 * Return true if Interchange is actively importing or exporting, or false otherwise.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool IsInterchangeActive();

	/**
	 * Return false if Interchange is not actively importing or exporting.
	 * If Interchange is active, it will display a notification to let the user know they can cancel the asynchronous import/export
	 * to be able to complete the operation they requested. (This is called by the exit editor operation.)
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool WarnIfInterchangeIsActive();

	/**
	 * Returns the list of supported formats for a given translator type.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API UInterchangeAssetImportData* GetAssetImportData(UObject* Asset) const;

	/**
	 * Check whether there is a translator registered that can translate the source data with the specified PayloadInterface.
	 * @Param SourceData - The source data input you want to translate to Uod.
	 * @return true if the source data can be translated using the specified PayloadInterface, or false otherwise.
	 */
	UE_API bool CanTranslateSourceDataWithPayloadInterface(const UInterchangeSourceData* SourceData, const UClass* PayloadInterfaceClass) const;

	/*
	 * Return the first translator that can translate the source data with the specified PayloadInterface.
	 * @Param SourceData - The source data that you want a translator for.
	 * @Param PayloadInterfaceClass - The PayloadInterface that the translator must implement.
	 * @return return a matching translator that implements the specified PayloadInterface, or nullptr if none exists.
	 */
	UE_API UInterchangeTranslatorBase* GetTranslatorSupportingPayloadInterfaceForSourceData(const UInterchangeSourceData* SourceData, const UClass* PayloadInterfaceClass) const;

	/**
	 * Return true if the object is being imported, or false otherwise. If the user imports multiple file in the same folder, it's possible to
	 * have the same asset name in two different files.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API bool IsObjectBeingImported(UObject* Object) const;

	/**
	 * Queue task that are not directly a import or re-import of assets.
	 * The post import tasks are execute only when the QueuedTasks is empty (no more import task running)
	 * Example: if a skeletal mesh re-import cannot apply the existing alternate skinning data, it will enqueue a post import task to re-import those alternate skinning files.
	 */
	UE_API bool EnqueuePostImportTask(TSharedPtr<FInterchangePostImportTask> PostImportTask);


	/**
	* Sets the bReplaceExistingAllDialogAnswer which is responsible for:
	* if the Import process will override or not all existing assets this Import stack tries to override.
	*/
	UE_API static void SetReplaceExistingAlldialogAnswer(bool bReplaceExistingAllDialogAnswer);

	/**
	* Checks if the bReplaceExistingAllDialogAnswer.
	*/
	UE_API static void ResetReplaceExistingAlldialogAnswerSet();

	/**
	* Gets the bReplaceExistingAllDialogAnswer.
	*/
	UE_API static TOptional<bool> GetReplaceExistingAlldialogAnswer();

	/**
	* Set the editor utilities, those are use for editor operation like saving an asset.
	*/
	UE_API void SetEditorUtilities(UClass* EditorUtilitiesClass);
	
	/**
	* Get the editor utilities, those are use for editor operation like saving an asset.
	*/
	UE_API UInterchangeEditorUtilitiesBase* GetEditorUtilities() const;

	/**
	 * Check if a non-parallel translator is unlocked for using.
	 */
	UE_API bool CanUseTranslator(UInterchangeTranslatorBase* Translator) const;

	/**
	* Sets class to use for analytics. This will create a new object during import and use for the Analytics.
	*/
	UE_API void SetAnalyticsHandlerClass(const TSoftClassPtr<UInterchangeAnalyticsHandlerBase>& InAnalyticsHandlerClass);

protected:

	/** Return true if Interchange can show UI. */
	static UE_API bool IsAttended();

	/*
	 * Find all pipeline candidates (C++, Blueprint, and Python).
	 * @Param SourceData - The source data you need a translator for.
	 * @return return a matching translator, or nullptr if there is no match.
	 */
	UE_API void FindPipelineCandidate(TArray<UClass*>& PipelineCandidates);

	/**
	 * This function cancels all tasks and finishes them as fast as possible.
	 * We use this if the user cancels the work or if the editor exits.
	 * @note - This is a asynchronous call. Tasks will be completed (canceled) soon.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API void CancelAllTasks();

	/**
	 * Wait synchronously until all tasks are done.
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange | Import Manager")
	UE_API void WaitUntilAllTasksDone(bool bCancel);


	/**
	 * If we set the mode to active, we will setup the timer and add the thread that will block the GC.
	 * If the we set the mode to inactive, we will remove the timer and finish the thread that blocks the GC.
	 */
	UE_API void SetActiveMode(bool IsActive);

	/**
	 * Start tasks until we reach the taskgraph worker number.
	 * @param bCancelAllTasks - If true, we will start all tasks but with the cancel state set, so tasks will complete fast and call the completion task.
	 */
	UE_API void StartQueuedTasks(bool bCancelAllTasks = false);

	/**
	 * Called by the public Import functions.
	 */
	UE_API TTuple<UE::Interchange::FAssetImportResultRef, UE::Interchange::FSceneImportResultRef>
	ImportInternal(const FString& ContentPath, const UInterchangeSourceData* SourceData, const FImportAssetParameters& ImportAssetParameters, const UE::Interchange::EImportType ImportType);

	TOptional<FString> ValidateReimportParameter(UObject* ObjectToReimport, const FImportAssetParameters& ImportAssetParameters, bool bRunSynchronous);

	int32 GetImportTaskCount() const;
	TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> GetImportTaskForIndex(int32 Index) const;
	TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> GetImportTaskForAsyncHelperUniqueId(int32 AsyncHelperUniqueId) const;
	/* Return the index in the ImportTasks array */
	int32 AddImportTask(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> ImportTask);
	void RemoveImportTask(TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> ImportTask);
	void RemoveImportTask(int32 AsyncHelperUniqueId);

private:
	TStrongObjectPtr<UInterchangeAnalyticsHandlerBase> CreateAnalyticsHandler(const FString& ImportFilePath);

private:

	static bool bIsCreatingSingleton;

	struct FQueuedTaskData
	{
		TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> AsyncHelper;
		UClass* TranslatorClass = nullptr;
	};

	TSoftClassPtr<UInterchangeAnalyticsHandlerBase> AnalyticsHandlerClass;

	TMap<UClass*, bool> NonParallelTranslatorLocks;
	TMap<UClass*, TArray<FQueuedTaskData>> NonParallelTranslatorQueueTasks;
	
	//Queue all incoming tasks if there are more started task then we have cores.
	TQueue<FQueuedTaskData> QueuedTasks;
	int32 QueueTaskCount = 0;

	//Queue all incoming tasks that are not directly import or re-import of assets.
	// The post import tasks are executed only when all the QueuedTasks are completed and empty.
	//Those tasks can start an import task (QueuedTasks), one example is the skeletal mesh alternate skinning which requires to import several files for each profile.
	TQueue<TSharedPtr<FInterchangePostImportTask>> QueuedPostImportTasks;

	//Ticker which is on only if we have some QueuedPostImportTasks
	FTSTicker::FDelegateHandle	QueuedPostImportTasksTickerHandle;

	/** Critical section to make the import tasks array thread safe */
	mutable FCriticalSection ImportTasksLock;
	//By using pointer, there is no issue if the array get resized.
	UE_DEPRECATED(5.6, "Use the provided API to manipulate the Import tasks array. This array need to be thread safe")
	TArray<TSharedPtr<UE::Interchange::FImportAsyncHelper, ESPMode::ThreadSafe> > ImportTasks;

	TSharedPtr<FAsyncTaskNotification> Notification = nullptr;
	FTSTicker::FDelegateHandle NotificationTickHandle;

	// Caching the registered translator classes to avoid double registration.
	UPROPERTY()
	TSet<TObjectPtr<const UClass>> RegisteredTranslatorsClass;

	//The manager will create translator at every import, translator must be able to retrieve payload information when the factory ask for it.
	//The translator stored has value is only use to know if we can use this type of translator.
//	UPROPERTY()
//	TArray<TObjectPtr<UInterchangeTranslatorBase>> RegisteredTranslators;
	
	//The manager will create only one pipeline per type
//	UPROPERTY()
//	TMap<TObjectPtr<const UClass>, TObjectPtr<UInterchangePipelineBase> > RegisteredPipelines;

	//The manager will create only one factory per type.
	UPROPERTY()
	TMap<TObjectPtr<const UClass>, TObjectPtr<const UClass> > RegisteredFactoryClasses;

	//The manager will create only one writer per type.
	UPROPERTY()
	TMap<TObjectPtr<const UClass>, TObjectPtr<UInterchangeWriterBase> > RegisteredWriters;

	//The manager will create only one converter per type.
	UPROPERTY()
	TMap<TObjectPtr<const UClass>, TObjectPtr<UInterchangeAssetImportDataConverterBase> > RegisteredConverters;

	//We support one editor utilities class
	TStrongObjectPtr<UInterchangeEditorUtilitiesBase> EditorUtilities = nullptr;

	//If interchange is currently importing, we have a timer to watch the cancel and we block GC.
	FThreadSafeBool bIsActive = false;

	//If the user wants to use the same import pipeline stack for all queued tasks.
	//This map is reset when the ImportTasks array is empty.
	TMap<UClass*, TArray<UInterchangePipelineBase*>> ImportAllWithSamePipelines;

	//Indicates that the import process was canceled by the user.
	//This Boolean is reset to false when the ImportTasks array is empty.
	bool bImportCanceled = false;

	//We want to avoid starting an import task during a GC.
	FDelegateHandle GCEndDelegate;
	FDelegateHandle GCPreDelegate;
	bool bGCEndDelegateCancellAllTask = false;

	friend class UE::Interchange::FScopedTranslator;
	// #todo_interchange: Remove when Interchange utility method for pipelines is done
	friend class UInterchangeAssetImportData;
};

#undef UE_API