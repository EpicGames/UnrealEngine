// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Async/Future.h"
#include "InterchangeFactoryBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"

#include "InterchangeMeshUtilities.generated.h"

#define UE_API INTERCHANGEENGINE_API

class UInterchangeSourceData;
class USkeletalMesh;

class FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask : public FInterchangePostImportTask
{
public:
	UE_API FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask(USkeletalMesh* InSkeletalMesh);

	virtual ~FInterchangeSkeletalMeshAlternateSkinWeightPostImportTask() = default;

	UE_API virtual void Execute() override;

	// This delegate should execute the following editor function (since we are a runtime module we need this delegate to call the editor function)
	// FSkinWeightsUtilities::ReimportAlternateSkinWeight(SkeletalMesh, LodIndex);
	DECLARE_DELEGATE_RetVal_TwoParams(bool, FInterchangeReimportAlternateSkinWeight, USkeletalMesh*, int32 LodIndex);
	FInterchangeReimportAlternateSkinWeight ReimportAlternateSkinWeightDelegate;

	UE_API bool AddLodToReimportAlternate(int32 LodToAdd);
	
private:
	TObjectPtr<USkeletalMesh> SkeletalMesh;

	//Alternate skin weights reimport
	TArray<int32> ReImportAlternateSkinWeightsLods;
};

UCLASS(BlueprintType, MinimalAPI)
class UInterchangeMeshUtilities : public UObject
{
	GENERATED_BODY()
public:

	/**
	 * This function pop a file picker with all meshes supported format.
	 *
	 * @param OutFilename - The filename the user pick.
	 * #param Title - The picker dialog title.
	 * @Return - return true if the users successfully pick a file, false if something went wrong or the user cancel.
	 *
	 */
	static INTERCHANGEENGINE_API bool ShowMeshFilePicker(FString& OutFilename, const FText& Title);

	/**
	 * This function import a mesh from the source data and add/replace the MeshObject LOD (at LodIndex) with the imported mesh LOD data.
	 *
	 * @Param MeshObject - The Mesh we want to add the lod
	 * @Param LodIndex - The index of the LOD we want to replace or add
	 * @Param SourceData - The source to import the custom LOD
	 * @param bAsync - If true the future will not be set when the function return and the import will be asynchronous.
	 * @Return - return future boolean which will be true if it successfully add or replace the MeshObject LOD at LodIndex with the imported data.
	 *
	 */
	static INTERCHANGEENGINE_API TFuture<bool> ImportCustomLod(UObject* MeshObject, const int32 LodIndex, const UInterchangeSourceData* SourceData, bool bAsync);

	/**
	 * This function import a morph target from the source data and add/replace the skeletal mesh morph target.
	 *
	 * @Param SkeletalMesh - The target skeletal mesh we want to add the morph targets
	 * @Param LodIndex - The index of the LOD we want to replace or add the morph targets
	 * @Param SourceData - The source to import the morph targets
	 * @Param bAsync - If true the future will not be set when the function return and the import will be asynchronous.
	 * @Param MorphTargetName - If not empty we will use this name to create the morph target, if there is already an existing morph target it will be re-import
	 * @Return - return future boolean which will be true if it successfully add or replace the skeletal mesh morph target at LodIndex with the imported data.
	 *
	 */
	static INTERCHANGEENGINE_API TFuture<bool> ImportMorphTarget(USkeletalMesh* SkeletalMesh, const int32 LodIndex, const UInterchangeSourceData* SourceData, bool bAsync, const FString& MorphTargetName);

	/**
	 * This function import a morph target from the source data and add/replace the skeletal mesh morph target.
	 *
	 * @Param SkeletalMesh - The target skeletal mesh we want to add the morph targets
	 * @Param LodIndex - The index of the LOD we want to replace or add the morph targets
	 * @Param SourceData - The source to import the morph targets
	 * @Param MorphTargetName - If not empty we will use this name to create the morph target, if there is already an existing morph target it will be re-import
	 * @Return - return true if it successfully add or replace the skeletal mesh morph target at LodIndex, flase otherwise.
	 *
	 */
	UFUNCTION(BlueprintCallable, Category = "SkeletalMesh | MorphTarget")
	bool ScriptedImportMorphTarget(USkeletalMesh* SkeletalMesh, const int32 LodIndex, const UInterchangeSourceData* SourceData, const FString& MorphTargetName) const
	{
		constexpr bool bAsyncFalse = false;
		return ImportMorphTarget(SkeletalMesh, LodIndex, SourceData, bAsyncFalse, MorphTargetName).Get();
	}


private:

	static TFuture<bool> InternalImportCustomLod(TSharedPtr<TPromise<bool>> Promise, UObject* MeshObject, const int32 LodIndex, const UInterchangeSourceData* SourceData, bool bAsync);
};


#undef UE_API
