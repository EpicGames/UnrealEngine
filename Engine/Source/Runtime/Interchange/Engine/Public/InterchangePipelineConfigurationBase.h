// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "CoreMinimal.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"
#include "InterchangeSourceData.h"
#include "InterchangeTranslatorBase.h"
#include "Nodes/InterchangeBaseNodeContainer.h"
#include "Misc/Optional.h"

#include "InterchangePipelineConfigurationBase.generated.h"

UENUM(BlueprintType)
enum class EInterchangePipelineConfigurationDialogResult : uint8
{
	Cancel		UMETA(DisplayName = "Cancel"),
	Import		UMETA(DisplayName = "Import"),
	ImportAll	UMETA(DisplayName = "Import All"),
	SaveConfig	UMETA(DisplayName = "Save Config"),
};

USTRUCT(BlueprintType)
struct FInterchangeStackInfo
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Translator")
	FName StackName;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Interchange | Translator")
	TArray<TObjectPtr<UInterchangePipelineBase>> Pipelines;
};

UCLASS(BlueprintType, Blueprintable, MinimalAPI)
class UInterchangePipelineConfigurationBase : public UObject
{
	GENERATED_BODY()

public:

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement ShowPipelineConfigurationDialog().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Translator")
	INTERCHANGEENGINE_API EInterchangePipelineConfigurationDialogResult ScriptedShowPipelineConfigurationDialog(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ShowPipelineConfigurationDialog(). */
	EInterchangePipelineConfigurationDialogResult ScriptedShowPipelineConfigurationDialog_Implementation(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		//By default we call the virtual import pipeline execution
		FPipelineConfigurationDialogParams DialogParams{
			.PipelineStacks = PipelineStacks,
			.OutPipelines = OutPipelines,
			.SourceData = SourceData,
			.Translator = Translator,
			.BaseNodeContainer = BaseNodeContainer,
			.ReimportAsset = nullptr,
			.bReimport = false,
			.bSceneImport = false,
			.bInvokedThroughTestPlan = false,
		};
		return ShowPipelineDialog_Internal(DialogParams);
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement ShowScenePipelineConfigurationDialog().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Translator")
	INTERCHANGEENGINE_API EInterchangePipelineConfigurationDialogResult ScriptedShowScenePipelineConfigurationDialog(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ShowScenePipelineConfigurationDialog(). */
	EInterchangePipelineConfigurationDialogResult ScriptedShowScenePipelineConfigurationDialog_Implementation(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer)
	{
		//By default we call the virtual import pipeline execution
		FPipelineConfigurationDialogParams DialogParams{
			.PipelineStacks = PipelineStacks,
			.OutPipelines = OutPipelines,
			.SourceData = SourceData,
			.Translator = Translator,
			.BaseNodeContainer = BaseNodeContainer,
			.ReimportAsset = nullptr,
			.bReimport = false,
			.bSceneImport = true,
			.bInvokedThroughTestPlan = false,
		};

		return ShowPipelineDialog_Internal(DialogParams);
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement ShowReimportPipelineConfigurationDialog().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Translator")
	INTERCHANGEENGINE_API EInterchangePipelineConfigurationDialogResult ScriptedShowReimportPipelineConfigurationDialog(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer
		, UObject* ReimportAsset
		, bool bSceneImport = false);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ShowReimportPipelineConfigurationDialog(). */
	EInterchangePipelineConfigurationDialogResult ScriptedShowReimportPipelineConfigurationDialog_Implementation(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer
		, UObject* ReimportAsset
		, bool bSceneImport = false)
	{
		//By default we call the virtual import pipeline execution
		FPipelineConfigurationDialogParams DialogParams{
			.PipelineStacks = PipelineStacks,
			.OutPipelines = OutPipelines,
			.SourceData = SourceData,
			.Translator = Translator,
			.BaseNodeContainer = BaseNodeContainer,
			.ReimportAsset = ReimportAsset,
			.bReimport = true,
			.bSceneImport = bSceneImport,
			.bInvokedThroughTestPlan = false,
		};

		return ShowPipelineDialog_Internal(DialogParams);
	}

	/**
	 * Non-virtual helper that allows Blueprint to implement an event-based function to implement ShowTestPlanPipelineConfigurationDialog().
	 */
	UFUNCTION(BlueprintNativeEvent, BlueprintCallable, Category = "Interchange | Translator")
	INTERCHANGEENGINE_API EInterchangePipelineConfigurationDialogResult ScriptedShowTestPlanConfigurationDialog(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer
		, UObject* ReimportAsset
		, bool bSceneImport = false
		, bool bReimport = false);

	/** The default implementation, which is called if the Blueprint does not have any implementation, calls the virtual ShowTestPlanAssetPipelineConfigurationDialog(). */
	EInterchangePipelineConfigurationDialogResult ScriptedShowTestPlanConfigurationDialog_Implementation(TArray<FInterchangeStackInfo>& PipelineStacks
		, TArray<UInterchangePipelineBase*>& OutPipelines
		, UInterchangeSourceData* SourceData
		, UInterchangeTranslatorBase* Translator
		, UInterchangeBaseNodeContainer* BaseNodeContainer
		, UObject* ReimportAsset
		, bool bSceneImport = false
		, bool bReimport = false)
	{
		//By default we call the virtual import pipeline execution
		FPipelineConfigurationDialogParams DialogParams{
			.PipelineStacks = PipelineStacks,
			.OutPipelines = OutPipelines,
			.SourceData = SourceData,
			.Translator = Translator,
			.BaseNodeContainer = BaseNodeContainer,
			.ReimportAsset = ReimportAsset,
			.bReimport = bReimport,
			.bSceneImport = bSceneImport,
			.bInvokedThroughTestPlan = true,
		};

		return ShowPipelineDialog_Internal(DialogParams);
	}

protected:
	struct FPipelineConfigurationDialogParams
	{
		TArray<FInterchangeStackInfo>& PipelineStacks;
		TArray<UInterchangePipelineBase*>& OutPipelines;
		TWeakObjectPtr<UInterchangeSourceData> SourceData;
		TWeakObjectPtr <UInterchangeTranslatorBase> Translator;
		TWeakObjectPtr <UInterchangeBaseNodeContainer> BaseNodeContainer;
		TWeakObjectPtr <UObject> ReimportAsset;
		bool bReimport;
		bool bSceneImport;
		bool bInvokedThroughTestPlan;
		TOptional<bool> bOverrideDefaultShowEssentials;
		TOptional<bool> bOverrideDefaultFilterOnContent;
	};

	virtual EInterchangePipelineConfigurationDialogResult ShowPipelineDialog_Internal(FPipelineConfigurationDialogParams& InParams)
	{
		//Not implemented
		return EInterchangePipelineConfigurationDialogResult::Cancel;
	}
};
