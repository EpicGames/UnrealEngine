// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Engine/DeveloperSettings.h"
#include "Engine/EngineTypes.h"
#include "InterchangeFilePickerBase.h"
#include "InterchangePipelineBase.h"
#include "InterchangePipelineConfigurationBase.h"
#include "UObject/SoftObjectPath.h"

#include "InterchangeProjectSettings.generated.h"

#define UE_API INTERCHANGEENGINE_API

class UInterchangeSourceData;

USTRUCT(BlueprintType)
struct FInterchangeTranslatorPipelines
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TranslatorPipelines", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangeTranslatorBase"))
	TSoftClassPtr<UInterchangeTranslatorBase> Translator;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TranslatorPipelines", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangePipelineBase, /Script/InterchangeEngine.InterchangeBlueprintPipelineBase, /Script/InterchangeEngine.InterchangePythonPipelineAsset"))
	TArray<FSoftObjectPath> Pipelines;
};

USTRUCT(BlueprintType)
struct FInterchangePipelineStack
{
	GENERATED_BODY()

	/** The list of pipelines in this stack. The pipelines are executed in fixed order, from top to bottom. */
	UPROPERTY(EditAnywhere, Category = "Pipelines", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangePipelineBase, /Script/InterchangeEngine.InterchangeBlueprintPipelineBase, /Script/InterchangeEngine.InterchangePythonPipelineAsset"))
	TArray<FSoftObjectPath> Pipelines;

	/** Specifies a different list of pipelines for this stack to use when importing data from specific translators. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "TranslatorPipelines")
	TArray<FInterchangeTranslatorPipelines> PerTranslatorPipelines;
};

USTRUCT(BlueprintType)
struct FInterchangePerTranslatorDialogOverride
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangeTranslatorBase"))
	TSoftClassPtr<UInterchangeTranslatorBase> Translator;

	/** Show the options dialog when Interchange imports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowImportDialog = true;

	/** Show the options dialog when Interchange reimports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowReimportDialog = false;
};

USTRUCT(BlueprintType)
struct FInterchangeDialogOverride
{
	GENERATED_BODY()

	/** Show the options dialog when Interchange imports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowImportDialog = true;

	/** Show the options dialog when Interchange reimports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowReimportDialog = false;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	TArray<FInterchangePerTranslatorDialogOverride> PerTranslatorImportDialogOverride;
};

USTRUCT(BlueprintType)
struct FInterchangeImportSettings
{
	GENERATED_BODY()

	/** Configures the pipeline stacks that are available when importing assets with Interchange. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline")
	TMap<FName, FInterchangePipelineStack> PipelineStacks;

	/** Specifies which pipeline stack Interchange should use by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline")
	FName DefaultPipelineStack = NAME_None;

	/** Specifies the class that should be used to define the configuration dialog that Interchange shows on import. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline")
	TSoftClassPtr <UInterchangePipelineConfigurationBase> ImportDialogClass;

	/** Show the options dialog when Interchange imports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowImportDialog = true;

	/** Show the options dialog when Interchange reimports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowReimportDialog = false;
};

USTRUCT(BlueprintType)
struct FInterchangeSceneImportSettings : public FInterchangeImportSettings
{
	GENERATED_BODY()

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride", Meta=(DisplayAfter="bShowReimportDialog"))
	TArray<FInterchangePerTranslatorDialogOverride> PerTranslatorDialogOverride;
};

USTRUCT(BlueprintType)
struct FInterchangeContentImportSettings : public FInterchangeImportSettings
{
	GENERATED_BODY()

	/** Specifies a different pipeline stack for Interchange to use by default when importing specific types of assets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline", Meta=(DisplayAfter="DefaultPipelineStack"))
	TMap<EInterchangeTranslatorAssetType, FName> DefaultPipelineStackOverride;

	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride", Meta=(DisplayAfter="bShowReimportDialog"))
	TMap<EInterchangeTranslatorAssetType, FInterchangeDialogOverride> ShowImportDialogOverride;
};

USTRUCT(BlueprintType)
struct FInterchangeGroup
{
	GENERATED_BODY()

	enum EUsedGroupStatus : uint8
	{
		NotSet,
		SetAndValid,
		SetAndInvalid
	};

	/** Specifies a different pipeline stack for Interchange to use by default when importing specific types of assets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "InterchangeGroup")
	FName DisplayName;

	/** This tell interchange if the import dialog should show or not when importing a particular type of asset.*/
	UPROPERTY(VisibleAnywhere, BlueprintReadWrite, Category = "InterchangeGroup", meta = (IgnoreForMemberInitializationTest, EditCondition = "false", EditConditionHides))
	FGuid UniqueID = FGuid::NewGuid();

	/** Specifies which pipeline stack Interchange should use by default. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline")
	FName DefaultPipelineStack = NAME_None;

	/** Specifies a different pipeline stack for Interchange to use by default when importing specific types of assets. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "Pipeline", Meta = (DisplayAfter = "DefaultPipelineStack"))
	TMap<EInterchangeTranslatorAssetType, FName> DefaultPipelineStackOverride;

	/** Show the options dialog when Interchange imports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowImportDialog = true;

	/** Show the options dialog when Interchange reimports. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	bool bShowReimportDialog = false;

	/** This tell interchange if the import dialog should show or not when importing a particular type of asset.*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, Category = "DialogOverride")
	TMap<EInterchangeTranslatorAssetType, FInterchangeDialogOverride> ShowImportDialogOverride;
};

UCLASS(config=Engine, BlueprintType, meta=(DisplayName=Interchange), MinimalAPI)
class UInterchangeProjectSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	/**
	 * Settings used when importing into the Content Browser.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "ImportContent")
	FInterchangeContentImportSettings ContentImportSettings;

	/**
	 * Settings used when importing into a level.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "ImportIntoLevel")
	FInterchangeSceneImportSettings SceneImportSettings;

	/** This tells Interchange which file picker class to construct when we need to choose a file for a source. */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "EditorInterface")
	TSoftClassPtr <UInterchangeFilePickerBase> FilePickerClass;

	/**
	 * If enabled, both Interchange translators and the legacy import process smooth the edges of static meshes that don't contain smoothing information.
	 * If you have an older project that relies on leaving hard edges by default, you can disable this setting to preserve consistency with older assets.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Generic|ImportSettings")
	bool bStaticMeshUseSmoothEdgesIfSmoothingInformationIsMissing = true;

	/**
	 * Specifies which pipeline class Interchange should use when editor tools import or reimport an asset with base settings.
	 * Unreal Editor depends on this class to be set. You can only edit this property in the .ini file.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Editor Generic Pipeline Class")
	TSoftClassPtr <UInterchangePipelineBase> GenericPipelineClass;

	/**
	 * Optional, the pipeline asset converters will duplicate to create interchange import data pipeline.
	 * If not set, converters will duplicate a pipeline class CDO of there choice.
	 */
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Converters", meta = (AllowedClasses = "/Script/InterchangeCore.InterchangePipelineBase, /Script/InterchangeEngine.InterchangeBlueprintPipelineBase, /Script/InterchangeEngine.InterchangePythonPipelineAsset"))
	FSoftObjectPath ConverterDefaultPipeline;

	/**
	* Groups that define PerTransalatorPipelines that user can select to use.
	*/
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Groups", meta = (NoElementDuplicate))
	TArray<FInterchangeGroup> InterchangeGroups;

private:
#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};

class FInterchangeProjectSettingsUtils
{
public:
	static INTERCHANGEENGINE_API const FInterchangeImportSettings& GetImportSettings(const UInterchangeProjectSettings& InterchangeProjectSettings, const bool bIsSceneImport);
	static INTERCHANGEENGINE_API FInterchangeImportSettings& GetMutableImportSettings(UInterchangeProjectSettings& InterchangeProjectSettings, const bool bIsSceneImport);
	static INTERCHANGEENGINE_API const FInterchangeImportSettings& GetDefaultImportSettings(const bool bIsSceneImport);
	static INTERCHANGEENGINE_API FInterchangeImportSettings& GetMutableDefaultImportSettings(const bool bIsSceneImport);

	static INTERCHANGEENGINE_API FName GetDefaultPipelineStackName(const bool bIsSceneImport, const UInterchangeSourceData& SourceData);
	static INTERCHANGEENGINE_API void SetDefaultPipelineStackName(const bool bIsSceneImport, const UInterchangeSourceData& SourceData, const FName StackName);

	static INTERCHANGEENGINE_API bool ShouldShowPipelineStacksConfigurationDialog(const bool bIsSceneImport, const bool bReImport, const UInterchangeSourceData& SourceData);

	static INTERCHANGEENGINE_API const FInterchangeGroup& GetUsedGroup(FInterchangeGroup::EUsedGroupStatus& UsedGroupStatus);

	static INTERCHANGEENGINE_API TArray<FName> GetGroupNames();
};

UCLASS(MinimalAPI)
class UInterchangeProjectSettingsScript : public UObject
{
	GENERATED_BODY()
public:
	/**
	 * Pipeline stack store SoftObjectPath which are not supported in python, you can use this function to retrieve the object from FInterchangePipelineStack.
	 * 
	 * @note: It return duplicated pipeline objects
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Settings")
	static UE_API TArray<UInterchangePipelineBase*> GetPipelineArrayFromPipelineStack(const FInterchangePipelineStack& InterchangePipelineStack);

	/**
	 * Pipeline stack store SoftObjectPath which are not supported in python, you can use this function to retrieve the object from FInterchangeTranslatorPipelines.
	 * 
	 * @note: It return duplicated pipeline objects
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Settings")
	static UE_API TArray<UInterchangePipelineBase*> GetPipelineArrayFromTranslatorPipelines(const FInterchangeTranslatorPipelines& InterchangeTranslatorPipeline);

	/**
	 * Return the default settings pipeline stack for the specified source data.
	 * 
	 * @note: It return duplicated pipeline objects
	 */
	UFUNCTION(BlueprintCallable, Category = "Interchange Settings")
	static UE_API TArray<UInterchangePipelineBase*> GetPipelineStackFromSourceData(const bool bIsSceneImport, const UInterchangeSourceData* SourceData);
};

UCLASS(config = EditorPerProjectUserSettings, BlueprintType, meta = (DisplayName = Interchange), MinimalAPI)
class UInterchangeEditorSettings : public UDeveloperSettings
{
	GENERATED_BODY()

public:
	UE_DEPRECATED(5.6, "Use the dialog overrides on the Interchange category of the Project Settings if you want to control when the Interchange reimport dialog is shown")
	UPROPERTY(EditAnywhere, BlueprintReadWrite, config, Category = "Show Dialog", meta=(EditCondition = "false", ToolTip="Use the dialog overrides on the Interchange category of the Project Settings if you want to control when the Interchange reimport dialog is shown"))
	bool bShowImportDialogAtReimport = false;

	const FGuid& GetUsedGroupUID() const { return UsedGroupUID; }

	UFUNCTION(BlueprintCallable, Category = "Interchange | Groups")
	const FName& GetUsedGroupName() const { return UsedGroupName; }

	UFUNCTION(BlueprintCallable, Category = "Interchange | Groups")
	void SetUsedGroupName(const FName& InUsedGroupName);

	void UpdateUsedGroupName();

private:
	UPROPERTY(EditAnywhere, Transient, Category = "Group Used", meta = (GetOptions = "GetSelectableItems", AllowPrivateAccess = "true"))
	FName UsedGroupName;

	UPROPERTY(EditAnywhere, Category = "Group Used", config, meta = (IgnoreForMemberInitializationTest, EditCondition = "false", EditConditionHides))
	FGuid UsedGroupUID;

	UFUNCTION()
	TArray<FName> GetSelectableItems() const;

	void UpdateUsedGroupUIDFromGroupName();

#if WITH_EDITOR
	virtual void PostEditChangeProperty(FPropertyChangedEvent& PropertyChangedEvent) override;
	virtual void PostInitProperties() override;
#endif
};

#undef UE_API
