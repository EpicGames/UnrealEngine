// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "InterchangePipelineBase.h"
#include "UObject/Object.h"
#include "UObject/ObjectMacros.h"


#include "InterchangePythonPipelineBase.generated.h"

/*
 * This class represents a Python pipeline. It is used by the TSoftClassPtr of the PythonPipeline asset.
 *
 */
UCLASS(BlueprintType, Abstract, MinimalAPI)
class UInterchangePythonPipelineBase : public UInterchangePipelineBase
{
	GENERATED_BODY()
public:

#if WITH_EDITOR
	INTERCHANGEENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif
};


USTRUCT()
struct FPropertyData
{
	GENERATED_BODY()

	FString DefaultValue;
	bool bLocked = false;
};

/*
 * This class is a helper class for Python pipelines. It allows the class picker to filter the Content Browser instance we want to support
 * in the pipeline stack.
 *
 * Use this class factory to be able to create an instance of a Python pipeline in the Content Browser.
 * A Python pipeline does not have any locked properties, and all default values must be set in the Python script.
 * This restriction exists because Python classes are transient, so any assets created from a Python class cannot be saved.
 * 
 */
UCLASS(BlueprintType, MinimalAPI)
class UInterchangePythonPipelineAsset : public UObject
{
	GENERATED_BODY()

public:
	INTERCHANGEENGINE_API virtual void PostLoad() override;
#if WITH_EDITOR
	INTERCHANGEENGINE_API virtual void PostEditChangeProperty(struct FPropertyChangedEvent& PropertyChangedEvent) override;
#endif

	INTERCHANGEENGINE_API void GeneratePipeline();

	INTERCHANGEENGINE_API void SetupFromPipeline(const UInterchangePythonPipelineBase* PythonPipeline, const bool bRegeneratePipeline = true);

	/** The Python class we want to use as a pipeline. */
	UPROPERTY(EditAnywhere, Category = "Interchange|Python")
	TSoftClassPtr<UInterchangePythonPipelineBase> PythonClass;

	/** The transient pipeline we generate when we load the Python pipeline. */
	UPROPERTY(VisibleAnywhere, Transient, Category = "Interchange|Python")
	TObjectPtr<UInterchangePythonPipelineBase> GeneratedPipeline;

	UPROPERTY(VisibleAnywhere, Category = "Interchange|Python")
	FString JsonDefaultProperties;
};

