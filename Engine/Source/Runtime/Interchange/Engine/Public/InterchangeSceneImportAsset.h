// Copyright Epic Games, Inc. All Rights Reserved.

#pragma once

#include "Misc/Guid.h"
#include "Interfaces/Interface_AssetUserData.h"
#include "UObject/SoftObjectPath.h"

#include "InterchangeSceneImportAsset.generated.h"

#define UE_API INTERCHANGEENGINE_API

class UInterchangeAssetImportData;
class UInterchangeFactoryBaseNode;
class ULevel;

/*
 * Class to hold all the data required to properly re-import a level
 */
UCLASS(MinimalAPI)
class UInterchangeSceneImportAsset : public UObject, public IInterface_AssetUserData
{
	GENERATED_BODY()

	UE_API virtual ~UInterchangeSceneImportAsset();

public:
#if WITH_EDITORONLY_DATA
	/** Importing data and options used for this scene */
	UPROPERTY(EditAnywhere, Instanced, Category=ImportSettings)
	TObjectPtr<UInterchangeAssetImportData> AssetImportData;

	/** Array of user data stored with the asset */
	UPROPERTY()
	TArray< TObjectPtr<UAssetUserData> > AssetUserData;
#endif // #if WITH_EDITORONLY_DATA

	//~ Begin UObject Interface
	UE_API virtual void GetAssetRegistryTags(FAssetRegistryTagsContext Context) const override;
	UE_DEPRECATED(5.4, "Implement the version that takes FAssetRegistryTagsContext instead.")
	UE_API virtual void GetAssetRegistryTags(TArray<FAssetRegistryTag>& OutTags) const override;
	UE_API virtual void PostLoad() override;
	//~ End UObject Interface

	//~ Begin IInterface_AssetUserData Interface
	UE_API virtual void AddAssetUserData( UAssetUserData* InUserData ) override;
	UE_API virtual void RemoveUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass ) override;
	UE_API virtual UAssetUserData* GetAssetUserDataOfClass( TSubclassOf<UAssetUserData> InUserDataClass ) override;
	UE_API virtual const TArray<UAssetUserData*>* GetAssetUserDataArray() const override;
	//~ End IInterface_AssetUserData Interface

#if WITH_EDITOR
	UE_API void RegisterWorldRenameCallbacks();
#endif

	/** Updates the SceneObjects cache based on the node container stored in AssetImportData */
	UE_API void UpdateSceneObjects();

	/**
	 * Returns the UObject which asset path is '//PackageName.AssetName[:SubPathString]'.
	 * Returns nullptr if the asset which path is '//PackageName.AssetName[:SubPathString]' was not part of
	 * the level import cached in this UInterchangeSceneImportAsset.
	 * @param PackageName: Package path of the actual object to reimport
	 * @param AssetName: Asset name of the actual object to reimport
	 * @param SubPathString: Optional subobject name
	 */
	UE_API UObject* GetSceneObject(const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString()) const;

	UE_DEPRECATED(5.8, "This function is deprecated. Use the GetFactoryNodeFromPath() instead.")
	const UInterchangeFactoryBaseNode* GetFactoryNode(const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString()) const
	{
		return GetFactoryNodeFromPath(PackageName, AssetName, SubPathString);
	}

	UE_DEPRECATED(5.8, "This function is deprecated. Use the GetFactoryNodeFromPath() instead.")
	const UInterchangeFactoryBaseNode* GetFactoryNode(const FSoftObjectPath& ObjectPath) const
	{
		return GetFactoryNodeFromPath(ObjectPath);
	}

	/**
	 * Returns the factory node associated with the asset which path is '//PackageName.AssetName[:SubPathString]'.
	 * Returns nullptr if the asset which path is '//PackageName.AssetName[:SubPathString]' was not part of
	 * the level import cached in this UInterchangeSceneImportAsset.
	 * @param PackageName: Package path of the actual object to reimport
	 * @param AssetName: Asset name of the actual object to reimport
	 * @param SubPathString: Optional subobject name
	 */
	UE_API const UInterchangeFactoryBaseNode* GetFactoryNodeFromPath(const FString& PackageName, const FString& AssetName, const FString& SubPathString = FString()) const;
	
	UE_API const UInterchangeFactoryBaseNode* GetFactoryNodeFromPath(const FSoftObjectPath& ObjectPath) const;

	/**
	 * Returns the factory node associated with a Node UID.
	 * @param InNodeUniqueId: Unique ID of the Node
	 * @return Factory Node if available or nullptr. 
	 */
	UE_API const UInterchangeFactoryBaseNode* GetFactoryNodeFromUniqueID(const FString& InNodeUniqueId) const;

	/**
	 * Returns the array of factory nodes corresponding to the given class.
	 * @param Class: Class to be looked for
	 */
	UE_API TArray<const UInterchangeFactoryBaseNode*> GetFactoryNodesOfClass(const UClass* Class) const;

	UE_API void GetSceneSoftObjectPaths(TArray<FSoftObjectPath>& SoftObjectPaths) const;

	UE_API bool ContainsObject(const FSoftObjectPath& SoftObjectPath) const;

private:

#if WITH_EDITOR
	/** Called before a world is renamed */
	void OnPreWorldRename(UWorld* World, const TCHAR* InName, UObject* NewOuter, ERenameFlags Flags, bool& bShouldFailRename);
	
	/** Invoked when a world is successfully renamed. Used to track when a temporary 'Untitled' unsaved map is saved with a new name. */
	void OnPostWorldRename(UWorld* World);

	/** Members used to cache the path and names related to the world to be renamed*/
	bool bWorldRenameCallbacksRegistered = false;
	FString PreviousWorldPath;
	FString PreviousWorldName;
	FString PreviousLevelName;
#endif

#if WITH_EDITORONLY_DATA
	/**
	 * Cache to easily retrieve a factory node from an asset's/actor's path
	 * FSoftObjectPath stores the path of an imported object
	 * FString stores the unique id of the factory node associated with the imported object
	 */
	TMap< FSoftObjectPath, FString > SceneObjects;
#endif // #if WITH_EDITORONLY_DATA
};

#undef UE_API
